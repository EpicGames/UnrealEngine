{
    "FileVersion" : 3,
    "Version" : 1,
    "VersionName" : "0.1",
    "FriendlyName" : "Editor DataflowGraph",
    "Description" : "Editor Dataflow Graph",
    "Category": "Editor",
    "CreatedBy" : "Epic Games, Inc.",
    "CreatedByURL" : "https://epicgames.com",
    "DocsURL" : "",
    "MarketplaceURL" : "",
    "SupportURL" : "",
    "EnabledByDefault" : true,
    "CanContainContent" : true,
    "Installed" : false,
	"Modules": [
		{
			"Name": "DataflowEditor",
			"Type": "Editor",
			"LoadingPhase": "PostDefault"
		},
		{
			"Name": "DataflowEnginePlugin",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "DataflowNodes",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "BaseCharacterFXEditor",
			"Enabled": true,
			"TargetAllowList": [ "Editor" ]
		},
		{
			"Name": "ProceduralMeshComponent",
			"Enabled": true
		},
		{
			"Name": "GeometryProcessing",
			"Enabled": true
		},
		{
			"Name": "MeshModelingToolsetExp",
			"Enabled": true,
			"TargetAllowList": [ "Editor" ]
		},
		{
			"Name": "ModelingToolsEditorMode",
			"Enabled": true,
			"TargetAllowList": [ "Editor" ]
		},
		{
			"Name": "ChaosCaching",
			"Enabled": true
		},
		{
			"Name": "GeometryCache",
			"Enabled": true
		},
		{
			"Name": "EditorDataStorage",
			"Enabled": true
		},
		{
			"Name": "EditorDataStorageFeatures",
			"Enabled": true
		},
		{
			"Name": "MeshModelingToolset",
			"Enabled": true
		},
		{
			"Name": "SkeletalMeshModelingTools",
			"Enabled": true,
			"TargetAllowList": [ "Editor" ]
		}
	]
}
