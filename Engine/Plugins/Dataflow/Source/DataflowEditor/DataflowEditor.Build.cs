// Copyright Epic Games, Inc. All Rights Reserved.
using System.IO;

namespace UnrealBuildTool.Rules
{
	public class DataflowEditor : ModuleRules
	{
		public DataflowEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateIncludePaths.AddRange(
				new string[] {
					Path.Combine(GetModuleDirectory("GeometryCacheEd"), "Private"),
					Path.Combine(GetModuleDirectory("SkeletalMeshModelingTools"), "Private")
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"TedsOutliner",
					"SkeletalMeshModelingTools",
					"ContentBrowser",
				}
			);

			PublicIncludePathModuleNames.AddRange(
				new string[]
				{
					"SkeletonEditor"
				}
			);

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"AdvancedPreviewScene",
					"ApplicationCore",
					"AssetDefinition",
					"AssetTools",
					"AssetRegistry",
					"BaseCharacterFXEditor",
					"BlueprintGraph",
					"Chaos",
					"Core",
					"CoreUObject",
					"DataflowCore",
					"DataflowEngine",
					"DataflowEnginePlugin",
					"DataflowNodes",
					"DataflowSimulation",
					"DetailCustomizations",
					"DeveloperSettings",
					"DynamicMesh",
					"Engine",
					"EditorFramework",
					"EditorInteractiveToolsFramework",
					"EditorStyle",
					"EditorSubsystem",
					"GeometryCache",
					"GeometryCore",
					"GeometryCollectionEngine",
					"GeometryFramework",
					"GraphEditor",
					"InputCore",
					"ImageCore",
					"InteractiveToolsFramework",
					"LevelEditor",
					"MeshDescription",
					"MeshConversion",
					"MeshModelingTools",
					"MeshModelingToolsEditorOnly",
					"MeshModelingToolsEditorOnlyExp",
					"MeshModelingToolsExp",
					"ModelingComponentsEditorOnly",
					"ModelingComponents",
					"ModelingToolsEditorMode",
					"PlanarCut",
					"Projects",
					"PropertyEditor",
					"RenderCore",
					"RHI",
					"SceneOutliner",
					"TedsOutliner",
					"SharedSettingsWidgets",
					"SkeletonEditor",
					"Slate",
					"SlateCore",
					"StaticMeshDescription",
					"ToolMenus",
					"ToolWidgets",
					"TypedElementRuntime",
					"TypedElementFramework",
					"UnrealEd",
					"WorkspaceMenuStructure",
					"XmlParser",
					"EditorWidgets",
					"KismetWidgets",      // SScrubControlPanel
					"AnimGraph", 
					"ChaosCaching", // UAnimSingleNodeInstance
					"StructUtilsEditor",
					"MessageLog",
					"AppFramework",
					"SkeletalMeshDescription",
					"SequencerWidgets",
					"TimeManagement",
					"GeometryCacheEd",
					"WidgetRegistration",
				}
			);
		}
	}
}
