// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/AssetDefinition_DataflowAsset.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "ContentBrowserMenuContexts.h"
#include "ContentBrowserModule.h"
#include "Dataflow/DataflowAttachment.h"
#include "Dataflow/DataflowEditor.h"
#include "Dataflow/DataflowEditorModule.h"
#include "Dataflow/DataflowObject.h"
#include "Dataflow/DataflowInstance.h"
#include "Dataflow/DataflowTemplateRegistry.h"
#include "Dataflow/SDataflowTemplatePicker.h"
#include "Dataflow/DataflowSimulationScene.h"
#include "Dialog/SMessageDialog.h"
#include "Framework/Notifications/NotificationManager.h"
#include "IContentBrowserSingleton.h"
#include "Math/Color.h"
#include "ScopedTransaction.h"
#include "Misc/FileHelper.h"
#include "ThumbnailRendering/SceneThumbnailInfo.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Dataflow/DataflowEditorToolkit.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_DataflowAsset)


#define LOCTEXT_NAMESPACE "AssetActions_DataflowAsset"

bool bCanEditDataflow = false;
FAutoConsoleVariableRef CVarDataflowIsEditable(TEXT("p.Dataflow.IsEditable"), bCanEditDataflow, TEXT("Whether to allow edits of the dataflow [def:true]"));

namespace UE::DataflowAssetDefinitionHelpers
{
	static UDataflow* CreateNewDataflowAssetInteractive(const UObject* BoundAsset, TFunction<UDataflow* (UPackage* const NewPackage, FName NewAssetName)> CreateAssetFunction)
	{
		const UClass* const DataflowClass = UDataflow::StaticClass();

		FSaveAssetDialogConfig NewDataflowAssetDialogConfig;
		{
			const FString PackageName = BoundAsset->GetOutermost()->GetName();
			NewDataflowAssetDialogConfig.DefaultPath = FPackageName::GetLongPackagePath(PackageName);
			const FString AssetName = BoundAsset->GetName();
			NewDataflowAssetDialogConfig.DefaultAssetName = "DF_" + AssetName;
			NewDataflowAssetDialogConfig.AssetClassNames.Add(DataflowClass->GetClassPathName());
			NewDataflowAssetDialogConfig.ExistingAssetPolicy = ESaveAssetDialogExistingAssetPolicy::Disallow;
			NewDataflowAssetDialogConfig.DialogTitleOverride = LOCTEXT("NewDataflowAssetDialogTitle", "Save Dataflow Asset As");
		}

		FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");

		FString NewPackageName;
		FText OutError;
		for (bool bFilenameValid = false; !bFilenameValid; bFilenameValid = FFileHelper::IsFilenameValidForSaving(NewPackageName, OutError))
		{
			const FString AssetSavePath = ContentBrowserModule.Get().CreateModalSaveAssetDialog(NewDataflowAssetDialogConfig);
			if (AssetSavePath.IsEmpty())
			{
				return nullptr;
			}
			NewPackageName = FPackageName::ObjectPathToPackageName(AssetSavePath);
		}

		const FName NewAssetName(FPackageName::GetLongPackageAssetName(NewPackageName));
		UPackage* const NewPackage = CreatePackage(*NewPackageName);

		UDataflow* const NewDataflowAsset = CreateAssetFunction(NewPackage, NewAssetName);
		if (NewDataflowAsset)
		{
			NewDataflowAsset->MarkPackageDirty();

			// Notify the asset registry
			FAssetRegistryModule::AssetCreated(NewDataflowAsset);
		}
		return NewDataflowAsset;
	}

	// Return true if we should proceed, false if we should re-open the dialog
	bool CreateNewDataflowAsset(const UObject* BoundAsset, UObject*& OutDataflowAsset)
	{
		OutDataflowAsset = CreateNewDataflowAssetInteractive(
			BoundAsset,
			[](UPackage* const NewPackage, FName NewAssetName)
			{
				return NewObject<UDataflow>(NewPackage, UDataflow::StaticClass(), NewAssetName, RF_Public | RF_Standalone | RF_Transactional);
			});
		return (OutDataflowAsset != nullptr);
	}

	bool MakeShareableDataflowAsset(UObject* BoundAsset, UDataflow*& OutDataflowAsset)
	{
		const UDataflow* DataflowObject = UE::Dataflow::InstanceUtils::GetDataflowAssetFromObject(BoundAsset);
		if (!DataflowObject)
		{
			// fallback to simply create a new one
			UObject* OutObject = nullptr;
			if (CreateNewDataflowAsset(BoundAsset, OutObject))
			{
				OutDataflowAsset = Cast<UDataflow>(OutObject);
				return true;
			}
			return false;
		}

		// Is the asset is already a standalone shared asset then do nothing 
		if (DataflowObject->IsAsset())
		{
			return false;
		}

		IDataflowInstanceInterface* Interface = UE::Dataflow::InstanceUtils::GetDataflowInterfaceFromObject(BoundAsset);
		if (!Interface)
		{
			// the bound asset do not support embedding
			return false;
		}

		OutDataflowAsset = CreateNewDataflowAssetInteractive(
			BoundAsset,
			[&DataflowObject](UPackage* const NewPackage, FName NewAssetName)
			{
				UDataflow* const DuplicateDataflowObject = DuplicateObject<UDataflow>(DataflowObject, NewPackage, NewAssetName);
				if (DuplicateDataflowObject)
				{
					// Adjust flags, otherwise the bound Dataflow copy won't be showing in the package
					DuplicateDataflowObject->SetFlags(RF_Public | RF_Standalone);
				}
				return DuplicateDataflowObject;
			});

		if (OutDataflowAsset)
		{
			Interface->GetDataflowInstance().SetDataflowAsset(OutDataflowAsset);
		}

		return (OutDataflowAsset != nullptr);
	}

	bool EmbedDataflowAsset(UObject* BoundAsset, UDataflow*& OutDataflowAsset)
	{
		const UDataflow* ExistingDataflowAsset = UE::Dataflow::InstanceUtils::GetDataflowAssetFromObject(BoundAsset);
		if (ExistingDataflowAsset && !ExistingDataflowAsset->IsAsset())
		{
			// already embedded, nothing to do
			return false;
		}

		IDataflowInstanceInterface* Interface = UE::Dataflow::InstanceUtils::GetDataflowInterfaceFromObject(BoundAsset);
		if (!Interface)
		{
			// the bound asset do not support embedding
			return false;
		}

		OutDataflowAsset =
			ExistingDataflowAsset
			? DuplicateObject<UDataflow>(ExistingDataflowAsset, BoundAsset, TEXT("EmbeddedDataflow"))
			: NewObject<UDataflow>(BoundAsset, UDataflow::StaticClass(), TEXT("EmbeddedDataflow"), RF_Public | RF_Standalone | RF_Transactional)
			;

		if (OutDataflowAsset)
		{
			Interface->GetDataflowInstance().SetDataflowAsset(OutDataflowAsset);
		}
		return (OutDataflowAsset != nullptr);
	}

	// Return true if we should proceed, false if we should re-open the dialog
	bool OpenDataflowAsset(const UObject* Asset, UObject*& OutDataflowAsset)
	{
		const UClass* const DataflowClass = UDataflow::StaticClass();

		FOpenAssetDialogConfig NewDataflowAssetDialogConfig;
		{
			const FString PackageName = Asset->GetOutermost()->GetName();
			NewDataflowAssetDialogConfig.DefaultPath = FPackageName::GetLongPackagePath(PackageName);
			NewDataflowAssetDialogConfig.AssetClassNames.Add(DataflowClass->GetClassPathName());
			NewDataflowAssetDialogConfig.bAllowMultipleSelection = false;
			NewDataflowAssetDialogConfig.DialogTitleOverride = LOCTEXT("OpenDataflowAssetDialogTitle", "Open Dataflow Asset");
		}

		FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
		TArray<FAssetData> AssetData = ContentBrowserModule.Get().CreateModalOpenAssetDialog(NewDataflowAssetDialogConfig);

		if (AssetData.Num() == 1)
		{
			OutDataflowAsset = AssetData[0].GetAsset();
			return true;
		}

		return false;
	}

	// Return true if we should proceed, false if we should re-open the dialog
	bool NewOrOpenDialog(const UObject* Asset, UObject*& OutDataflowAsset)
	{
		TSharedRef<SMessageDialog> ConfirmDialog = SNew(SMessageDialog)
			.Title(FText(LOCTEXT("Dataflow_WindowTitle", "Create or Open Dataflow graph?")))
			.Message(LOCTEXT("Dataflow_WindowText", "This Asset currently has no Dataflow graph"))
			.Buttons({
				SMessageDialog::FButton(LOCTEXT("Dataflow_NewText", "Create new Dataflow")),
				SMessageDialog::FButton(LOCTEXT("Dataflow_OpenText", "Open existing Dataflow")),
				SMessageDialog::FButton(LOCTEXT("Dataflow_ContinueText", "Continue without Dataflow")),
				});

		const int32 ResultButtonIdx = ConfirmDialog->ShowModal();
		switch (ResultButtonIdx)
		{
		case 0:
			return CreateNewDataflowAsset(Asset, OutDataflowAsset);
		case 1:
			return OpenDataflowAsset(Asset, OutDataflowAsset);
		default:
			break;
		}

		return true;
	}

	/**
	 * Shows the Dataflow template picker dialog and applies the user's choice directly onto Asset:
	 *   - Template : duplicates the chosen template and embeds it into Asset (owned, no standalone flags).
	 *   - Existing asset : sets the selected shared UDataflow on Asset without duplication.
	 *   - None / Cancelled : leaves Asset unchanged.
	 *
	 * Always returns true - the picker is a single non-repeating step with no sub-dialogs.
	 */
	bool SetDataflowFromTemplatePicker(UObject* Asset, bool bShowDefaultBlankOption)
	{
		const bool bIsNotInteractive = GIsAutomationTesting || FApp::IsUnattended() || IsRunningCommandlet() || GIsRunningUnattendedScript;
		if (bIsNotInteractive)
		{
			return true;
		}

		if (!IsValid(Asset))
		{
			return true;
		}

		const TArray<FDataflowTemplateOption> Templates =
			FDataflowTemplateRegistry::Get().GetTemplateOptions(Asset->GetClass(), bShowDefaultBlankOption);

		const FDataflowPickerResult PickerResult = SDataflowTemplatePicker::ShowModal(
			Templates,
			FSlateApplication::Get().GetActiveTopLevelWindow());

		switch (PickerResult.Type)
		{
		case EDataflowPickerResult::Template:
		{
			// Wrap creation + assignment in a single transaction so undo reverts atomically.
			const FScopedTransaction Transaction(LOCTEXT("SetDataflowFromTemplate", "Set Dataflow From Template"));
			Asset->Modify();

			// Load the chosen template; nullptr means Blank (TemplateId == NAME_None).
			const FString TemplatePath = PickerResult.TemplateId.ToString();
			UDataflow* const Source = (TemplatePath.IsEmpty() || PickerResult.TemplateId.IsNone())
				? nullptr
				: LoadObject<UDataflow>(nullptr, *TemplatePath);

			// If an embedded Dataflow with the canonical name already exists on this asset,
			// using the same name again would cause DuplicateObject/NewObject to TRASH the
			// existing one to free the name. The Modify() snapshot we just took points at
			// that existing object. Trashing it leaves the snapshot referencing a Garbage
			// pointer, so undo can't restore a live previous Dataflow. We thus need to generate a unique
			// name when a collision would happen so the previous embedded stays alive.
			const FName DesiredName = TEXT("EmbeddedDataflow");
			const FName EmbedName = StaticFindObjectFast(UDataflow::StaticClass(), Asset, DesiredName) != nullptr
				? MakeUniqueObjectName(Asset, UDataflow::StaticClass(), DesiredName)
				: DesiredName;

			// Embed a duplicate of the template, or a new empty graph for Blank.
			// Embedded dataflows are owned by the asset - no standalone/public flags.
			UDataflow* Embedded = Source
				? DuplicateObject<UDataflow>(Source, Asset, EmbedName)
				: NewObject<UDataflow>(Asset, UDataflow::StaticClass(), EmbedName, RF_Transactional);

			if (Embedded)
			{
				if (Source)
				{
					Embedded->ClearFlags(RF_Public | RF_Standalone);
					Embedded->SetFlags(RF_Transactional);
				}
				UE::Dataflow::InstanceUtils::SetDataflowAssetOnObject(Embedded, Asset);
			}
			return true;
		}

		case EDataflowPickerResult::ExistingAsset:
		{
			// Reference the selected asset directly - no duplication.
			if (UDataflow* const SharedDataflow = Cast<UDataflow>(PickerResult.SelectedAsset.GetAsset()))
			{
				const FScopedTransaction Transaction(LOCTEXT("SetDataflowFromExistingAsset", "Set Dataflow From Existing Asset"));
				Asset->Modify();
				UE::Dataflow::InstanceUtils::SetDataflowAssetOnObject(SharedDataflow, Asset);
			}
			return true;
		}

		case EDataflowPickerResult::None:
		case EDataflowPickerResult::Cancelled:
		default:
			return true;
		}
	}

	// Create a new UDataflow if one doesn't already exist for the Asset
	UObject* NewOrOpenDataflowAsset(const UObject* Asset)
	{
		UObject* DataflowAsset = nullptr;
		bool bDialogDone = false;
		while (!bDialogDone)
		{
			bDialogDone = NewOrOpenDialog(Asset, DataflowAsset);
		}

		return DataflowAsset;
	}

	bool CanOpenDataflowAssetInEditor(const UObject* Asset)
	{
		const UDataflow* const DataflowAsset = Cast<UDataflow>(Asset);
		return DataflowAsset && (bCanEditDataflow || DataflowAsset->Type == EDataflowType::Simulation);
	}

	//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

	static void ExecuteEmbedDataflow(const TArray<FAssetData> Assets)
	{
		for (const FAssetData& AssetData: Assets)
		{
			if (UObject* SelectedAsset = AssetData.GetAsset())
			{
				UDataflow* NewDataflow = nullptr;
				if (UE::DataflowAssetDefinitionHelpers::EmbedDataflowAsset(SelectedAsset, NewDataflow))
				{
					SelectedAsset->Modify();
				}
			}
		}
	}

	static bool CanExecuteEmbedDataflow(const TArray<FAssetData> Assets)
	{
		for (const FAssetData& AssetData : Assets)
		{
			if (UObject* SelectedAsset = AssetData.GetAsset())
			{
				if (IDataflowInstanceInterface* Interface = UE::Dataflow::InstanceUtils::GetDataflowInterfaceFromObject(SelectedAsset))
				{
					if (UDataflow* DataflowObject = Interface->GetDataflowInstance().GetDataflowAsset())
					{
						if (DataflowObject->IsAsset())
						{
							return true;
						}
					}
				}
			}
		}
		return false;
	}

	//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

	static void ExecuteSetDataflow(const TArray<FAssetData> Assets)
	{
		for (const FAssetData& AssetData: Assets)
		{
			if (UObject* AssetBase = AssetData.GetAsset())
			{
				UE::DataflowAssetDefinitionHelpers::SetDataflowFromTemplatePicker(AssetBase);

				if (UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>())
				{
					// note : we cannot use CloseAllEditorsForAsset for the Dataflow Editor , the persona based editor seem to have an issue with calling this
					// and will crash if because of an order of destruction issue with the toolkit. the path to solve it is unclear and may be risky
					// As an alternative let's not reload the dataflow editor and leverage the in place dataflow asset change feature that already exists
					bool bDataflowEditorAlreadyOpened = false;

					TArray<IAssetEditorInstance*> EditorInstances = AssetEditorSubsystem->FindEditorsForAssetAndSubObjects(AssetBase);
					for (IAssetEditorInstance* EditorInstance : EditorInstances)
					{
						if (EditorInstance->GetEditorName() != "DataflowEditor")
						{
							EditorInstance->CloseWindow(EAssetEditorCloseReason::CloseAllEditorsForAsset);
						}
						else if (!bDataflowEditorAlreadyOpened && EditorInstance->GetEditorName() == "DataflowEditor")
						{
							if (FDataflowEditorToolkit* DataflowEditorToolkit = static_cast<FDataflowEditorToolkit*>(EditorInstance))
							{
								DataflowEditorToolkit->OnDataflowAssetChanged();
							}
							bDataflowEditorAlreadyOpened = true;
						}
					}
					AssetEditorSubsystem->OnAssetEditorRequestClose().Broadcast(AssetBase, EAssetEditorCloseReason::CloseAllEditorsForAsset);

					if (!bDataflowEditorAlreadyOpened)
					{
						// Open a new editor to be able to edit the asset with dataflow
						AssetEditorSubsystem->OpenEditorForAsset(AssetBase);
					}
				}
			}
		}
	}
	
	static bool CanExecuteSetDataflow(const TArray<FAssetData> Assets)
	{
		// only enable when one asset is selected because this will open a dialog box for each of them 
		return (Assets.Num() == 1);
	}

	//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

	static void ExecuteReEvaluateDataflow(const TArray<FAssetData> Assets)
	{
		for (const FAssetData& AssetData : Assets)
		{
			if (UObject* SelectedAsset = AssetData.GetAsset())
			{
				if (IDataflowInstanceInterface* Interface = UE::Dataflow::InstanceUtils::GetDataflowInterfaceFromObject(SelectedAsset))
				{
					if (Interface->GetDataflowInstance().UpdateOwnerAsset(/*bUpdateDependentAsset*/true))
					{
						SelectedAsset->Modify();
					}
				}
			}
		}
	}

	static bool CanReEvaluateDataflow(const TArray<FAssetData> Assets)
	{
		for (const FAssetData& AssetData : Assets)
		{
			if (UObject* SelectedAsset = AssetData.GetAsset())
			{
				if (IDataflowInstanceInterface* Interface = UE::Dataflow::InstanceUtils::GetDataflowInterfaceFromObject(SelectedAsset))
				{
					if (UDataflow* DataflowObject = Interface->GetDataflowInstance().GetDataflowAsset())
					{
						return true;
					}
				}
			}
		}
		return false;
	}

	//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

	static void ExecuteOpenDataflowEditor(const TArray<FAssetData> Assets)
	{
		if (UAssetEditorSubsystem* const AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>())
		{
			for (const FAssetData& AssetData : Assets)
			{
				if (UObject* SelectedAsset = AssetData.GetAsset())
				{
					if (UDataflowAttachment* Attachment = UE::Dataflow::InstanceUtils::GetDataflowAttachmentFromObject(SelectedAsset))
					{
						if (UDataflow* DataflowObject = Attachment->GetDataflowInstance().GetDataflowAsset())
						{
							if (UDataflowEditor* const AssetEditor = NewObject<UDataflowEditor>(AssetEditorSubsystem, NAME_None, RF_Transient))
							{
								// TODO(dataflow) : should probably read the setting from the attachment if available
								const FString PreviewActorClassPath = Attachment->GetPreviewActorPath();
								const TSubclassOf<AActor> PreviewActorClass = StaticLoadClass(AActor::StaticClass(), nullptr, PreviewActorClassPath, nullptr, LOAD_None, nullptr);
								AssetEditor->Initialize({ Attachment }, PreviewActorClass);
							}
						}
					}
				}
			}
		}
	}

	static bool CanOpenDataflowEditor(const TArray<FAssetData> Assets)
	{
		for (const FAssetData& AssetData : Assets)
		{
			if (UObject* SelectedAsset = AssetData.GetAsset())
			{
				if (UDataflowAttachment* Attachment = UE::Dataflow::InstanceUtils::GetDataflowAttachmentFromObject(SelectedAsset))
				{
					if (UDataflow* DataflowObject = Attachment->GetDataflowInstance().GetDataflowAsset())
					{
						return true;
					}
				}
			}
		}
		return false;
	}

	//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

	static void ExecuteCreateDerivedAsset(const TArray<FAssetData> Assets)
	{
		if (Assets.Num() == 1)
		{
			if (UObject* SelectedAsset = Assets[0].GetAsset())
			{
				if (UDataflow* DataflowAsset = UE::Dataflow::InstanceUtils::GetDataflowAssetFromObject(SelectedAsset))
				{
					const UClass* const AssetClass = SelectedAsset->GetClass();
					FSaveAssetDialogConfig NewDataflowAssetDialogConfig;
					{
						const FString PackageName = SelectedAsset->GetOutermost()->GetName();
						NewDataflowAssetDialogConfig.DefaultPath = FPackageName::GetLongPackagePath(PackageName);
						const FString AssetName = SelectedAsset->GetName();
						NewDataflowAssetDialogConfig.DefaultAssetName = AssetName + "_Derived";
						NewDataflowAssetDialogConfig.AssetClassNames.Add(AssetClass->GetClassPathName());
						NewDataflowAssetDialogConfig.ExistingAssetPolicy = ESaveAssetDialogExistingAssetPolicy::Disallow;
						NewDataflowAssetDialogConfig.DialogTitleOverride = LOCTEXT("SaveDerivedAssetAsDlg_Label", "Save Derived Asset As");
					}

					FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");

					FString NewPackageName;
					FText OutError;
					for (bool bFilenameValid = false; !bFilenameValid; bFilenameValid = FFileHelper::IsFilenameValidForSaving(NewPackageName, OutError))
					{
						const FString AssetSavePath = ContentBrowserModule.Get().CreateModalSaveAssetDialog(NewDataflowAssetDialogConfig);
						if (AssetSavePath.IsEmpty())
						{
							return;
						}
						NewPackageName = FPackageName::ObjectPathToPackageName(AssetSavePath);
					}

					const FName NewAssetName(FPackageName::GetLongPackageAssetName(NewPackageName));
					UPackage* const NewPackage = CreatePackage(*NewPackageName);

					UObject* const NewAsset = DuplicateObject<UObject>(SelectedAsset, NewPackage, NewAssetName);
					if (NewAsset)
					{
						// Make sure the override properties are up to date with the variables set in the Dataflow asset
						if (IDataflowInstanceInterface* Interface = UE::Dataflow::InstanceUtils::GetDataflowInterfaceFromObject(NewAsset))
						{
							Interface->GetDataflowInstance().SyncVariables();
						}
						NewAsset->SetFlags(RF_Public | RF_Standalone);
						NewAsset->MarkPackageDirty();

						// Notify the asset registry
						FAssetRegistryModule::AssetCreated(NewAsset);
					}
				}
			}
		}
	}

	static bool CanCreateDerivedAsset(const TArray<FAssetData> Assets)
	{
		if (Assets.Num() == 1)
		{
			if (UObject* SelectedAsset = Assets[0].GetAsset())
			{
				if (UDataflow* DataflowAsset = UE::Dataflow::InstanceUtils::GetDataflowAssetFromObject(SelectedAsset))
				{
					return (DataflowAsset->IsAsset() && DataflowAsset->ReferenceAsset == SelectedAsset);
				}
			}
		}
		return false;
	}

	//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

	namespace Private
	{
		static void FillMenu(FMenuBuilder& MenuBuilder, const TArray<FAssetData> Assets, UClass* AssetClass)
		{
			if (Assets.Num() == 1 && Assets[0].IsInstanceOf(AssetClass))
			{
				MenuBuilder.BeginSection("DataflowActionsMenu_Dataflow", LOCTEXT("MenuSection", "DataFlow Actions"));

				// Set Dataflow
				{
					FMenuEntryParams Params;
					Params.DirectActions.ExecuteAction = FExecuteAction::CreateStatic(&ExecuteSetDataflow, Assets);
					Params.DirectActions.CanExecuteAction = FCanExecuteAction::CreateStatic(&CanExecuteSetDataflow, Assets);
					Params.LabelOverride = LOCTEXT("SetDataflow_Label", "Set Dataflow...");
					Params.ToolTipOverride = LOCTEXT("SetDataflow_Tooltip", "Select and set a Dataflow graph for this asset. This will open the Dataflow template selection dialog.");
					Params.IconOverride = FSlateIcon(FAppStyle::GetAppStyleSetName(), "GraphEditor.EventGraph_16x");
					MenuBuilder.AddMenuEntry(Params);
				}

				// Create Derived Asset
				{
					FMenuEntryParams Params;
					Params.DirectActions.ExecuteAction = FExecuteAction::CreateStatic(&ExecuteCreateDerivedAsset, Assets);
					Params.DirectActions.IsActionVisibleDelegate = FCanExecuteAction::CreateStatic(&CanCreateDerivedAsset, Assets);
					Params.LabelOverride = LOCTEXT("CreateDerivedAsset_Label", "Create a derived asset...");
					Params.ToolTipOverride = LOCTEXT("CreateDerivedAsset_Tooltip", "Create a derived asset from this one. THe derived asset will share the same dataflow but won't be able to change the graph");
					Params.IconOverride = FSlateIcon(FAppStyle::GetAppStyleSetName(), "GraphEditor.EventGraph_16x");
					MenuBuilder.AddMenuEntry(Params);
				}

				// Embed Dataflow
				{
					FMenuEntryParams Params;
					Params.DirectActions.ExecuteAction = FExecuteAction::CreateStatic(&ExecuteEmbedDataflow, Assets);
					Params.DirectActions.CanExecuteAction = FCanExecuteAction::CreateStatic(&CanExecuteEmbedDataflow, Assets);
					Params.LabelOverride = LOCTEXT("EmbedDataflow_Label", "Embed Dataflow");
					Params.ToolTipOverride = LOCTEXT("EmbedDataflow_Tooltip", "Embed the external Dataflow graph in the current asset, the external asset will remain but will be disconnected");
					Params.IconOverride = FSlateIcon(FAppStyle::GetAppStyleSetName(), "GraphEditor.EventGraph_16x");
					MenuBuilder.AddMenuEntry(Params);
				}

				// Re-evaluate dataflow Dataflow
				{
					FMenuEntryParams Params;
					Params.DirectActions.ExecuteAction = FExecuteAction::CreateStatic(&ExecuteReEvaluateDataflow, Assets);
					Params.DirectActions.CanExecuteAction = FCanExecuteAction::CreateStatic(&CanReEvaluateDataflow, Assets);
					Params.LabelOverride = LOCTEXT("ReEvaluateDataflow_Label", "Re-Evaluate Dataflow");
					Params.ToolTipOverride = LOCTEXT("ReEvaluateDataflow_Tooltip", "Re-evaluate the associated Dataflow to regenerate the selected asset(s)");
					Params.IconOverride = FSlateIcon(FAppStyle::GetAppStyleSetName(), "GraphEditor.EventGraph_16x");
					MenuBuilder.AddMenuEntry(Params);
				}

				// Open Dataflow Editor ( only for asset using the Dataflow Attachment mechanism)
				{
					FMenuEntryParams Params;
					Params.DirectActions.ExecuteAction = FExecuteAction::CreateStatic(&ExecuteOpenDataflowEditor, Assets);
					Params.DirectActions.IsActionVisibleDelegate = FCanExecuteAction::CreateStatic(&CanOpenDataflowEditor, Assets);
					Params.LabelOverride = LOCTEXT("OpenDataflowEditor_Label", "Edit in Dataflow Editor");
					Params.ToolTipOverride = LOCTEXT("OpenDataflowEditor_Tooltip", "Open the dataflow editor to edit the corresponding Dataflow graph");
					Params.IconOverride = FSlateIcon(FAppStyle::GetAppStyleSetName(), "GraphEditor.EventGraph_16x");
					MenuBuilder.AddMenuEntry(Params);
				}

				MenuBuilder.EndSection();
			}
		}

		static TSharedRef<FExtender> ExtendAssetContextMenu(const TArray<FAssetData>& Assets, UClass* AssetClass)
		{
			TSharedRef<FExtender> Extender = MakeShared<FExtender>();
			Extender->AddMenuExtension("GetAssetActions", EExtensionHook::After, nullptr, FMenuExtensionDelegate::CreateStatic(&FillMenu, Assets, AssetClass));
			return Extender;
		}
	}

	FDelegateHandle RegisterDataflowAssetMenus(UClass* AssetClass)
	{
		FContentBrowserModule& ContentBrowser = FModuleManager::LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));
		TArray<FContentBrowserMenuExtender_SelectedAssets>& AssetMenuExtenders = ContentBrowser.GetAllAssetViewContextMenuExtenders();
		AssetMenuExtenders.Add(FContentBrowserMenuExtender_SelectedAssets::CreateStatic(&Private::ExtendAssetContextMenu, AssetClass));
		return AssetMenuExtenders.Last().GetHandle();
	}

	void UnregisterDataflowAssetMenus(FDelegateHandle Handle)
	{
		if (Handle.IsValid() && FModuleManager::Get().IsModuleLoaded(TEXT("ContentBrowser")))
		{
			FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));
			TArray<FContentBrowserMenuExtender_SelectedAssets>& AssetMenuExtenders = ContentBrowserModule.GetAllAssetViewContextMenuExtenders();

			AssetMenuExtenders.RemoveAll([Handle](const FContentBrowserMenuExtender_SelectedAssets& InDelegate)
				{
					return InDelegate.GetHandle() == Handle;
				});

			Handle.Reset();
		}
	}

	UObject* FactoryCreateNew(
		UClass* AssetClass,
		UObject* Parent,
		FName Name,
		EObjectFlags Flags,
		const FString* DataflowTemplatePath,
		bool bEmbedDataflow,
		const TCHAR* AssetPrefix)
	{
		UObject* const NewAsset = NewObject<UObject>(Parent, AssetClass, Name, Flags | RF_Transactional | RF_Public | RF_Standalone);
		if (!NewAsset)
		{
			FNotificationInfo NotificationInfo(FText::Format(LOCTEXT("ErrorCreatingNewAsset",
				"Failed to create a new asset {0}."), FText::FromName(Name)));
			NotificationInfo.ExpireDuration = 10.f;
			FSlateNotificationManager::Get().AddNotification(NotificationInfo);
			UE_LOGF(LogDataflowEditor, Error, "%ls", *NotificationInfo.Text.Get().ToString());
			return nullptr;
		}
		NewAsset->MarkPackageDirty();

		if (!DataflowTemplatePath || !ensureMsgf(AssetClass->ImplementsInterface(UDataflowInstanceInterface::StaticClass()), TEXT("Only use UE::DataflowAssetDefinitionHelpers::FactoryCreateNew() with Dataflow asset types.")))
		{
			return NewAsset;
		}

		UDataflow* DataflowTemplate = nullptr;
		if (!DataflowTemplatePath->IsEmpty())
		{
			DataflowTemplate = LoadObject<UDataflow>(nullptr, *DataflowTemplatePath);
			if (!DataflowTemplate)
			{
				FNotificationInfo NotificationInfo(FText::Format(LOCTEXT("ErrorLoadingTemplate",
					"Failed to load Dataflow template [{0}].\n"
					"An empty Dataflow will be used to create new asset {1}."), FText::FromString(*DataflowTemplatePath), FText::FromName(Name)));
				NotificationInfo.ExpireDuration = 10.f;
				FSlateNotificationManager::Get().AddNotification(NotificationInfo);
				UE_LOGF(LogDataflowEditor, Error, "%ls", *NotificationInfo.Text.Get().ToString());
			}
		}
		if (!DataflowTemplate)
		{
			DataflowTemplate = NewObject<UDataflow>();  // Use empty Dataflow instead
		}

		if (bEmbedDataflow)
		{
			// Duplicate it and embed it into the new asset
			if (UDataflow* const Dataflow = DuplicateObject(DataflowTemplate, NewAsset, TEXT("EmbeddedDataflow")))
			{
				// Clear standalone asset flags
				Dataflow->ClearFlags(RF_Public | RF_Standalone);

				// Mark as transactional
				Dataflow->SetFlags(RF_Transactional);

				// Set the Dataflow to the new asset
				CastChecked<IDataflowInstanceInterface>(NewAsset)->GetDataflowInstance().SetDataflowAsset(Dataflow);
			}
			else
			{
				FNotificationInfo NotificationInfo(FText::Format(LOCTEXT("ErrorEmbeddingDataflow",
					"Failed to embed a Dataflow into new asset {0}."), FText::FromName(Name)));
				NotificationInfo.ExpireDuration = 10.f;
				FSlateNotificationManager::Get().AddNotification(NotificationInfo);
				UE_LOGF(LogDataflowEditor, Error, "%ls", *NotificationInfo.Text.Get().ToString());
			}
		}
		else
		{
			// Create a new Dataflow asset
			static const TCHAR* DataflowAssetPrefix = TEXT("DF_");
			const FString DataflowPath = FPackageName::GetLongPackagePath(NewAsset->GetOutermost()->GetName());
			const FString OutfitAssetName = NewAsset->GetName();
			FString DataflowName = FString(DataflowAssetPrefix) +
				(AssetPrefix && OutfitAssetName.StartsWith(AssetPrefix) ? OutfitAssetName.RightChop(FCString::Strlen(AssetPrefix)) : OutfitAssetName);
			FString DataflowPackageName = FPaths::Combine(DataflowPath, DataflowName);
			if (FindPackage(nullptr, *DataflowPackageName))
			{
				// If a Dataflow asset already exists with this name, make a unique name from it to avoid clobbering it
				MakeUniqueObjectName(nullptr, UPackage::StaticClass(), FName(DataflowPackageName)).ToString(DataflowPackageName);
				DataflowName = FPaths::GetBaseFilename(DataflowPackageName);
			}
			if (UPackage* const DataflowPackage = CreatePackage(*DataflowPackageName))
			{
				// Load the outfit template into the new Dataflow asset
				if (UDataflow* const Dataflow = DuplicateObject(DataflowTemplate, DataflowPackage, FName(DataflowName)))
				{
					// Mark as a transactional asset and notify creation
					Dataflow->SetFlags(RF_Transactional | RF_Public | RF_Standalone);
					Dataflow->MarkPackageDirty();
					FAssetRegistryModule::AssetCreated(Dataflow);

					// Set the Dataflow to the new asset
					CastChecked<IDataflowInstanceInterface>(NewAsset)->GetDataflowInstance().SetDataflowAsset(Dataflow);
				}
				else
				{
					FNotificationInfo NotificationInfo(FText::Format(LOCTEXT("ErrorCreatingDataflow",
						"Failed to create a new Dataflow for new asset {0}."), FText::FromName(Name)));
					NotificationInfo.ExpireDuration = 10.f;
					FSlateNotificationManager::Get().AddNotification(NotificationInfo);
					UE_LOGF(LogDataflowEditor, Error, "%ls", *NotificationInfo.Text.Get().ToString());
				}
			}
			else
			{
				FNotificationInfo NotificationInfo(FText::Format(LOCTEXT("ErrorCreatingPackage",
					"Failed to create a new Dataflow package for new asset {0}."), FText::FromName(Name)));
				NotificationInfo.ExpireDuration = 10.f;
				FSlateNotificationManager::Get().AddNotification(NotificationInfo);
				UE_LOGF(LogDataflowEditor, Error, "%ls", *NotificationInfo.Text.Get().ToString());
			}
		}
		return NewAsset;
	}
}



namespace UE::Dataflow::DataflowAsset
{
	struct FColorScheme
	{
		static inline const FLinearColor Asset = FColor(180, 120, 110);
		static inline const FLinearColor NodeHeader = FColor(180, 120, 110);
		static inline const FLinearColor NodeBody = FColor(18, 12, 11, 127);
	};
}

FText UAssetDefinition_DataflowAsset::GetAssetDisplayName() const
{
	return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_DataflowAsset", "DataflowAsset");
}

TSoftClassPtr<UObject> UAssetDefinition_DataflowAsset::GetAssetClass() const
{
	return UDataflow::StaticClass();
}

FLinearColor UAssetDefinition_DataflowAsset::GetAssetColor() const
{
	return UE::Dataflow::DataflowAsset::FColorScheme::Asset;
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_DataflowAsset::GetAssetCategories() const
{
	static const TArray<FAssetCategoryPath> Categories = 
		{
			FAssetCategoryPath(EAssetCategoryPaths::Physics, LOCTEXT("DataflowAsset_SubMenu", "Dataflow"), ECategoryMenuType::Section)
		};
	return Categories;
}

UThumbnailInfo* UAssetDefinition_DataflowAsset::LoadThumbnailInfo(const FAssetData& InAsset) const
{
	return UE::Editor::FindOrCreateThumbnailInfo(InAsset.GetAsset(), USceneThumbnailInfo::StaticClass());
}

FAssetOpenSupport UAssetDefinition_DataflowAsset::GetAssetOpenSupport(const FAssetOpenSupportArgs& OpenSupportArgs) const
{
	return Super::GetAssetOpenSupport(OpenSupportArgs);
}


EAssetCommandResult UAssetDefinition_DataflowAsset::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	TArray<UDataflow*> DataflowObjects = OpenArgs.LoadObjects<UDataflow>();

	// For now the dataflow editor only works on one asset at a time
	ensure(DataflowObjects.Num() == 0 || DataflowObjects.Num() == 1);

	if (DataflowObjects.Num() == 1)
	{
		if (UE::DataflowAssetDefinitionHelpers::CanOpenDataflowAssetInEditor(DataflowObjects[0]))
		{
			UAssetEditorSubsystem* const AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
			UDataflowEditor* const AssetEditor = NewObject<UDataflowEditor>(AssetEditorSubsystem, NAME_None, RF_Transient);
			AssetEditor->RegisterToolCategories({"General"});

			// Validate the asset
			if (UDataflow* const DataflowAsset = CastChecked<UDataflow>(DataflowObjects[0]))
			{
				AssetEditor->Initialize({ DataflowAsset });
				return EAssetCommandResult::Handled;
			}
		}
		else
		{
			TSharedRef<SMessageDialog> MessageDialog = SNew(SMessageDialog)
				.Title(FText(LOCTEXT("Dataflow_OpenAssetDialog_Title", "Dataflow Asset")))
				.Message(LOCTEXT("Dataflow_OpenAssetDialog_Text", "Dataflow assets can only be changed while editing assets using them (Cloth, Flesh, Geometry Collection, ...)"))
				.Buttons({
					SMessageDialog::FButton(LOCTEXT("Ok", "Ok"))
					.SetPrimary(true)
				});
			MessageDialog->ShowModal();
			return EAssetCommandResult::Handled;
		}
	}
	return EAssetCommandResult::Unhandled;
}

void FDataflowConnectionData::Set(const FDataflowOutput& Output, const FDataflowInput& Input)
{
	static constexpr TCHAR Format[] = TEXT("/{0}:{1}|{2}");

	const FDataflowNode* OutputNode = Output.GetOwningNode();
	const FDataflowNode* InputNode = Input.GetOwningNode();

	Out = FString::Format(Format, { OutputNode ? OutputNode->GetName().ToString() : FString(), Output.GetName().ToString(), Output.GetType().ToString() });
	In = FString::Format(Format, { InputNode ? InputNode->GetName().ToString() : FString(), Input.GetName().ToString(), Input.GetType().ToString() });
}

FString FDataflowConnectionData::GetNode(const FString InConnection)
{
	FString Left, Right;

	if (InConnection.Split(TEXT(":"), &Left, nullptr, ESearchCase::IgnoreCase, ESearchDir::FromEnd))
	{
		if (Left.Split(TEXT("/"), nullptr, &Right, ESearchCase::IgnoreCase, ESearchDir::FromEnd))
		{
			return Right;
		}
	}

	return {};
}

FString FDataflowConnectionData::GetProperty(const FString InConnection)
{
	FString Left, Right;

	if (InConnection.Split(TEXT(":"), nullptr, &Right, ESearchCase::IgnoreCase, ESearchDir::FromEnd))
	{
		if (Right.Split(TEXT("|"), &Left, nullptr, ESearchCase::IgnoreCase, ESearchDir::FromEnd))
		{
			return Left;
		}
		else
		{
			return Right; // no types
		}
	}

	return {};
}

void FDataflowConnectionData::GetNodePropertyAndType(const FString InConnection, FString& OutNode, FString& OutProperty, FString& OutType)
{
	FString Left, Right;

	// String should look like "/NodeName:Property|Type"
	if (InConnection.Split(TEXT(":"), &Left, &Right, ESearchCase::IgnoreCase, ESearchDir::FromEnd))
	{
		// get the node name from "/NodeName"
		bool bTrimmed = false;
		OutNode = Left.TrimChar('/', &bTrimmed);

		checkf(bTrimmed, TEXT("Invalid node name received in GetNodePropertyAndType, Attempted to get node name from: %s"), *Left)

		// get the property and type from "Property|Type"
		const bool bHasType = Right.Split(TEXT("|"), &OutProperty, &OutType, ESearchCase::IgnoreCase, ESearchDir::FromEnd);
		if (!bHasType)
		{
			OutProperty = Right;
			OutType = {};
		}
	}
}


#undef LOCTEXT_NAMESPACE
