// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/AssetDefinition_DataflowContext.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "ContentBrowserModule.h"
#include "Dataflow/DataflowEditor.h"
#include "Dataflow/DataflowEditorModule.h"
#include "Dataflow/DataflowEditorUtil.h"
#include "Dataflow/DataflowContent.h"
#include "Dialog/SMessageDialog.h"
#include "IContentBrowserSingleton.h"
#include "Math/Color.h"
#include "Misc/FileHelper.h"
#include "ThumbnailRendering/SceneThumbnailInfo.h"
#include "UObject/UObjectGlobals.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_DataflowContext)


#define LOCTEXT_NAMESPACE "AssetActions_DataflowContext"


namespace UE::Dataflow::DataflowContext
{
	struct FColorScheme
	{
		static inline const FLinearColor Asset = FColor(180, 120, 110);
		static inline const FLinearColor NodeHeader = FColor(180, 120, 110);
		static inline const FLinearColor NodeBody = FColor(18, 12, 11, 127);
	};
}

FText UAssetDefinition_DataflowContext::GetAssetDisplayName() const
{
	return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_DataflowContext", "DataflowContext");
}

TSoftClassPtr<UObject> UAssetDefinition_DataflowContext::GetAssetClass() const
{
	return UDataflowBaseContent::StaticClass();
}

FLinearColor UAssetDefinition_DataflowContext::GetAssetColor() const
{
	return UE::Dataflow::DataflowContext::FColorScheme::Asset;
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_DataflowContext::GetAssetCategories() const
{
	static const auto Categories = { EAssetCategoryPaths::Physics };
	return Categories;
}

UThumbnailInfo* UAssetDefinition_DataflowContext::LoadThumbnailInfo(const FAssetData& InAsset) const
{
	return UE::Editor::FindOrCreateThumbnailInfo(InAsset.GetAsset(), USceneThumbnailInfo::StaticClass());
}

FAssetOpenSupport UAssetDefinition_DataflowContext::GetAssetOpenSupport(const FAssetOpenSupportArgs& OpenSupportArgs) const
{
	// this asset shoul dnot be editable at any time
	return FAssetOpenSupport(EAssetOpenMethod::View, false);
}

#undef LOCTEXT_NAMESPACE
