// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowCollectionSpreadSheetWidget.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Dataflow/DataflowSettings.h"
#include "GeometryCollection/GeometryCollection.h"
#include "Dataflow/DataflowCollectionSpreadSheetHelpers.h"
#include "Dataflow/DataflowEditorUtil.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SCheckBox.h"

#define LOCTEXT_NAMESPACE "CollectionSpreadSheet"

const FName FCollectionSpreadSheetHeader::IndexColumnName = FName("Index");
const FName FCollectionSpreadSheetHeader::NoColumnName = FName("[NoName]");

void SCollectionSpreadSheetRow::Construct(const FArguments& InArgs, TSharedRef<STableViewBase> OwnerTableView, const TSharedPtr<const FCollectionSpreadSheetHeader>& InHeader, const TSharedPtr<const FCollectionSpreadSheetItem>& InItem)
{
	Header = InHeader;
	Item = InItem;

	SMultiColumnTableRow<TSharedPtr<const FCollectionSpreadSheetItem>>::Construct(
		FSuperRowType::FArguments()
		.Style(&FStarshipCoreStyle::GetCoreStyle().GetWidgetStyle<FTableRowStyle>("TableView.AlternatingRow"))
		, OwnerTableView);
}


TSharedRef<SWidget> SCollectionSpreadSheetRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	int32 FoundIndex;
	if (Header->ColumnNames.Find(ColumnName, FoundIndex))
	{
		const FString& AttrValue = Item->Values[FoundIndex];

		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(FText::FromString(AttrValue))
				.ShadowColorAndOpacity(FLinearColor(0.1f, 0.1f, 0.1f, 1.f))
				.Visibility(EVisibility::Visible)
			];
	}

	return SNullWidget::NullWidget;
}

//
// ----------------------------------------------------------------------------
//

void SCollectionSpreadSheet::Construct(const FArguments& InArgs)
{
	SelectedOutput = InArgs._SelectedOutput;

	HeaderRowWidget = SNew(SHeaderRow)
		.Visibility(EVisibility::Visible);

	if (!CollectionInfoMap.IsEmpty())
	{
		RegenerateHeader();
		RepopulateListView();
	}

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.Padding(FMargin(0.0f, 3.f))
		[
			SNew(SOverlay)
			+ SOverlay::Slot()
			[
				SAssignNew(ListView, SListView<TSharedPtr<const FCollectionSpreadSheetItem>>)
				.SelectionMode(ESelectionMode::Multi)
				.ListItemsSource(&ListItems)
				.OnGenerateRow(this, &SCollectionSpreadSheet::GenerateRow)
				.HeaderRow(HeaderRowWidget)
				.ExternalScrollbar(InArgs._ExternalVerticalScrollBar)
			]
		]
	];
}


void SCollectionSpreadSheet::SetSelectedOutput(const FName& InSelectedOutput)
{
	SelectedOutput = InSelectedOutput;

	RegenerateHeader();
	RepopulateListView();
}


const FName& SCollectionSpreadSheet::GetSelectedOutput() const
{
	return SelectedOutput;
}

const FString SCollectionSpreadSheet::GetSelectedOutputStr() const
{
	return SelectedOutput.ToString();
}

void SCollectionSpreadSheet::SetSelectedGroup(const FName& InSelectedGroup)
{
	SelectedGroup = InSelectedGroup;

	RegenerateHeader();
	RepopulateListView();
}


const FName& SCollectionSpreadSheet::GetSelectedGroup() const
{
	return SelectedGroup;
}

void SCollectionSpreadSheet::RegenerateHeader()
{
	HeaderRowWidget->ClearColumns();

	Header = MakeShared<FCollectionSpreadSheetHeader>();
	TArray<FString> AttrTypes;

	if (CollectionInfoMap.Num() > 0 && !SelectedOutput.IsNone() && SelectedOutput.ToString() != "" &&
		!SelectedGroup.IsNone() && SelectedGroup.ToString() != "")
	{
		Header->ColumnNames.Add(FCollectionSpreadSheetHeader::IndexColumnName);

		if (const SCollectionInfo* CollectionInfo = CollectionInfoMap.Find(SelectedOutput.ToString()))
		{
			if (CollectionInfo->Collection.IsValid())
			{
				const TArray<FName> AttributeNames = CollectionInfo->Collection->AttributeNames(SelectedGroup);
				const int32 NumAttributes = AttributeNames.Num();
				for (int32 AttributeIndex = 0; AttributeIndex < NumAttributes; ++AttributeIndex)
				{
					// column cannot have empty names
					const FName AttributeName = AttributeNames[AttributeIndex].IsNone() ? FCollectionSpreadSheetHeader::NoColumnName : AttributeNames[AttributeIndex];
					Header->ColumnNames.Add(AttributeName);
					AttrTypes.Add(UE::Dataflow::CollectionSpreadSheetHelpers::GetArrayTypeString(CollectionInfo->Collection->GetAttributeType(AttributeName, SelectedGroup)).ToString());
				}
			}
		}
	}

	for (int32 IdxAttr = 0; IdxAttr < Header->ColumnNames.Num(); ++IdxAttr)
	{
		const FName& ColumnName = Header->ColumnNames[IdxAttr];
		FName ToolTip;

		FString ColumnNameStr = ColumnName.ToString();
		FString AttrTypeStr;

		if (IdxAttr > 0)
		{
			ToolTip = FName(*FString::Printf(TEXT("Attr: %s\nType: %s"), *ColumnName.ToString(), *AttrTypes[IdxAttr - 1])); // IdxAttr needs to be adjusted because of the first Index column

			AttrTypeStr = AttrTypes[IdxAttr - 1];
		}
		else
		{
			ToolTip = FName("");
		}

		int32 ColumnWidth = 100;
		if (ColumnNameStr == "Index")
		{
			ColumnWidth = 100;
		}
		else
		{
			int32 ColumnNameStrLen = ColumnNameStr.Len() * 9;
			int32 AttrTypeWidth = 100;
			if (UE::Dataflow::CollectionSpreadSheetHelpers::AttrTypeWidthMap.Contains(AttrTypeStr))
			{
				AttrTypeWidth = UE::Dataflow::CollectionSpreadSheetHelpers::AttrTypeWidthMap[AttrTypeStr];
			}
			ColumnWidth = ColumnNameStrLen > AttrTypeWidth ? ColumnNameStrLen : AttrTypeWidth;
		}

		HeaderRowWidget->AddColumn(
			SHeaderRow::Column(ColumnName)
			.DefaultLabel(FText::FromName(ColumnName))
			.DefaultTooltip(FText::FromName(ToolTip))
			.ManualWidth(ColumnWidth)
			//			.SortMode(EColumnSizeMode)
			.HAlignCell(HAlign_Center)
			.HAlignHeader(HAlign_Center)
			.VAlignCell(VAlign_Center)
		);
	}
}

void SCollectionSpreadSheet::RepopulateListView()
{
	ListItems.Empty();

	if (CollectionInfoMap.Num() > 0 &&
		!SelectedOutput.IsNone() && SelectedOutput.ToString() != "" &&
		!SelectedGroup.IsNone() && SelectedGroup.ToString() != "")
	{
		if (const SCollectionInfo* CollectionInfo = CollectionInfoMap.Find(SelectedOutput.ToString()))
		{
			if (CollectionInfo->Collection.IsValid())
			{
				const int32 NumElems = CollectionInfo->Collection->NumElements(SelectedGroup);
				ListItems.SetNum(NumElems);

				constexpr int32 BatchSize = 50;
				ParallelFor(TEXT("ScollectionSpreadSheet_GenerateItems"), NumElems, BatchSize,
					[this, CollectionPtr = CollectionInfo->Collection](int32 IdxElem)
					{
						const TSharedPtr<FCollectionSpreadSheetItem> NewItem = MakeShared<FCollectionSpreadSheetItem>();
						NewItem->Values.SetNum(Header->ColumnNames.Num());

						for (int32 IdxColumn = 0; IdxColumn < Header->ColumnNames.Num(); ++IdxColumn)
						{
							const FName& ColumnName = Header->ColumnNames[IdxColumn];
							if (ColumnName == FCollectionSpreadSheetHeader::IndexColumnName)
							{
								NewItem->Values[IdxColumn] = FString::FromInt(IdxElem);
							}
							else
							{
								const FName AttributeName = (ColumnName == FCollectionSpreadSheetHeader::NoColumnName) ? NAME_None : ColumnName;
								NewItem->Values[IdxColumn] = UE::Dataflow::CollectionSpreadSheetHelpers::AttributeValueToString(*CollectionPtr, AttributeName, SelectedGroup, IdxElem);
							}
						}
						ListItems[IdxElem] = NewItem;
					});
			}
		}
	}

	if (ListView)
	{
		ListView->RequestListRefresh();
	}
}

TSharedRef<ITableRow> SCollectionSpreadSheet::GenerateRow(TSharedPtr<const FCollectionSpreadSheetItem> InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	TSharedRef<SCollectionSpreadSheetRow> NewCollectionSpreadSheetRow = SNew(SCollectionSpreadSheetRow, OwnerTable, this->Header, InItem);

	return NewCollectionSpreadSheetRow;
}


//
// ----------------------------------------------------------------------------
//

void SCollectionSpreadSheetWidget::UpdateTransformOutliner()
{
	if (CollectionTable)
	{
		const bool bIsTransformGroupSelected = CollectionTable->GetSelectedGroup() == TEXT("Transform");
		if (TransformOutliner && bIsTransformGroupSelected)
		{
			const FString OutputStr = CollectionTable->GetSelectedOutputStr();
			if (const SCollectionInfo* CollectionInfo = CollectionTable->GetCollectionInfoMap().Find(OutputStr))
			{
				TransformOutliner->SetCollection(CollectionInfo->Collection, CollectionTable->GetSelectedOutput());
				TransformOutliner->RegenerateHeader();
			}
		}
	}
}

void SCollectionSpreadSheetWidget::UpdateVertexOutliner()
{
	if (CollectionTable)
	{
		const bool bIsVerticesGroupSelected = CollectionTable->GetSelectedGroup() == TEXT("Vertices");
		if (VerticesOutliner && bIsVerticesGroupSelected)
		{
			const FString OutputStr = CollectionTable->GetSelectedOutputStr();
			if (const SCollectionInfo* CollectionInfo = CollectionTable->GetCollectionInfoMap().Find(OutputStr))
			{
				VerticesOutliner->SetCollection(CollectionInfo->Collection);
				VerticesOutliner->RegenerateHeader();
			}
		}
	}
}

void SCollectionSpreadSheetWidget::UpdateFaceOutliner()
{
	if (CollectionTable)
	{
		const bool bIsFacesGroupSelected = CollectionTable->GetSelectedGroup() == TEXT("Faces");
		if (FacesOutliner && bIsFacesGroupSelected)
		{
			const FString OutputStr = CollectionTable->GetSelectedOutputStr();
			if (const SCollectionInfo* CollectionInfo = CollectionTable->GetCollectionInfoMap().Find(OutputStr))
			{
				FacesOutliner->SetCollection(CollectionInfo->Collection);
				FacesOutliner->RegenerateHeader();
			}
		}
	}
}

void SCollectionSpreadSheetWidget::NodeOutputsComboBoxSelectionChanged(FName InSelectedOutput, ESelectInfo::Type InSelectInfo)
{
	const int32 TotalWork = 400;
	UE::Dataflow::FScopedProgressNotification ProgressNotification(LOCTEXT("SCollectionSpreadSheetWidget_SelectionChangedProgress", "Updating Collection Spreadsheet"), TotalWork);

	ProgressNotification.SetProgress(0);
	if (CollectionTable)
	{
		if (CollectionTable->GetSelectedOutput() != InSelectedOutput)
		{
			CollectionTable->SetSelectedOutput(InSelectedOutput);

			NodeOutputsComboBoxLabel->SetText(FText::FromName(CollectionTable->GetSelectedOutput()));

			CollectionGroupsComboBox->RefreshOptions();
			CollectionGroupsComboBox->ClearSelection();

			UpdateCollectionGroups(InSelectedOutput);

			if (CollectionGroups.Num() > 0)
			{
				CollectionGroupsComboBox->SetSelectedItem(CollectionGroups[0]);
			}
		}
	}
	ProgressNotification.SetProgress(100);
	UpdateTransformOutliner();

	ProgressNotification.SetProgress(200);
	UpdateVertexOutliner();

	ProgressNotification.SetProgress(300);
	UpdateFaceOutliner();

	SetStatusText();
}


void SCollectionSpreadSheetWidget::CollectionGroupsComboBoxSelectionChanged(FName InSelectedGroup, ESelectInfo::Type InSelectInfo)
{
	const int32 TotalWork = 400;
	UE::Dataflow::FScopedProgressNotification ProgressNotification(LOCTEXT("SCollectionSpreadSheetWidget_SelectionChangedProgress", "Updating Collection Spreadsheet"), TotalWork);

	if (CollectionTable)
	{
		if (CollectionTable->GetSelectedGroup() != InSelectedGroup)
		{
			CollectionTable->SetSelectedGroup(InSelectedGroup);

			CollectionGroupsComboBoxLabel->SetText(FText::FromName(CollectionTable->GetSelectedGroup()));

			if (!InSelectedGroup.IsNone())
			{
				if (const SCollectionInfo* CollectionInfo = CollectionTable->GetCollectionInfoMap().Find(CollectionTable->GetSelectedOutput().ToString()))
				{
					const int32 NumElems = CollectionInfo->Collection.IsValid() ? CollectionInfo->Collection->NumElements(InSelectedGroup): 0;
					CollectionTable->SetNumItems(NumElems);
				}
			}
		}
	}

	ProgressNotification.SetProgress(100);
	UpdateTransformOutliner();

	ProgressNotification.SetProgress(200);
	UpdateVertexOutliner();

	ProgressNotification.SetProgress(300);
	UpdateFaceOutliner();

	SetStatusText();
}


FText SCollectionSpreadSheetWidget::GetNoOutputText()
{
	return FText::FromString("No Output(s)");
}


FText SCollectionSpreadSheetWidget::GetNoGroupText()
{
	return FText::FromString("No Group(s)");
}

const FSlateBrush* SCollectionSpreadSheetWidget::GetPinButtonImage() const
{
	if (bIsPinnedDown)
	{
		return FAppStyle::Get().GetBrush("Icons.Pinned");
	}
	else
	{
		return FAppStyle::Get().GetBrush("Icons.Unpinned");
	}
}

const FSlateBrush* SCollectionSpreadSheetWidget::GetLockButtonImage() const
{
	if (bIsRefreshLocked)
	{
		return FAppStyle::Get().GetBrush("Icons.Lock");
	}
	else
	{
		return FAppStyle::Get().GetBrush("Icons.Unlock");
	}
}

void SCollectionSpreadSheetWidget::Construct(const FArguments& InArgs)
{
	constexpr float CScrollBarWidth = 12.f;

	// Output: [ TransformSelection        |V|]
	TSharedRef<SWidget> OutputSelectionWidget =
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(EVerticalAlignment::VAlign_Center)
		[
			SNew(STextBlock)
				.Text(FText::FromString("Output: "))
		]
		+ SHorizontalBox::Slot()
		.FillWidth(1.0f)
		[
			SAssignNew(NodeOutputsComboBox, SComboBox<FName>)
			.ToolTipText(FText::FromString("Select a node output to see the output's data"))
			.OptionsSource(&NodeOutputs)
			.OnGenerateWidget(SComboBox<FName>::FOnGenerateWidget::CreateLambda([](FName Output)->TSharedRef<SWidget>
			{
				return SNew(STextBlock)
					.Text(FText::FromName(Output));
			}))
			.OnSelectionChanged(this, &SCollectionSpreadSheetWidget::NodeOutputsComboBoxSelectionChanged)
			[
				SAssignNew(NodeOutputsComboBoxLabel, STextBlock)
				.Text(GetNoOutputText())
			]
		];

	// Group: [ TransformGroup        |V|]
	TSharedRef<SWidget> GroupSelectionWidget =
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(EVerticalAlignment::VAlign_Center)
		[
			SNew(STextBlock)
				.Text(FText::FromString("Group: "))
		]
		+ SHorizontalBox::Slot()
		.FillWidth(1.0f)
		[
			SAssignNew(CollectionGroupsComboBox, SComboBox<FName>)
			.ToolTipText(FText::FromString("Select a group see the corresponding data"))
			.OptionsSource(&CollectionGroups)
			.OnGenerateWidget(SComboBox<FName>::FOnGenerateWidget::CreateLambda([](FName Group)->TSharedRef<SWidget>
			{
				return SNew(STextBlock)
					.Text(FText::FromName(Group));
			}))
			.OnSelectionChanged(this, &SCollectionSpreadSheetWidget::CollectionGroupsComboBoxSelectionChanged)
			[
				SAssignNew(CollectionGroupsComboBoxLabel, STextBlock)
					.Text(GetNoGroupText())
			]
		];

	// Pin button widget
	TSharedRef<SWidget> PinButtonWidget =
		SNew(SCheckBox)
		.Style(FAppStyle::Get(), "ToggleButtonCheckbox")
		.ToolTipText(FText::FromString("The button pins down the panel. When it pinned down it doesn't react to node selection change."))
		.IsChecked_Lambda([this]()-> ECheckBoxState
		{
			if (bIsPinnedDown)
			{
				return ECheckBoxState::Checked;
			}
			return ECheckBoxState::Unchecked;
		})
		.OnCheckStateChanged_Lambda([this](ECheckBoxState State)
		{
			if (!NodeNameTextBlock->GetText().IsEmpty())
			{
				bIsPinnedDown = !bIsPinnedDown;
				OnPinnedDownChangedDelegate.Broadcast(bIsPinnedDown);
			}
		})
		.Padding(2.0f)
		.HAlign(HAlign_Center)
		[
			SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(this, &SCollectionSpreadSheetWidget::GetPinButtonImage)
		];

	// Lock button widget
	TSharedRef<SWidget> LockButtonWidget =
		SNew(SCheckBox)
		.Style(FAppStyle::Get(), "ToggleButtonCheckbox")
		.ToolTipText(FText::FromString("The button locks the refresh of the values in the panel."))
		.IsChecked_Lambda([this]()-> ECheckBoxState
		{
			if (bIsRefreshLocked)
			{
				return ECheckBoxState::Checked;
			}
			return ECheckBoxState::Unchecked;
		})
		.OnCheckStateChanged_Lambda([this](ECheckBoxState State)
		{
			if (!NodeNameTextBlock->GetText().IsEmpty())
			{
				bIsRefreshLocked = !bIsRefreshLocked;
				OnRefreshLockedChangedDelegate.Broadcast(bIsRefreshLocked);
			}
		})
		.Padding(2.0f)
		.HAlign(HAlign_Center)
		[
			SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(this, &SCollectionSpreadSheetWidget::GetLockButtonImage)
		];

	TSharedRef<SWidget> PinAndLockButtonsWidget =
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(1.0f, 0)
		[
			PinButtonWidget
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(1.0f, 0)
		[
			LockButtonWidget
		];

	TSharedRef<SWidget> NodeNameWidget =
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(EVerticalAlignment::VAlign_Center)
		[
			SNew(STextBlock)
			.Text(FText::FromString("Node: "))
		]
		+ SHorizontalBox::Slot()
		.FillWidth(1.f)
		.VAlign(EVerticalAlignment::VAlign_Center)
		[
			SAssignNew(NodeNameTextBlock, STextBlock)
		];

	SAssignNew(SpreadSheetHorizontalScrollBar, SScrollBar)
	.Orientation(Orient_Horizontal)
	.Thickness(FVector2D(CScrollBarWidth, CScrollBarWidth));

	SAssignNew(CollectionSpreadSheetExternalVerticalScrollBar, SScrollBar)
	.Orientation(Orient_Vertical)
	.Thickness(FVector2D(CScrollBarWidth, CScrollBarWidth))
	.Visibility(this, &SCollectionSpreadSheetWidget::GetCollectionSpreadSheetVisibility);

	SAssignNew(TransformOutlinerExternalVerticalScrollBar, SScrollBar)
	.Orientation(Orient_Vertical)
	.Thickness(FVector2D(CScrollBarWidth, CScrollBarWidth))
	.Visibility(this, &SCollectionSpreadSheetWidget::GetTransformOutlinerVisibility);

	SAssignNew(VerticesOutlinerExternalVerticalScrollBar, SScrollBar)
	.Orientation(Orient_Vertical)
	.Thickness(FVector2D(CScrollBarWidth, CScrollBarWidth))
	.Visibility(this, &SCollectionSpreadSheetWidget::GetVerticesOutlinerVisibility);

	SAssignNew(FacesOutlinerExternalVerticalScrollBar, SScrollBar)
	.Orientation(Orient_Vertical)
	.Thickness(FVector2D(CScrollBarWidth, CScrollBarWidth))
	.Visibility(this, &SCollectionSpreadSheetWidget::GetFacesOutlinerVisibility);

	ChildSlot
	[
		SNew(SVerticalBox)

		// Selection header
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(4.0f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.MinWidth(150.0)
			.Padding(4.0f, 0.0f)
			[
				OutputSelectionWidget
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.MinWidth(150.0)
			.Padding(4.0f, 0.0f)
			[
				GroupSelectionWidget
			]
			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.Padding(4.0f, 0.0f)
			.HAlign(EHorizontalAlignment::HAlign_Center)
			[
				NodeNameWidget
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(4.0f, 0.0f)
			.HAlign(EHorizontalAlignment::HAlign_Right)
			[
				PinAndLockButtonsWidget
			]
		]

		// Data section
		+SVerticalBox::Slot()
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			[
				SNew(SScrollBox)
				.Orientation(Orient_Horizontal)
				.ExternalScrollbar(SpreadSheetHorizontalScrollBar)

				// All the groups not using a TreeView can use SCollectionSpreadSheet
				+ SScrollBox::Slot()
				[
					SAssignNew(CollectionTable, SCollectionSpreadSheet)
					.ExternalVerticalScrollBar(CollectionSpreadSheetExternalVerticalScrollBar)
					.Visibility(this, &SCollectionSpreadSheetWidget::GetCollectionSpreadSheetVisibility)
				]

				// Displaying Transform group with hierarchy
				+ SScrollBox::Slot()
				.FillSize(1)
				[
					SAssignNew(TransformOutliner, STransformOutliner)
					.ExternalVerticalScrollBar(TransformOutlinerExternalVerticalScrollBar)
					.Visibility(this, &SCollectionSpreadSheetWidget::GetTransformOutlinerVisibility)
				]

				// Displaying Vertices group with hierarchy
				+ SScrollBox::Slot()
				.FillSize(1)
				[
					SAssignNew(VerticesOutliner, SVerticesOutliner)
					.ExternalVerticalScrollBar(VerticesOutlinerExternalVerticalScrollBar)
					.Visibility(this, &SCollectionSpreadSheetWidget::GetVerticesOutlinerVisibility)
				]

				// Displaying Faces group with hierarchy
				+ SScrollBox::Slot()
				.FillSize(1)
				[
					SAssignNew(FacesOutliner, SFacesOutliner)
					.ExternalVerticalScrollBar(FacesOutlinerExternalVerticalScrollBar)
					.Visibility(this, &SCollectionSpreadSheetWidget::GetFacesOutlinerVisibility)
				]
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				CollectionSpreadSheetExternalVerticalScrollBar.ToSharedRef()
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				TransformOutlinerExternalVerticalScrollBar.ToSharedRef()
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				VerticesOutlinerExternalVerticalScrollBar.ToSharedRef()
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				FacesOutlinerExternalVerticalScrollBar.ToSharedRef()
			]
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SpreadSheetHorizontalScrollBar.ToSharedRef()
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(10.0f, 10.0f, 0.0f, 5.0f)
			[
				SAssignNew(StatusTextBlock, STextBlock)
			]
		]
	];
}

void SCollectionSpreadSheetWidget::SetData(const FString& InNodeName)
{
	NodeName = InNodeName;

	NodeOutputs.Empty();

	if (!NodeName.IsEmpty())
	{
		if (CollectionTable && CollectionTable->GetCollectionInfoMap().Num() > 0)
		{
			for (auto& Info : CollectionTable->GetCollectionInfoMap())
			{
				NodeOutputs.Add(FName(*Info.Key));
			}
		}
	}

	CollectionGroups.Empty();
}


void SCollectionSpreadSheetWidget::RefreshWidget()
{
	NodeNameTextBlock->SetText(FText::FromString(NodeName));

	NodeOutputsComboBox->RefreshOptions();
	NodeOutputsComboBox->ClearSelection();

	if (NodeOutputs.Num() > 0)
	{
		NodeOutputsComboBox->SetSelectedItem(NodeOutputs[0]);
	}
	else
	{
		NodeOutputsComboBoxLabel->SetText(GetNoOutputText());
	}

	CollectionGroupsComboBox->RefreshOptions();
	CollectionGroupsComboBox->ClearSelection();

	if (NodeOutputs.Num() > 0)
	{
		UpdateCollectionGroups(NodeOutputs[0]);
	}

	if (CollectionGroups.Num() > 0)
	{
		CollectionGroupsComboBox->SetSelectedItem(CollectionGroups[0]);
	}
	else
	{
		CollectionGroupsComboBoxLabel->SetText(GetNoGroupText());
	}
}


void SCollectionSpreadSheetWidget::SetStatusText()
{
	if (!NodeName.IsEmpty())
	{
		FString NumberStr = FString::FormatAsNumber(CollectionTable? CollectionTable->GetNumItems(): 0);
		FString Str = "Group has " + NumberStr + " elements";
		StatusTextBlock->SetText(FText::FromString(Str));
	}
	else
	{
		StatusTextBlock->SetText(FText::FromString(" "));
	}
}


void SCollectionSpreadSheetWidget::UpdateCollectionGroups(const FName& InOutputName)
{
	if (!InOutputName.IsNone())
	{
		CollectionGroups.Empty();

		if (CollectionTable)
		{
			if (const SCollectionInfo* CollectionInfo = CollectionTable->GetCollectionInfoMap().Find(InOutputName.ToString()))
			{
				if (CollectionInfo->Collection.IsValid())
				{
					for (const FName Group : CollectionInfo->Collection->GroupNames())
					{
						CollectionGroups.Add(Group);
					}
				}
			}
		}
	}
}


#undef LOCTEXT_NAMESPACE // "CollectionSpreadSheet"
