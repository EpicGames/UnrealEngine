// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "PropertyEditorDelegates.h"
#include "AdvancedPreviewSceneModule.h"
#include "Dataflow/DataflowPreviewProfileController.h"

class FAdvancedPreviewScene;
class IDetailsView;
class UAssetViewerSettings;

// This class is almost identical to SAdvancedPreviewDetailsTab except that it doesn't use UEditorPerProjectUserSettings
// to get the current scene profile index. Instead it is supplied with an IProfileIndexStorage object which stores and 
// loads the scene profile index. This allows a separate details tab to be created for each AdvancedPreviewScene.

class SDataflowAdvancedPreviewDetailsTab : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SDataflowAdvancedPreviewDetailsTab)
		: _ProfileIndexStorage(nullptr),
		_AdditionalSettings(nullptr)
	{}

	SLATE_ARGUMENT(TSharedPtr<FDataflowPreviewProfileController::IProfileIndexStorage>, ProfileIndexStorage)

	/** Additional settings object to display in the view */
	SLATE_ARGUMENT(UObject*, AdditionalSettings)

	/** Customizations to use for this details tab */
	SLATE_ARGUMENT(TArray<FAdvancedPreviewSceneModule::FDetailCustomizationInfo>, DetailCustomizations)

	/** Customizations to use for this details tab */
	SLATE_ARGUMENT(TArray<FAdvancedPreviewSceneModule::FPropertyTypeCustomizationInfo>, PropertyTypeCustomizations)

	/** Delegates to use for this details tab */
	SLATE_ARGUMENT(TArray<FAdvancedPreviewSceneModule::FDetailDelegates>, Delegates)

	SLATE_END_ARGS()

public:
	/** **/
	SDataflowAdvancedPreviewDetailsTab();
	~SDataflowAdvancedPreviewDetailsTab();

	/** SWidget functions */
	void Construct(const FArguments& InArgs, const TSharedRef<FAdvancedPreviewScene>& InPreviewScene);

	void Refresh();

protected:
	void CreateSettingsView();
	void ComboBoxSelectionChanged(TSharedPtr<FString> NewSelection, ESelectInfo::Type /*SelectInfo*/);
	void UpdateSettingsView();
	void UpdateProfileNames();
	FReply AddProfileButtonClick();
	FReply RemoveOrResetProfileButtonClick();
protected:
	void OnAssetViewerSettingsRefresh(const FName& InPropertyName);
	void OnAssetViewerSettingsPostUndo();
protected:
	/** Property viewing widget */
	TSharedPtr<IDetailsView> SettingsView;
	TSharedPtr<STextComboBox> ProfileComboBox;
	TWeakPtr<FAdvancedPreviewScene> PreviewScenePtr;
	UAssetViewerSettings* DefaultSettings;
	UObject* AdditionalSettings;

	TArray<TSharedPtr<FString>> ProfileNames;
	int32 ProfileIndex;

	void OnPreviewSceneChanged(TSharedRef<FAdvancedPreviewScene> PreviewScene);

	FDelegateHandle RefreshDelegate;
	FDelegateHandle AddRemoveProfileDelegate;
	FDelegateHandle PostUndoDelegate;

	TSharedPtr<FDataflowPreviewProfileController::IProfileIndexStorage> ProfileIndexStorage;

	TArray<FAdvancedPreviewSceneModule::FDetailCustomizationInfo> DetailCustomizations;

	TArray<FAdvancedPreviewSceneModule::FPropertyTypeCustomizationInfo> PropertyTypeCustomizations;

	TArray<FAdvancedPreviewSceneModule::FDetailDelegates> Delegates;
};
