// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowAnyTypeCustomization.h"
#include "Dataflow/DataflowAnyType.h"
#include "Customizations/MathStructCustomizations.h"
#include "PropertyEditorModule.h"

#include "PropertyHandle.h"
#include "DetailWidgetRow.h"

namespace UE::Dataflow
{
	static const FName DataflowNumericTypesName = TEXT("DataflowNumericTypes");
	static const FName DataflowBoolTypesName = TEXT("DataflowBoolTypes");
	static const FName DataflowStringTypesName = TEXT("DataflowStringTypes");
	static const FName DataflowStringConvertibleTypesName = TEXT("DataflowStringConvertibleTypes");
	static const FName DataflowVectorTypesName = TEXT("DataflowVectorTypes");
	static const FName DataflowTransformTypesName = TEXT("DataflowTransformTypes");

	void RegisterAnyTypeCustomizations(FPropertyEditorModule& PropertyModule)
	{
		PropertyModule.RegisterCustomPropertyTypeLayout(DataflowNumericTypesName, FOnGetPropertyTypeCustomizationInstance::CreateStatic(&UE::Dataflow::FNumericAnyTypeCustomization::MakeInstance));
		PropertyModule.RegisterCustomPropertyTypeLayout(DataflowBoolTypesName, FOnGetPropertyTypeCustomizationInstance::CreateStatic(&UE::Dataflow::FBoolAnyTypeCustomization::MakeInstance));
		PropertyModule.RegisterCustomPropertyTypeLayout(DataflowStringTypesName, FOnGetPropertyTypeCustomizationInstance::CreateStatic(&UE::Dataflow::FStringAnyTypeCustomization::MakeInstance));
		PropertyModule.RegisterCustomPropertyTypeLayout(DataflowStringConvertibleTypesName, FOnGetPropertyTypeCustomizationInstance::CreateStatic(&UE::Dataflow::FStringConvertibleAnyTypeCustomization::MakeInstance));
		PropertyModule.RegisterCustomPropertyTypeLayout(DataflowVectorTypesName, FOnGetPropertyTypeCustomizationInstance::CreateStatic(&UE::Dataflow::FVectorAnyTypeCustomization::MakeInstance));
		PropertyModule.RegisterCustomPropertyTypeLayout(DataflowTransformTypesName, FOnGetPropertyTypeCustomizationInstance::CreateStatic(&UE::Dataflow::FTransformAnyTypeCustomization::MakeInstance));
	}

	void UnregisterAnyTypeCustomizations(FPropertyEditorModule& PropertyModule)
	{
		PropertyModule.UnregisterCustomPropertyTypeLayout(DataflowNumericTypesName);
		PropertyModule.UnregisterCustomPropertyTypeLayout(DataflowBoolTypesName);
		PropertyModule.UnregisterCustomPropertyTypeLayout(DataflowStringTypesName);
		PropertyModule.UnregisterCustomPropertyTypeLayout(DataflowStringConvertibleTypesName);
		PropertyModule.UnregisterCustomPropertyTypeLayout(DataflowVectorTypesName);
		PropertyModule.UnregisterCustomPropertyTypeLayout(DataflowTransformTypesName);
	}

	/////////////////////////////////////////////////////////////////////////////////////////////////////

	namespace Private
	{
		template <typename T>
		void CustomizeHeaderForTypeWithoutCustomization(TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow)
		{
			TSharedPtr<IPropertyHandle> ValuePropertyHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(T, Value));
			check(ValuePropertyHandle);

			HeaderRow
				.NameContent()
				[
					StructPropertyHandle->CreatePropertyNameWidget(StructPropertyHandle->GetPropertyDisplayName())
				]
				.ValueContent()
				[
					ValuePropertyHandle->CreatePropertyValueWidget()
				];
		}

		template <typename T, typename TCustomization>
		void CustomizeHeaderForTypeWithCustomization(TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
		{
			TSharedPtr<IPropertyHandle> ValuePropertyHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(T, Value));
			check(ValuePropertyHandle);

			TSharedRef<TCustomization> MathStructCustomization = MakeShareable(new TCustomization);
			MathStructCustomization->CustomizeHeader(ValuePropertyHandle.ToSharedRef(), HeaderRow, StructCustomizationUtils);;

			// make sure we fix the name to be the original parent 
			HeaderRow
				.NameContent()
				[
					StructPropertyHandle->CreatePropertyNameWidget(StructPropertyHandle->GetPropertyDisplayName())
				];
		}
	}

	////////////////////////////////////////////////////////////////////////////////////////////////

	TSharedRef<IPropertyTypeCustomization> FNumericAnyTypeCustomization::MakeInstance()
	{
		return MakeShareable(new FNumericAnyTypeCustomization);
	}

	void FNumericAnyTypeCustomization::CustomizeHeader(TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
	{
		Private::CustomizeHeaderForTypeWithoutCustomization<FDataflowNumericTypes>(StructPropertyHandle, HeaderRow);
	}

	////////////////////////////////////////////////////////////////////////////////////////////////

	TSharedRef<IPropertyTypeCustomization> FBoolAnyTypeCustomization::MakeInstance()
	{
		return MakeShareable(new FBoolAnyTypeCustomization);
	}

	void FBoolAnyTypeCustomization::CustomizeHeader(TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
	{
		Private::CustomizeHeaderForTypeWithoutCustomization<FDataflowBoolTypes>(StructPropertyHandle, HeaderRow);
	}

	////////////////////////////////////////////////////////////////////////////////////////////////

	TSharedRef<IPropertyTypeCustomization> FStringAnyTypeCustomization::MakeInstance()
	{
		return MakeShareable(new FStringAnyTypeCustomization);
	}

	void FStringAnyTypeCustomization::CustomizeHeader(TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
	{
		Private::CustomizeHeaderForTypeWithoutCustomization<FDataflowStringTypes>(StructPropertyHandle, HeaderRow);
	}

	////////////////////////////////////////////////////////////////////////////////////////////////

	TSharedRef<IPropertyTypeCustomization> FStringConvertibleAnyTypeCustomization::MakeInstance()
	{
		return MakeShareable(new FStringConvertibleAnyTypeCustomization);
	}

	void FStringConvertibleAnyTypeCustomization::CustomizeHeader(TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
	{
		Private::CustomizeHeaderForTypeWithoutCustomization<FDataflowStringConvertibleTypes>(StructPropertyHandle, HeaderRow);
	}

	////////////////////////////////////////////////////////////////////////////////////////////////
	
	TSharedRef<IPropertyTypeCustomization> FVectorAnyTypeCustomization::MakeInstance()
	{
		return MakeShareable(new FVectorAnyTypeCustomization);
	}

	void FVectorAnyTypeCustomization::CustomizeHeader(TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
	{
		Private::CustomizeHeaderForTypeWithCustomization<FDataflowVectorTypes, FMathStructCustomization>(StructPropertyHandle, HeaderRow, StructCustomizationUtils);
	}

	////////////////////////////////////////////////////////////////////////////////////////////////

	TSharedRef<IPropertyTypeCustomization> FTransformAnyTypeCustomization::MakeInstance()
	{
		return MakeShareable(new FTransformAnyTypeCustomization);
	}

	void FTransformAnyTypeCustomization::CustomizeHeader(TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
	{
		Private::CustomizeHeaderForTypeWithoutCustomization<FDataflowTransformTypes>(StructPropertyHandle, HeaderRow);
	}

	////////////////////////////////////////////////////////////////////////////////////////////////
}
