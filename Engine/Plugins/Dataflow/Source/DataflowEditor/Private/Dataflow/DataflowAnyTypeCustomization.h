// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"

class SDataflowGraphEditor;
class SWidget;
class ITableRow;
class STableViewBase;
class SComboButton;

class FPropertyEditorModule;

namespace UE::Dataflow
{
	class FAnyTypeCustomizationBase
		: public IPropertyTypeCustomization
	{
		//~ Begin IPropertyTypeCustomization implementation
		virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override {}
		//~ End IPropertyTypeCustomization implementation
	protected:
	};

	//////////////////////////////////////////////////////////////////////////////////////////////////////

	class FNumericAnyTypeCustomization
		: public FAnyTypeCustomizationBase
	{
	public:
		static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	protected:
		//~ Begin IPropertyTypeCustomization implementation
		virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
		//~ End IPropertyTypeCustomization implementation
	};

	//////////////////////////////////////////////////////////////////////////////////////////////////////

	class FBoolAnyTypeCustomization
		: public FAnyTypeCustomizationBase
	{
	public:
		static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	protected:
		//~ Begin IPropertyTypeCustomization implementation
		virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
		//~ End IPropertyTypeCustomization implementation
	};

	//////////////////////////////////////////////////////////////////////////////////////////////////////

	class FStringAnyTypeCustomization
		: public FAnyTypeCustomizationBase
	{
	public:
		static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	protected:
		//~ Begin IPropertyTypeCustomization implementation
		virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
		//~ End IPropertyTypeCustomization implementation
	};

	//////////////////////////////////////////////////////////////////////////////////////////////////////

	class FStringConvertibleAnyTypeCustomization
		: public FAnyTypeCustomizationBase
	{
	public:
		static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	protected:
		//~ Begin IPropertyTypeCustomization implementation
		virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
		//~ End IPropertyTypeCustomization implementation
	};

	//////////////////////////////////////////////////////////////////////////////////////////////////////

	class FVectorAnyTypeCustomization
		: public FAnyTypeCustomizationBase
	{
	public:
		static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	protected:
		//~ Begin IPropertyTypeCustomization implementation
		virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
		//~ End IPropertyTypeCustomization implementation
	};

	//////////////////////////////////////////////////////////////////////////////////////////////////////

	class FTransformAnyTypeCustomization
		: public FAnyTypeCustomizationBase
	{
	public:
		static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	protected:
		//~ Begin IPropertyTypeCustomization implementation
		virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
		//~ End IPropertyTypeCustomization implementation
	};

	//////////////////////////////////////////////////////////////////////////////////////////////////////

	void RegisterAnyTypeCustomizations(FPropertyEditorModule& PropertyModule);
	void UnregisterAnyTypeCustomizations(FPropertyEditorModule& PropertyModule);
}
