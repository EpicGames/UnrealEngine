// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowAssetEditUtils.h"

#include "HAL/PlatformApplicationMisc.h"
#include "Dataflow/AssetDefinition_DataflowAsset.h"
#include "Dataflow/DataflowObject.h"
#include "Dataflow/DataflowSchema.h"
#include "Dataflow/DataflowNode.h"
#include "Dataflow/DataflowEdNode.h"
#include "Dataflow/DataflowSNode.h"
#include "Dataflow/DataflowSubGraph.h"
#include "Dataflow/DataflowSubGraphNodes.h"
#include "Dataflow/DataflowVariableNodes.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "PropertyBagDetails.h"
#include "ScopedTransaction.h"
#include "Misc/StringOutputDevice.h"
#include "Settings/EditorStyleSettings.h"

#define LOCTEXT_NAMESPACE "DataflowAssetEditUtils"

namespace UE::Dataflow
{
	namespace EditAssetUtils::Private
	{
		static const TCHAR* DataflowVariableClipboardPrefix = TEXT("DataflowVariable_");
		
		enum class EChangeResult
		{
			None,
			Changed,
			Cancel,
		};

		/** 
		* Change a dataflow asset with a transaction
		* The InFunction return paramater determines if the asset will be modified or not 
		* if modification happens, a PostEditChangeProperty notification is sent
		*/
		static void ChangeDataflowAssetWithTransaction(UDataflow* DataflowAsset, const FText &TransactionName, TFunctionRef<EChangeResult(UDataflow&)> InFunction, FName ChangedPropertyName)
		{
			if (DataflowAsset)
			{
				FScopedTransaction Transaction(TransactionName);

				const EChangeResult Result = InFunction(*DataflowAsset);
				switch (Result)
				{
				case EChangeResult::Cancel:
					Transaction.Cancel();
					break;

				case EChangeResult::Changed:
				{
					DataflowAsset->Modify();
					if (!ChangedPropertyName.IsNone())
					{
						FPropertyChangedEvent PropertyChangedEvent(nullptr);
						if (UClass* DataflowClass = DataflowAsset->GetClass())
						{
							FProperty* MemberProperty = DataflowClass->FindPropertyByName(ChangedPropertyName);
							PropertyChangedEvent.SetActiveMemberProperty(MemberProperty);
						}
						DataflowAsset->PostEditChangeProperty(PropertyChangedEvent);
					}
					break;
				}
				case EChangeResult::None:
				default:
					break;
				}
			}
		}

		/** Generate a Dataflow child object unique name from a BaseName ( Node or Subgraph for example ) */
		static FName GenerateUniqueObjectName(UDataflow* Dataflow, const FName InBaseName)
		{
			FString Left, Right;
			int32 NameIndex = 1;

			// Check if NodeBaseName already ends with "_dd"
			FName BaseName(InBaseName);
			if (BaseName.ToString().Split(TEXT("_"), &Left, &Right, ESearchCase::IgnoreCase, ESearchDir::FromEnd))
			{
				if (Right.IsNumeric())
				{
					NameIndex = FCString::Atoi(*Right);
					BaseName = FName(Left);
				}
			}

			// name must be unique for all nodes in the Dataflow::FGraph 
			// Unreal require names to be unique within the parent but because we have one FGraph across all EdGraph ( include SubGraphs) objects 
			// we need to make sure the name is unique across them, so that we don't get a assert when creating the EdNode
			FName UniqueName = BaseName;
			bool bNameWasChanged = false;
			do {
				// reset for this loop 
				bNameWasChanged = false;
				if (!::IsUniqueObjectName(UniqueName, Dataflow))
				{
					UniqueName = ::MakeUniqueObjectName(Dataflow, UDataflowEdNode::StaticClass(), UniqueName);
					bNameWasChanged = true;
				}

				for (TObjectPtr<UDataflowSubGraph> SubGraph: Dataflow->GetSubGraphs())
				{
					if (!::IsUniqueObjectName(UniqueName, SubGraph))
					{
						UniqueName = ::MakeUniqueObjectName(SubGraph, UDataflowEdNode::StaticClass(), UniqueName);
						bNameWasChanged = true;
					}
				}
			} while (bNameWasChanged);

			return UniqueName;
		}

		static TSharedPtr<FDataflowNode> AddDataflowNode(UDataflow* Dataflow, FName NodeName, FName NodeTypeName)
		{
			if (UE::Dataflow::FNodeFactory* Factory = UE::Dataflow::FNodeFactory::GetInstance())
			{
				FNewNodeParameters Parameters
				{
					.Guid = FGuid::NewGuid(),
					.Type = NodeTypeName,
					.Name = GenerateUniqueObjectName(Dataflow, NodeName),
					.OwningObject = Dataflow,
				};

				return Factory->NewNodeFromRegisteredType(*Dataflow->GetDataflow(), Parameters);
			}
			return nullptr;
		}

		static UDataflowEdNode* CreateDataflowEdNode(UEdGraph* EdGraph, TSharedPtr<FDataflowNode> DataflowNode, const FVector2D& Location, UEdGraphPin* FromPin)
		{
			if (UDataflow* Dataflow = UDataflow::GetDataflowAssetFromEdGraph(EdGraph))
			{
				if (UDataflowEdNode* EdNode = NewObject<UDataflowEdNode>(EdGraph, UDataflowEdNode::StaticClass(), DataflowNode->GetName()))
				{
					EdNode->SetFlags(RF_Transactional);

					Dataflow->Modify();
					EdGraph->Modify();

					// make sure we set the guid before adding to graph so that the listener to the graph notification have all the info needed
					EdNode->SetDataflowGraph(Dataflow->GetDataflow());
					EdNode->SetDataflowNodeGuid(DataflowNode->GetGuid());

					EdGraph->AddNode(EdNode, /*bUserAction*/true, /*bSelectNewNode*/false);
					
					EdNode->CreateNewGuid();
					EdNode->PostPlacedNewNode();
					EdNode->AllocateDefaultPins();

					if (FromPin)
					{
						FromPin->Modify();
						EdNode->AutowireNewNode(FromPin);
					}

					EdNode->NodePosX = Location.X;
					EdNode->NodePosY = Location.Y;

					return EdNode;
				}
			}
			return nullptr;
		}

		UEdGraphNode_Comment* CreateCommentEdNode(UEdGraph* EdGraph, const FVector2D& Location, const FString& Comment, const FVector2D Size, const FLinearColor& Color, int32 FontSize)
		{
			if (UDataflow* Dataflow = UDataflow::GetDataflowAssetFromEdGraph(EdGraph))
			{
				if (UEdGraphNode_Comment* CommentTemplate = NewObject<UEdGraphNode_Comment>(EdGraph))
				{
					CommentTemplate->SetFlags(RF_Transactional);

					Dataflow->Modify();
					EdGraph->Modify();

					CommentTemplate->bCommentBubbleVisible_InDetailsPanel = false;
					CommentTemplate->bCommentBubbleVisible = false;
					CommentTemplate->bCommentBubblePinned = false;

					// set outer to be the graph so it doesn't go away
					CommentTemplate->Rename(NULL, EdGraph, REN_NonTransactional);
					EdGraph->AddNode(CommentTemplate, true, /*bSelectNewNode*/false);

					CommentTemplate->CreateNewGuid();
					CommentTemplate->PostPlacedNewNode();
					CommentTemplate->AllocateDefaultPins();

					CommentTemplate->NodePosX = Location.X;
					CommentTemplate->NodePosY = Location.Y;
					CommentTemplate->NodeWidth = Size.X;
					CommentTemplate->NodeHeight = Size.Y;
					CommentTemplate->SnapToGrid(GetDefault<UEditorStyleSettings>()->GridSnapSize);
					CommentTemplate->CommentColor = Color;
					CommentTemplate->FontSize = FontSize;

					CommentTemplate->NodeComment = Comment;


					EdGraph->NotifyGraphChanged();

					return CommentTemplate;
				}
			}
			return nullptr;
		}

		/** returns true if the variable was modified */
		static bool ModifyVariable(UDataflow& DataflowAsset, FName Variable, TFunctionRef<void(FPropertyBagPropertyDesc& PropertyDesc)> InFunction)
		{
			bool bModified = false;

			TArray<FPropertyBagPropertyDesc> NewPropertyDescs;
			NewPropertyDescs.Append(DataflowAsset.Variables.GetPropertyBagStruct()->GetPropertyDescs());
			for (FPropertyBagPropertyDesc& PropertyDesc : NewPropertyDescs)
			{
				if (PropertyDesc.Name == Variable)
				{
					InFunction(PropertyDesc);
					bModified = true;
				}
			}

			if (bModified)
			{
				if (const UPropertyBag* NewBagStruct = UPropertyBag::GetOrCreateFromDescs(NewPropertyDescs))
				{
					DataflowAsset.Variables.MigrateToNewBagStruct(NewBagStruct);
					FDataflowAssetDelegates::OnVariablesChanged.Broadcast(&DataflowAsset, Variable);
					return true;
				}
			}
			return false;
		}

		/** Generate a Dataflow asset variable unique name from a BaseName */
		static FName GenerateUniqueVariableName(const UDataflow& DataflowAsset, FName BaseName)
		{
			int32 Counter = 1;
			FName UniqueName = FInstancedPropertyBag::SanitizePropertyName(BaseName);
			const FString BasenameStr{ BaseName.ToString() };
			while (true)
			{
				if (DataflowAsset.Variables.FindPropertyDescByName(UniqueName) == nullptr)
				{
					break; // found an available name exit 
				}
				UniqueName = FName(FString::Format(TEXT("{0}_{1}"), { BasenameStr, FString::FromInt(Counter++)}));
			}
			return UniqueName;
		}

		static UEdGraphPin* FindPin(const UDataflowEdNode* Node, const EEdGraphPinDirection Direction, const FName Name)
		{
			for (UEdGraphPin* Pin : Node->GetAllPins())
			{
				if (Pin->PinName == Name && Pin->Direction == Direction)
				{
					return Pin;
				}
			}

			return nullptr;
		}

		void RenameSubGraphCallNodes(UDataflow& DataflowAsset, UEdGraph& EdGraph, const FGuid& SubGraphGuid, FName NewSubGraphName)
		{
			for (UEdGraphNode* EdNode : EdGraph.Nodes)
			{
				if (UDataflowEdNode* DataflowEdNode = Cast<UDataflowEdNode>(EdNode))
				{
					if (TSharedPtr<FDataflowNode> DataflowNode = DataflowEdNode->GetDataflowNode())
					{
						if (FDataflowCallSubGraphNode* CallNode = DataflowNode->AsType<FDataflowCallSubGraphNode>())
						{
							if (CallNode->GetSubGraphGuid() == SubGraphGuid)
							{
								const FName UniqueName = GenerateUniqueObjectName(&DataflowAsset, NewSubGraphName);
								DataflowEdNode->Rename(*UniqueName.ToString(), &EdGraph);
								CallNode->SetName(UniqueName);
								CallNode->RefreshSubGraphName();
							}
						}
					}
				}
			}
		}

		void RenameSubGraphCallNodes(UDataflow& DataflowAsset, const FGuid& SubGraphGuid, FName NewSubGraphName)
		{
			RenameSubGraphCallNodes(DataflowAsset, DataflowAsset, SubGraphGuid, NewSubGraphName);
			for (UDataflowSubGraph* SubGraph : DataflowAsset.GetSubGraphs())
			{
				if (SubGraph)
				{
					RenameSubGraphCallNodes(DataflowAsset, *SubGraph, SubGraphGuid, NewSubGraphName);
				}
			}
		}

		void RenameVariableCallNodes(UDataflow& DataflowAsset, UEdGraph& EdGraph, FName VariableName, FName NewVariableName)
		{
			for (UEdGraphNode* EdNode : EdGraph.Nodes)
			{
				if (UDataflowEdNode* DataflowEdNode = Cast<UDataflowEdNode>(EdNode))
				{
					if (TSharedPtr<FDataflowNode> DataflowNode = DataflowEdNode->GetDataflowNode())
					{
						if (FGetDataflowVariableNode* VariableNode = DataflowNode->AsType<FGetDataflowVariableNode>())
						{
							if (VariableNode->GetVariableName() == VariableName)
							{
								const FName UniqueName = GenerateUniqueObjectName(&DataflowAsset, NewVariableName);
								DataflowEdNode->Rename(*UniqueName.ToString(), &EdGraph);
								VariableNode->SetName(UniqueName);
								VariableNode->SetVariable(&DataflowAsset, NewVariableName);
							}
						}
					}
				}
			}
		}

		void RenameVariableCallNodes(UDataflow& DataflowAsset, FName VariableName, FName NewVariableName)
		{
			RenameVariableCallNodes(DataflowAsset, DataflowAsset, VariableName, NewVariableName);
			for (UDataflowSubGraph* SubGraph : DataflowAsset.GetSubGraphs())
			{
				if (SubGraph)
				{
					RenameVariableCallNodes(DataflowAsset, *SubGraph, VariableName, NewVariableName);
				}
			}
		}

		void DeleteNodesNoTransaction(UEdGraph* EdGraph, const TArray<UEdGraphNode*>& NodesToDelete)
		{
			if (EdGraph && NodesToDelete.Num())
			{
				for (UEdGraphNode* EdNode : NodesToDelete)
				{
					EdNode->Modify();
					if (UDataflowEdNode* DataflowEdNode = Cast<UDataflowEdNode>(EdNode))
					{
						if (const TSharedPtr<UE::Dataflow::FGraph> DataflowGraph = DataflowEdNode->GetDataflowGraph())
						{
							EdGraph->RemoveNode(EdNode);
							if (TSharedPtr<FDataflowNode> DataflowNode = DataflowEdNode->GetDataflowNode())
							{
								DataflowGraph->RemoveNode(DataflowNode);
							}
						}
					}
					else if (UEdGraphNode_Comment* CommentNode = Cast<UEdGraphNode_Comment>(EdNode))
					{
						EdGraph->RemoveNode(CommentNode);
					}

					// Auto-rename node so that its current name is made available until it is garbage collected
					EdNode->Rename();
				}
			}
		}

		/**
		* Serialize a flat list of EdGraph nodes (regular + comment) into the per-graph payload arrays.
		* Connections are emitted only when both endpoints are inside the same input set.
		* Used by both the top-level copy path and the recursive subgraph-copy walk.
		*/
		void SerializeNodesToPayload(
			const TArray<const UEdGraphNode*>& NodesToCopy,
			TArray<FDataflowNodeData>& OutNodeData,
			TArray<FDataflowCommentNodeData>& OutCommentNodeData,
			TArray<FDataflowConnectionData>& OutConnectionData)
		{
			TSet<FGuid> NodeGuids;
			TArray<const FDataflowInput*> NodeInputsToSave;

			for (const UEdGraphNode* EdNode : NodesToCopy)
			{
				if (const UDataflowEdNode* DataflowEdNode = Cast<const UDataflowEdNode>(EdNode))
				{
					if (const TSharedPtr<const FDataflowNode> DataflowNode = DataflowEdNode->GetDataflowNode())
					{
						NodeGuids.Add(DataflowNode->GetGuid());
						NodeInputsToSave.Append(DataflowNode->GetInputs());

						FString ContentString;
						const TUniquePtr<FDataflowNode> DefaultElement;
						DataflowNode->TypedScriptStruct()->ExportText(ContentString, DataflowNode.Get(), DataflowNode.Get(), UDataflow::GetDataflowAssetFromEdGraph(EdNode->GetGraph()), PPF_Copy, nullptr);

						FDataflowNodeData NodeData
						{
							.Type = DataflowNode->GetType().ToString(),
							.Name = DataflowNode->GetName().ToString(),
							.Properties = MoveTemp(ContentString),
							.Position = { (double)EdNode->NodePosX, (double)EdNode->NodePosY },
						};
						OutNodeData.Emplace(NodeData);
					}
				}
				else if (const UEdGraphNode_Comment* CommentEdNode = Cast<const UEdGraphNode_Comment>(EdNode))
				{
					FDataflowCommentNodeData CommentNodeData
					{
						.Name = CommentEdNode->NodeComment,
						.Size = { (double)CommentEdNode->NodeWidth, (double)CommentEdNode->NodeHeight },
						.Color = CommentEdNode->CommentColor,
						.Position = { (double)CommentEdNode->NodePosX, (double)CommentEdNode->NodePosY },
						.FontSize = CommentEdNode->FontSize,
					};
					OutCommentNodeData.Emplace(CommentNodeData);
				}
			}

			for (const FDataflowInput* Input : NodeInputsToSave)
			{
				if (!Input)
				{
					continue;
				}

				const FDataflowOutput* Output = Input->GetConnection();
				if (!Output)
				{
					continue;
				}

				if (NodeGuids.Contains(Output->GetOwningNodeGuid()))
				{
					FDataflowConnectionData DataflowConnectionData;
					DataflowConnectionData.Set(*Output, *Input);
					OutConnectionData.Emplace(DataflowConnectionData);
				}
			}
		}

		/**
		* Recreate nodes / comments / connections inside TargetEdGraph from a payload.
		* Pasted node positions are computed as Location + (NodePosition - RefLocation),
		* where RefLocation is taken from the first node/comment in the payload (matches the
		* legacy in-place behavior). Used by both the top-level paste path and the new
		* subgraph paste path (which passes Location = RefLocation to preserve absolute coords).
		*/
		void PastePayloadIntoEdGraph(
			UEdGraph& TargetEdGraph,
			const FVector2D& Location,
			const TArray<FDataflowNodeData>& NodeData,
			const TArray<FDataflowCommentNodeData>& CommentNodeData,
			const TArray<FDataflowConnectionData>& ConnectionData,
			TArray<UEdGraphNode*>& OutPastedNodes)
		{
			UDataflow* DataflowAsset = UDataflow::GetDataflowAssetFromEdGraph(&TargetEdGraph);
			if (!DataflowAsset)
			{
				return;
			}

			TMap<FString, UDataflowEdNode*> OriginalNodeNameToEdNode;

			FVector2D RefLocation { 0, 0 };
			if (NodeData.Num())
			{
				RefLocation.X = NodeData[0].Position.X;
				RefLocation.Y = NodeData[0].Position.Y;
			}
			else if (CommentNodeData.Num())
			{
				RefLocation.X = CommentNodeData[0].Position.X;
				RefLocation.Y = CommentNodeData[0].Position.Y;
			}

			for (const FDataflowNodeData& Node : NodeData)
			{
				const FName NodeType(*Node.Type);
				const FName NodeName(*Node.Name);
				const FVector2D NodeLocation(Location + (Node.Position - RefLocation));

				if (TSharedPtr<FDataflowNode> DataflowNode = AddDataflowNode(DataflowAsset, NodeName, NodeType))
				{
					if (!Node.Properties.IsEmpty())
					{
						DataflowNode->TypedScriptStruct()->ImportText(*Node.Properties, DataflowNode.Get(), DataflowAsset, EPropertyPortFlags::PPF_InstanceSubobjects, nullptr, DataflowNode->TypedScriptStruct()->GetName(), true);
					}

					// if we are pasting a dataflow variable, let's create the variable if needed
					if (FGetDataflowVariableNode* VariableNode = DataflowNode->AsType<FGetDataflowVariableNode>())
					{
						VariableNode->TryAddVariableToDataflowAsset(*DataflowAsset);
					}

					// Do any post-import fixup.
					FArchive Ar;
					Ar.SetIsLoading(true);
					DataflowNode->PostSerialize(Ar);

					if (UDataflowEdNode* EdNode = CreateDataflowEdNode(&TargetEdGraph, DataflowNode, NodeLocation, /*FromPin*/nullptr))
					{
						OriginalNodeNameToEdNode.Add(Node.Name, EdNode);
						OutPastedNodes.Add(EdNode);
					}
				}
			}

			for (const FDataflowCommentNodeData& Comment : CommentNodeData)
			{
				const FVector2D CommentNodeLocation(Location + (Comment.Position - RefLocation));

				if (UEdGraphNode_Comment* CommentEdNode = CreateCommentEdNode(&TargetEdGraph, CommentNodeLocation, Comment.Name, Comment.Size, Comment.Color, Comment.FontSize))
				{
					OutPastedNodes.Add(CommentEdNode);
				}
			}

			for (const FDataflowConnectionData& Connection : ConnectionData)
			{
				FString NodeIn, PropertyIn, TypeIn;
				FDataflowConnectionData::GetNodePropertyAndType(Connection.In, NodeIn, PropertyIn, TypeIn);

				FString NodeOut, PropertyOut, TypeOut;
				FDataflowConnectionData::GetNodePropertyAndType(Connection.Out, NodeOut, PropertyOut, TypeOut);

				if (TypeIn == TypeOut)
				{
					const UDataflowEdNode* EdNodeIn = OriginalNodeNameToEdNode.FindRef(NodeIn);
					const UDataflowEdNode* EdNodeOut = OriginalNodeNameToEdNode.FindRef(NodeOut);

					const FGuid GuidIn = EdNodeIn ? EdNodeIn->DataflowNodeGuid : FGuid();
					const FGuid GuidOut = EdNodeOut ? EdNodeOut->DataflowNodeGuid : FGuid();

					const FName InputputName = *PropertyIn;
					const FName OutputputName = *PropertyOut;

					if (TSharedPtr<UE::Dataflow::FGraph> DataflowGraph = DataflowAsset->GetDataflow())
					{
						if (TSharedPtr<FDataflowNode> DataflowNodeFrom = DataflowGraph->FindBaseNode(GuidOut))
						{
							if (TSharedPtr<FDataflowNode> DataflowNodeTo = DataflowGraph->FindBaseNode(GuidIn))
							{
								FDataflowInput* InputConnection = DataflowNodeTo->FindInput(InputputName);
								FDataflowOutput* OutputConnection = DataflowNodeFrom->FindOutput(OutputputName);

								// make sure we set the right type before attempting any connection
								DataflowNodeFrom->TrySetConnectionType(OutputConnection, FName(TypeOut));
								DataflowNodeTo->TrySetConnectionType(InputConnection, FName(TypeIn));

								// first connect the edgraph as this may affect the dataflow inputs ( for AnyType )
								UEdGraphPin* OutputPin = FindPin(EdNodeOut, EEdGraphPinDirection::EGPD_Output, OutputputName);
								UEdGraphPin* InputPin = FindPin(EdNodeIn, EEdGraphPinDirection::EGPD_Input, InputputName);
								if (OutputPin && InputPin)
								{
									TargetEdGraph.GetSchema()->TryCreateConnection(OutputPin, InputPin);
								}

								// now connect the dataflow
								DataflowGraph->Connect(OutputConnection, InputConnection);
							}
						}
					}
				}
				else
				{
					UE_LOGF(LogChaosDataflow, Error, "Failed to reconnect output [%ls] to input [%ls] - incompatible types", *Connection.Out, *Connection.In);
				}
			}
		}

	}

	/**
	* Create a new UDataflowSubGraph that preserves a specific GUID and ForEach state,
	* mirrors the setup done by FEditAssetUtils::AddNewSubGraph (schema, transactional flag,
	* AddSubGraph + Created broadcast). Caller is expected to populate the subgraph's
	* contents via PastePayloadIntoEdGraph afterwards. UniqueName must already be unique.
	*
	* Lives on FEditAssetUtils so that UDataflowSubGraph's friend-only SetSubGraphGuid
	* setter is reachable here.
	*/
	UDataflowSubGraph* FEditAssetUtils::CreateSubGraphWithGuid(UDataflow& DataflowAsset, FName UniqueName, const FGuid& Guid, bool bIsForEach)
	{
		UDataflowSubGraph* NewSubGraph = NewObject<UDataflowSubGraph>(&DataflowAsset, UniqueName);
		ensure(NewSubGraph->GetFName().IsEqual(UniqueName));
		NewSubGraph->Schema = UDataflowSchema::StaticClass();
		NewSubGraph->SetFlags(RF_Transactional);

		// Override the auto-generated GUID. Friend access via UDataflowSubGraph's friendship with FEditAssetUtils.
		NewSubGraph->SetSubGraphGuid(Guid);

		DataflowAsset.AddSubGraph(NewSubGraph);

		// Broadcast Pasted (not Created) so listeners refresh their views but don't auto-open
		// the subgraph in a tab - the paste happened in a different graph.
		FDataflowAssetDelegates::OnSubGraphsChanged.Broadcast(&DataflowAsset, NewSubGraph->GetSubGraphGuid(), UE::Dataflow::ESubGraphChangedReason::Pasted);

		// SetForEachSubGraph may broadcast ChangedType - run it after the subgraph is registered
		// on the asset so listeners can resolve its GUID against UDataflow::FindSubGraphByGuid.
		if (bIsForEach)
		{
			NewSubGraph->SetForEachSubGraph(true);
		}

		return NewSubGraph;
	}

	bool FEditAssetUtils::IsUniqueDataflowSubObjectName(UDataflow* DataflowAsset, FName SubObjectName)
	{
		return ::IsUniqueObjectName(SubObjectName, DataflowAsset);
	}

	TSharedPtr<IAssetReferenceFilter> FEditAssetUtils::MakeAssetReferenceFilter(const UEdGraph* Graph)
	{
		if (const UDataflow* Dataflow = UDataflow::GetDataflowAssetFromEdGraph(Graph))
		{
			if (GEditor)
			{
				FAssetReferenceFilterContext AssetReferenceFilterContext;
				AssetReferenceFilterContext.AddReferencingAsset(Dataflow);
				return GEditor->MakeAssetReferenceFilter(AssetReferenceFilterContext);
			}
		}

		return {};
	}

	//////////////////////////////////////////////////////////////////////////////////////////////////////////
	// 
	// NODE API
	// 
	//////////////////////////////////////////////////////////////////////////////////////////////////////////
	UDataflowEdNode* FEditAssetUtils::AddNewNode(UEdGraph* EdGraph, const FVector2D& Location, const FName NodeName, const FName NodeTypeName, UEdGraphPin* FromPin)
	{
		using namespace EditAssetUtils::Private;

		const FText TransactionName{ LOCTEXT("AddNewDataflowNode", "Add New Dataflow Node") };

		UDataflowEdNode* EdNodeToReturn = nullptr;

		if (UDataflow* Dataflow = UDataflow::GetDataflowAssetFromEdGraph(EdGraph))
		{
			auto AddNewNodeInternal =
				[&Dataflow, &EdGraph, &Location, &NodeName, &NodeTypeName, &FromPin, &EdNodeToReturn]
				(UDataflow& DataflowAsset) -> EChangeResult
				{
					if (TSharedPtr<FDataflowNode> DataflowNode = AddDataflowNode(Dataflow, NodeName, NodeTypeName))
					{
						if (UDataflowEdNode* EdNode = CreateDataflowEdNode(EdGraph, DataflowNode, Location, FromPin))
						{
							EdNodeToReturn = EdNode;
							return EChangeResult::Changed;
						}
					}
					return EChangeResult::None;
				};

			ChangeDataflowAssetWithTransaction(Dataflow, TransactionName, AddNewNodeInternal, NAME_None);
		}
		return EdNodeToReturn;
	}

	//////////////////////////////////////////////////////////////////////////////////////////////////////////
	UEdGraphNode* FEditAssetUtils::AddNewComment(UEdGraph* EdGraph, const FVector2D& Location, const FVector2D& Size)
	{
		using namespace EditAssetUtils::Private;

		const FText TransactionName{ LOCTEXT("AddNewDataflowComment", "Add New Dataflow Comment") };

		UEdGraphNode* EdNodeToReturn = nullptr;

		if (UDataflow* Dataflow = UDataflow::GetDataflowAssetFromEdGraph(EdGraph))
		{
			auto AddNewCommentInternal =
				[&Dataflow, &EdGraph, &Location, &Size, &EdNodeToReturn]
				(UDataflow& DataflowAsset) -> EChangeResult
				{
					const FString DefaultText = "Comment";
					const FLinearColor DefaultColor = FLinearColor::White;
					const int32 DefaultFontSize = 18;
					if (UEdGraphNode* EdNode = CreateCommentEdNode(EdGraph, Location, DefaultText, Size, DefaultColor, DefaultFontSize))
					{
						EdNodeToReturn = EdNode;
						return EChangeResult::Changed;
					}
					return EChangeResult::None;
				};

			ChangeDataflowAssetWithTransaction(Dataflow, TransactionName, AddNewCommentInternal, NAME_None);
		}
		return EdNodeToReturn;
	}
	
	//////////////////////////////////////////////////////////////////////////////////////////////////////////
	void FEditAssetUtils::DeleteNodes(UEdGraph* EdGraph, const TArray<UEdGraphNode*>& NodesToDelete)
	{
		using namespace EditAssetUtils::Private;

		const FText TransactionName{ LOCTEXT("DeleteDataflowNodes", "Delete Dataflow Nodes") };

		if (UDataflow* Dataflow = UDataflow::GetDataflowAssetFromEdGraph(EdGraph))
		{
			auto DeleteNodesInternal =
				[&EdGraph, &NodesToDelete]
				(UDataflow& DataflowAsset) -> EChangeResult
				{
					DeleteNodesNoTransaction(EdGraph, NodesToDelete);
					return EChangeResult::Changed;
				};

			ChangeDataflowAssetWithTransaction(Dataflow, TransactionName, DeleteNodesInternal, NAME_None);
		}
	}

	//////////////////////////////////////////////////////////////////////////////////////////////////////////
	void FEditAssetUtils::DuplicateNodes(UEdGraph* EdGraph, const TArray<UEdGraphNode*>& EdNodesToDuplicate, const FVector2D& Location, TArray<UEdGraphNode*>& OutDuplicatedNodes)
	{
		TMap<FGuid, FGuid> NodeGuidMap;
		DuplicateNodes(EdGraph, EdNodesToDuplicate, EdGraph, Location, OutDuplicatedNodes, NodeGuidMap);
	}

	void FEditAssetUtils::DuplicateNodes(UEdGraph* SourceEdGraph, const TArray<UEdGraphNode*>& EdNodesToDuplicate, UEdGraph* TargetEdGraph, const FVector2D& Location, TArray<UEdGraphNode*>& OutDuplicatedNodes, TMap<FGuid, FGuid>& OutNodeGuidMap)
	{
		using namespace EditAssetUtils::Private;

		UDataflow* SourceDataflowAsset = UDataflow::GetDataflowAssetFromEdGraph(SourceEdGraph);
		UDataflow* TargetDataflowAsset = UDataflow::GetDataflowAssetFromEdGraph(TargetEdGraph);
		if (!SourceDataflowAsset || !TargetDataflowAsset)
		{
			// no graph to copy to
			return;
		}

		if (EdNodesToDuplicate.Num() == 0)
		{
			return;
		}

		const FText TransactionName = FText::Format(LOCTEXT("DuplicateDataflowNode", "Duplicate {0} Dataflow Nodes"), FText::AsNumber(EdNodesToDuplicate.Num()));

		// location of the first node as a reference for all the others 
		const FVector2D RefLocation
		{
			(double)EdNodesToDuplicate[0]->NodePosX,
			(double)EdNodesToDuplicate[0]->NodePosY
		};

		auto DuplicateNodesInternal =
			[&OutNodeGuidMap, &SourceEdGraph, TargetEdGraph, &Location, &RefLocation, &EdNodesToDuplicate, &OutDuplicatedNodes]
			(UDataflow& TargetDataflowAsset) -> EChangeResult
			{
				TMap<FGuid, UDataflowEdNode*> EdNodeMap;
				TMap<FGuid, FGuid> NodeGuidMap;

				// copy the nodes and comments first 
				for (UEdGraphNode* EdNodeToDuplicate : EdNodesToDuplicate)
				{
					const FVector2D OriginalNodeLocation
					{
						(double)EdNodeToDuplicate->NodePosX,
						(double)EdNodeToDuplicate->NodePosY
					};
					const FVector2D NodeLocation(Location + (OriginalNodeLocation - RefLocation));

					if (UDataflowEdNode* DataflowEdNodeToDuplicate = Cast<UDataflowEdNode>(EdNodeToDuplicate))
					{
						if (const TSharedPtr<FDataflowNode> NodeToDuplicate = DataflowEdNodeToDuplicate->GetDataflowNode())
						{
							const FName NodeName = NodeToDuplicate->GetName();
							const FName NodeTypeName = NodeToDuplicate->GetType();

							if (TSharedPtr<FDataflowNode> DataflowNode = AddDataflowNode(&TargetDataflowAsset, NodeName, NodeTypeName))
							{
								SDataflowEdNode::CopyDataflowNodeSettings(NodeToDuplicate, DataflowNode);
	
								if (UDataflowEdNode* EdNode = CreateDataflowEdNode(TargetEdGraph, DataflowNode, NodeLocation, /*FromPin*/nullptr))
								{
									EdNodeMap.Add(DataflowEdNodeToDuplicate->DataflowNodeGuid, EdNode);
									NodeGuidMap.Add(DataflowEdNodeToDuplicate->DataflowNodeGuid, EdNode->DataflowNodeGuid);
									OutDuplicatedNodes.Add(EdNode);
								}
							}
						}
					}
					else if (const UEdGraphNode_Comment* CommentEdNodeToDuplicate = Cast<const UEdGraphNode_Comment>(EdNodeToDuplicate))
					{
						const FVector2D CommentSize
						{
							(double)CommentEdNodeToDuplicate->NodeWidth,
							(double)CommentEdNodeToDuplicate->NodeHeight
						};
						if (UEdGraphNode_Comment* CommentEdNode = CreateCommentEdNode(TargetEdGraph, NodeLocation, CommentEdNodeToDuplicate->NodeComment, CommentSize, CommentEdNodeToDuplicate->CommentColor, CommentEdNodeToDuplicate->FontSize))
						{
							OutDuplicatedNodes.Add(CommentEdNode);
						}
					}
				}

				// Recreate connections between duplicated nodes
				for (const UEdGraphNode* EdNodeToDuplicate : EdNodesToDuplicate)
				{
					if (const UDataflowEdNode* DataflowEdNodeToDuplicate = Cast<const UDataflowEdNode>(EdNodeToDuplicate))
					{
						if (const TSharedPtr<const FDataflowNode> DataflowNode = DataflowEdNodeToDuplicate->GetDataflowNode())
						{
							const FGuid DataflowNodeAGuid = DataflowNode->GetGuid();
							for (const FDataflowOutput* Output : DataflowNode->GetOutputs())
							{
								for (const FDataflowInput* Connection : Output->Connections)
								{
									const FName OutputputName = Connection->GetConnection()->GetName();

									// Check if the node on the end of the connection was duplicated
									const FGuid DataflowNodeBGuid = Connection->GetOwningNode()->GetGuid();

									if (NodeGuidMap.Contains(DataflowNodeBGuid))
									{
										const FName InputputName = Connection->GetName();
										if (TSharedPtr<FDataflowNode> DuplicatedDataflowNodeA = TargetDataflowAsset.GetDataflow()->FindBaseNode(NodeGuidMap[DataflowNodeAGuid]))
										{
											FDataflowOutput* OutputConnection = DuplicatedDataflowNodeA->FindOutput(OutputputName);
											if (TSharedPtr<FDataflowNode> DuplicatedDataflowNodeB = TargetDataflowAsset.GetDataflow()->FindBaseNode(NodeGuidMap[DataflowNodeBGuid]))
											{
												FDataflowInput* InputConnection = DuplicatedDataflowNodeB->FindInput(InputputName);

												TargetDataflowAsset.GetDataflow()->Connect(OutputConnection, InputConnection);

												// Connect the UDataflowEdNode FPins as well
												if (UEdGraphPin* OutputPin = FindPin(EdNodeMap[DataflowNodeAGuid], EEdGraphPinDirection::EGPD_Output, OutputputName))
												{
													if (UEdGraphPin* InputPin = FindPin(EdNodeMap[DataflowNodeBGuid], EEdGraphPinDirection::EGPD_Input, InputputName))
													{
														OutputPin->MakeLinkTo(InputPin);
													}
												}
											}
										}
									}
								}
							}
						}
					}
				}
				OutNodeGuidMap = MoveTemp(NodeGuidMap);
				return EChangeResult::Changed;
			};

		ChangeDataflowAssetWithTransaction(TargetDataflowAsset, TransactionName, DuplicateNodesInternal, NAME_None);
	}

	//////////////////////////////////////////////////////////////////////////////////////////////////////////
	void FEditAssetUtils::CopyNodesToClipboard(const TArray<const UEdGraphNode*>& NodesToCopy, int32& OutNumCopiedNodes)
	{
		using namespace EditAssetUtils::Private;

		FDataflowCopyPasteContent CopyPasteContent;

		// Top-level selection serialization.
		SerializeNodesToPayload(NodesToCopy, CopyPasteContent.NodeData, CopyPasteContent.CommentNodeData, CopyPasteContent.ConnectionData);

		// Resolve the source asset (any node in the selection lives in the same UDataflow).
		const UDataflow* SourceDataflowAsset = nullptr;
		for (const UEdGraphNode* EdNode : NodesToCopy)
		{
			if (EdNode)
			{
				SourceDataflowAsset = UDataflow::GetDataflowAssetFromEdGraph(EdNode->GetGraph());
				if (SourceDataflowAsset)
				{
					break;
				}
			}
		}

		// Walk every FDataflowCallSubGraphNode in the selection and embed the referenced subgraphs
		// (transitively, since a subgraph may itself contain call nodes pointing at other subgraphs).
		if (SourceDataflowAsset)
		{
			auto CollectCallSubGraphGuids = [](const TArray<const UEdGraphNode*>& Nodes, TArray<FGuid>& OutGuids)
			{
				for (const UEdGraphNode* EdNode : Nodes)
				{
					if (const UDataflowEdNode* DataflowEdNode = Cast<const UDataflowEdNode>(EdNode))
					{
						if (TSharedPtr<const FDataflowNode> DataflowNode = DataflowEdNode->GetDataflowNode())
						{
							if (const FDataflowCallSubGraphNode* CallNode = DataflowNode->AsType<const FDataflowCallSubGraphNode>())
							{
								if (CallNode->GetSubGraphGuid().IsValid())
								{
									OutGuids.Add(CallNode->GetSubGraphGuid());
								}
							}
						}
					}
				}
			};

			TSet<FGuid> VisitedSubGraphGuids;
			TArray<FGuid> PendingGuids;
			CollectCallSubGraphGuids(NodesToCopy, PendingGuids);

			while (PendingGuids.Num())
			{
				const FGuid SubGraphGuid = PendingGuids.Pop(EAllowShrinking::No);
				bool bAlreadyVisited = false;
				VisitedSubGraphGuids.Add(SubGraphGuid, &bAlreadyVisited);
				if (bAlreadyVisited)
				{
					continue;
				}

				const UDataflowSubGraph* SubGraph = SourceDataflowAsset->FindSubGraphByGuid(SubGraphGuid);
				if (!SubGraph)
				{
					continue;
				}

				FDataflowSubGraphData SubGraphData;
				SubGraphData.SubGraphGuid = SubGraph->GetSubGraphGuid();
				SubGraphData.Name = SubGraph->GetFName().ToString();
				SubGraphData.bIsForEach = SubGraph->IsForEachSubGraph();

				TArray<const UEdGraphNode*> SubGraphNodes;
				SubGraphNodes.Reserve(SubGraph->Nodes.Num());
				for (const UEdGraphNode* SubGraphEdNode : SubGraph->Nodes)
				{
					SubGraphNodes.Add(SubGraphEdNode);
				}

				SerializeNodesToPayload(SubGraphNodes, SubGraphData.NodeData, SubGraphData.CommentNodeData, SubGraphData.ConnectionData);

				// Queue any nested subgraph references for the next iteration.
				CollectCallSubGraphGuids(SubGraphNodes, PendingGuids);

				CopyPasteContent.SubGraphData.Emplace(MoveTemp(SubGraphData));
			}
		}

		// copy to clipboard
		if (CopyPasteContent.NodeData.Num() ||
			CopyPasteContent.CommentNodeData.Num() ||
			CopyPasteContent.ConnectionData.Num() ||
			CopyPasteContent.SubGraphData.Num())
		{
			FString ClipboardContent;
			const FDataflowCopyPasteContent DefaultContent;
			FDataflowCopyPasteContent::StaticStruct()->ExportText(ClipboardContent, &CopyPasteContent, &DefaultContent, nullptr, PPF_None, nullptr);

			FPlatformApplicationMisc::ClipboardCopy(*ClipboardContent);
		}

		OutNumCopiedNodes = CopyPasteContent.NodeData.Num() + CopyPasteContent.CommentNodeData.Num();
	}

	//////////////////////////////////////////////////////////////////////////////////////////////////////////
	void FEditAssetUtils::PasteNodesFromClipboard(UEdGraph* EdGraph, const FVector2D& Location, TArray<UEdGraphNode*>& OutPastedNodes)
	{
		using namespace EditAssetUtils::Private;

		UDataflow* DataflowAsset = UDataflow::GetDataflowAssetFromEdGraph(EdGraph);
		if (!DataflowAsset)
		{
			// no graph to copy to
			return;
		}

		FString ClipboardPayload;
		FPlatformApplicationMisc::ClipboardPaste(ClipboardPayload);

		if (ClipboardPayload.IsEmpty())
		{
			// nothing to paste, nothing to do
			return;
		}

		const FDataflowCopyPasteContent DefaultContent;
		FDataflowCopyPasteContent CopyPasteContent;
		FDataflowCopyPasteContent::StaticStruct()->ImportText(*ClipboardPayload, &CopyPasteContent, nullptr, EPropertyPortFlags::PPF_None, nullptr, FDataflowCopyPasteContent::StaticStruct()->GetName(), true);

		const int32 TotalNodesToPaste = CopyPasteContent.NodeData.Num() + CopyPasteContent.CommentNodeData.Num();
		const FText TransactionName = FText::Format(LOCTEXT("PasteDataflowNodes", "Paste {0} Dataflow Nodes"), FText::AsNumber(TotalNodesToPaste));

		auto PasteNodesInternal =
		[&EdGraph, &CopyPasteContent, &Location, &OutPastedNodes](UDataflow& DataflowAsset)->EChangeResult
		{
			// 1) Recreate any subgraphs from the payload that the destination doesn't already have
			//    (matched by GUID). Two-pass: create-all-then-fill so nested call-subgraph references
			//    resolve regardless of the array order in the payload.
			//
			//    A newly created subgraph is given a *fresh* GUID rather than the original one. This
			//    intentionally breaks the GUID linkage between source and destination so that later
			//    edits in either asset don't silently appear "the same subgraph" on subsequent copy
			//    passes. Any call nodes pasted in this same operation that referenced the original
			//    GUID are remapped to the fresh GUID at the end (see the remap pass below).
			//
			//    Same-asset paste (e.g. Ctrl+C / Ctrl+V the call node in the same Dataflow asset)
			//    still short-circuits via FindSubGraphByGuid: the existing subgraph is reused with
			//    no recreation and no remap, matching the original behavior.
			TArray<UDataflowSubGraph*> NewlyCreatedSubGraphs;
			NewlyCreatedSubGraphs.Reserve(CopyPasteContent.SubGraphData.Num());

			TMap<FGuid, FGuid> SubGraphGuidRemap; // original GUID -> fresh GUID for subgraphs we just created

			for (int32 SubGraphIndex = 0; SubGraphIndex < CopyPasteContent.SubGraphData.Num(); ++SubGraphIndex)
			{
				const FDataflowSubGraphData& SubGraphData = CopyPasteContent.SubGraphData[SubGraphIndex];
				if (!SubGraphData.SubGraphGuid.IsValid())
				{
					NewlyCreatedSubGraphs.Add(nullptr);
					continue;
				}

				if (DataflowAsset.FindSubGraphByGuid(SubGraphData.SubGraphGuid))
				{
					// destination already has this subgraph (same GUID) - leave it untouched
					NewlyCreatedSubGraphs.Add(nullptr);
					continue;
				}

				const FGuid FreshGuid = FGuid::NewGuid();
				SubGraphGuidRemap.Add(SubGraphData.SubGraphGuid, FreshGuid);

				const FName UniqueName = GenerateUniqueObjectName(&DataflowAsset, FName(*SubGraphData.Name));
				UDataflowSubGraph* NewSubGraph = CreateSubGraphWithGuid(DataflowAsset, UniqueName, FreshGuid, SubGraphData.bIsForEach);
				NewlyCreatedSubGraphs.Add(NewSubGraph);
			}

			for (int32 SubGraphIndex = 0; SubGraphIndex < CopyPasteContent.SubGraphData.Num(); ++SubGraphIndex)
			{
				UDataflowSubGraph* NewSubGraph = NewlyCreatedSubGraphs[SubGraphIndex];
				if (!NewSubGraph)
				{
					continue;
				}

				const FDataflowSubGraphData& SubGraphData = CopyPasteContent.SubGraphData[SubGraphIndex];

				// Preserve original positions inside subgraphs by aligning Location with the payload's
				// first-node position (the helper subtracts RefLocation, so the offset cancels out).
				FVector2D SubGraphRefLocation { 0, 0 };
				if (SubGraphData.NodeData.Num())
				{
					SubGraphRefLocation.X = SubGraphData.NodeData[0].Position.X;
					SubGraphRefLocation.Y = SubGraphData.NodeData[0].Position.Y;
				}
				else if (SubGraphData.CommentNodeData.Num())
				{
					SubGraphRefLocation.X = SubGraphData.CommentNodeData[0].Position.X;
					SubGraphRefLocation.Y = SubGraphData.CommentNodeData[0].Position.Y;
				}

				TArray<UEdGraphNode*> IgnoredPastedNodes;
				PastePayloadIntoEdGraph(*NewSubGraph, SubGraphRefLocation, SubGraphData.NodeData, SubGraphData.CommentNodeData, SubGraphData.ConnectionData, IgnoredPastedNodes);
			}

			// 2) Top-level paste into the user-clicked graph.
			PastePayloadIntoEdGraph(*EdGraph, Location, CopyPasteContent.NodeData, CopyPasteContent.CommentNodeData, CopyPasteContent.ConnectionData, OutPastedNodes);

			// 3) Remap call-subgraph references in any pasted node (top-level + inside recreated subgraphs)
			//    so they point at the fresh GUIDs assigned in pass 1. Calling SetSubGraphGuid forces the
			//    call node to re-resolve against the destination asset and re-sync its dynamic pin bag.
			if (SubGraphGuidRemap.Num() > 0)
			{
				auto RemapCallNodes = [&SubGraphGuidRemap](TArrayView<UEdGraphNode* const> Nodes)
				{
					for (UEdGraphNode* EdNode : Nodes)
					{
						if (UDataflowEdNode* DataflowEdNode = Cast<UDataflowEdNode>(EdNode))
						{
							if (TSharedPtr<FDataflowNode> DataflowNode = DataflowEdNode->GetDataflowNode())
							{
								if (FDataflowCallSubGraphNode* CallNode = DataflowNode->AsType<FDataflowCallSubGraphNode>())
								{
									if (const FGuid* RemappedGuid = SubGraphGuidRemap.Find(CallNode->GetSubGraphGuid()))
									{
										CallNode->SetSubGraphGuid(*RemappedGuid);
									}
								}
							}
						}
					}
				};

				RemapCallNodes(OutPastedNodes);
				for (UDataflowSubGraph* NewSubGraph : NewlyCreatedSubGraphs)
				{
					if (NewSubGraph)
					{
						RemapCallNodes(NewSubGraph->Nodes);
					}
				}
			}

			return EChangeResult::Changed;
		};

		// make sure we notify that variables may have changed as a property since we may have pasted variable nodes
		ChangeDataflowAssetWithTransaction(DataflowAsset, TransactionName, PasteNodesInternal, GET_MEMBER_NAME_CHECKED(UDataflow, Variables));
	}

	//////////////////////////////////////////////////////////////////////////////////////////////////////////
	// 
	// VARIABLES API
	// 
	//////////////////////////////////////////////////////////////////////////////////////////////////////////
	FName FEditAssetUtils::AddNewVariable(UDataflow* DataflowAsset, FName BaseName)
	{
		using namespace EditAssetUtils::Private;

		const FText TransactionName{ LOCTEXT("AddNewDataflowVariable", "Add New Dataflow Variable") };

		FName UniqueVariableName;

		auto AddNewVariableInternal =
			[&BaseName, &UniqueVariableName](UDataflow& DataflowAsset) -> EChangeResult
			{
				UniqueVariableName = GenerateUniqueVariableName(DataflowAsset, BaseName);
				DataflowAsset.Variables.AddProperty(UniqueVariableName, EPropertyBagPropertyType::Int32);
				FDataflowAssetDelegates::OnVariablesChanged.Broadcast(&DataflowAsset, UniqueVariableName);
				return EChangeResult::Changed;
			};

		ChangeDataflowAssetWithTransaction(DataflowAsset, TransactionName, AddNewVariableInternal, GET_MEMBER_NAME_CHECKED(UDataflow, Variables));

		return UniqueVariableName;
	}

	FName FEditAssetUtils::AddNewVariable(UDataflow* DataflowAsset, FName BaseName, const FPropertyBagPropertyDesc& TemplateDesc)
	{
		using namespace EditAssetUtils::Private;

		const FText TransactionName{ LOCTEXT("AddNewDataflowVariable", "Add New Dataflow Variable") };

		FName UniqueVariableName;

		auto AddNewVariableInternal =
			[&BaseName, &UniqueVariableName, &TemplateDesc](UDataflow& DataflowAsset) -> EChangeResult
			{
				UniqueVariableName = GenerateUniqueVariableName(DataflowAsset, BaseName);
				FPropertyBagPropertyDesc Desc{ TemplateDesc };
				Desc.Name = UniqueVariableName;
				DataflowAsset.Variables.AddProperties({ Desc });
				FDataflowAssetDelegates::OnVariablesChanged.Broadcast(&DataflowAsset, UniqueVariableName);
				return EChangeResult::Changed;
			};

		ChangeDataflowAssetWithTransaction(DataflowAsset, TransactionName, AddNewVariableInternal, GET_MEMBER_NAME_CHECKED(UDataflow, Variables));

		return UniqueVariableName;
	}

	//////////////////////////////////////////////////////////////////////////////////////////////////////////
	void FEditAssetUtils::DeleteVariable(UDataflow* DataflowAsset, FName VariableName)
	{
		using namespace EditAssetUtils::Private;

		const FText TransactionName{ LOCTEXT("DeleteDataflowVariable", "Delete Dataflow Variable") };

		auto DeleteVariableInternal =
			[&VariableName](UDataflow& DataflowAsset) -> EChangeResult
			{
				DataflowAsset.Variables.RemovePropertyByName(VariableName);
				FDataflowAssetDelegates::OnVariablesChanged.Broadcast(&DataflowAsset, VariableName);
				return EChangeResult::Changed;
			};

		ChangeDataflowAssetWithTransaction(DataflowAsset, TransactionName, DeleteVariableInternal, GET_MEMBER_NAME_CHECKED(UDataflow, Variables));
	}

	//////////////////////////////////////////////////////////////////////////////////////////////////////////
	void FEditAssetUtils::RenameVariable(UDataflow* DataflowAsset, FName OldVariableName, FName NewVariableName)
	{
		using namespace EditAssetUtils::Private;

		const FText TransactionName{ LOCTEXT("RenameDataflowVariable", "Rename Dataflow Variable") };

		auto SetVariableNameInternal =
			[OldVariableName, NewVariableName](UDataflow& DataflowAsset) -> EChangeResult
			{
				auto ChangePropertyNameLambda =
					[NewVariableName](FPropertyBagPropertyDesc& PropertyDesc)
					{
						PropertyDesc.Name = NewVariableName;
					};

				if (ModifyVariable(DataflowAsset, OldVariableName, ChangePropertyNameLambda))
				{
					RenameVariableCallNodes(DataflowAsset, OldVariableName, NewVariableName);
					return EChangeResult::Changed;
				}
				return EChangeResult::None;
			};

		ChangeDataflowAssetWithTransaction(DataflowAsset, TransactionName, SetVariableNameInternal, GET_MEMBER_NAME_CHECKED(UDataflow, Variables));
	}

	//////////////////////////////////////////////////////////////////////////////////////////////////////////
	FName FEditAssetUtils::DuplicateVariable(UDataflow* DataflowAsset, FName VariableName)
	{
		using namespace EditAssetUtils::Private;

		const FText TransactionName = FText::Format(LOCTEXT("DuplicateDataflowVariable", "Duplicate Dataflow Variable: {0}"), FText::FromName(VariableName));
		
		FName NewVariableName;

		auto DuplicateVariableInternal =
			[VariableName, &NewVariableName](UDataflow& DataflowAsset) -> EChangeResult
			{
				if (const FPropertyBagPropertyDesc* PropertyDescPtr = DataflowAsset.Variables.FindPropertyDescByName(VariableName))
				{
					NewVariableName = GenerateUniqueVariableName(DataflowAsset, VariableName);

					// make sure the name is unique and the GUID is invalidated to avoid copying it as is 
					FPropertyBagPropertyDesc NewDesc(*PropertyDescPtr);
					NewDesc.Name = NewVariableName;
					NewDesc.ID.Invalidate();
					DataflowAsset.Variables.AddProperties(MakeConstArrayView(&NewDesc, 1));
					FDataflowAssetDelegates::OnVariablesChanged.Broadcast(&DataflowAsset, NewVariableName);
					return EChangeResult::Changed;
				}
				return EChangeResult::None;
			};

		ChangeDataflowAssetWithTransaction(DataflowAsset, TransactionName, DuplicateVariableInternal, GET_MEMBER_NAME_CHECKED(UDataflow, Variables));

		return NewVariableName;
	}


	//////////////////////////////////////////////////////////////////////////////////////////////////////////
	void FEditAssetUtils::SetVariableType(UDataflow* DataflowAsset, FName VariableName, const FEdGraphPinType& PinType)
	{
		using namespace EditAssetUtils::Private;

		const FText TransactionName{ LOCTEXT("ChangeDataflowVariableType", "Change Dataflow Variable Type") };

		auto SetVariableTypeInternal =
			[VariableName, &PinType](UDataflow& DataflowAsset) -> EChangeResult
			{
				auto ChangePropertyTypeLambda =
					[VariableName, &PinType](FPropertyBagPropertyDesc& PropertyDesc)
					{
						UE::StructUtils::SetPropertyDescFromPin(PropertyDesc, PinType);
					};

				const bool bModified = ModifyVariable(DataflowAsset, VariableName, ChangePropertyTypeLambda);
				return (bModified) ? EChangeResult::Changed : EChangeResult::None;
			};

		ChangeDataflowAssetWithTransaction(DataflowAsset, TransactionName, SetVariableTypeInternal, GET_MEMBER_NAME_CHECKED(UDataflow, Variables));
	}

	//////////////////////////////////////////////////////////////////////////////////////////////////////////
	void FEditAssetUtils::SetVariableValue(UDataflow* DataflowAsset, FName VariableName, const FInstancedPropertyBag& SourceBag)
	{
		using namespace EditAssetUtils::Private;

		const FText TransactionName{ LOCTEXT("ChangeDataflowVariableValue", "Change Dataflow Variable Value") };

		auto SetVariableValueInternal =
			[VariableName, &SourceBag](UDataflow& DataflowAsset) -> EChangeResult
			{
				if (const FPropertyBagPropertyDesc* SourceDesc = SourceBag.FindPropertyDescByName(VariableName))
				{
					if (SourceDesc->CachedProperty)
					{
						EPropertyBagResult Result = DataflowAsset.Variables.SetValue(VariableName, SourceDesc->CachedProperty, SourceBag.GetValue().GetMemory());
						if (Result == EPropertyBagResult::Success)
						{
							FDataflowAssetDelegates::OnVariablesChanged.Broadcast(&DataflowAsset, VariableName);
							return EChangeResult::Changed;
						}
					}
				}
				return EChangeResult::None;
			};

		// do we need a transction in that case ? 
		ChangeDataflowAssetWithTransaction(DataflowAsset, TransactionName, SetVariableValueInternal, GET_MEMBER_NAME_CHECKED(UDataflow, Variables));
	}

	//////////////////////////////////////////////////////////////////////////////////////////////////////////
	void FEditAssetUtils::CopyVariableToClipboard(UDataflow* DataflowAsset, FName VariableName)
	{
		using namespace EditAssetUtils::Private;

		if (DataflowAsset)
		{
			// no transaction needed in that case as we write to an external system 
			if (const FPropertyBagPropertyDesc* PropertyDescPtr = DataflowAsset->Variables.FindPropertyDescByName(VariableName))
			{
				FString ClipboardPayload;

				FPropertyBagPropertyDesc::StaticStruct()->ExportText(ClipboardPayload, PropertyDescPtr, PropertyDescPtr, nullptr, 0, nullptr, false);

				if (!ClipboardPayload.IsEmpty())
				{
					ClipboardPayload = DataflowVariableClipboardPrefix + ClipboardPayload;
					FPlatformApplicationMisc::ClipboardCopy(ClipboardPayload.GetCharArray().GetData());
				}
			}
		}
	}

	//////////////////////////////////////////////////////////////////////////////////////////////////////////
	FName FEditAssetUtils::PasteVariableFromClipboard(UDataflow* DataflowAsset)
	{
		using namespace EditAssetUtils::Private;

		FString ClipboardPayload;
		FPlatformApplicationMisc::ClipboardPaste(ClipboardPayload);

		if (!ensure(ClipboardPayload.StartsWith(DataflowVariableClipboardPrefix, ESearchCase::CaseSensitive)))
		{
			return NAME_None;
		}

		FStringOutputDevice Errors;
		const TCHAR* ImportPayload = ClipboardPayload.GetCharArray().GetData() + FCString::Strlen(DataflowVariableClipboardPrefix);

		FPropertyBagPropertyDesc PropertyDesc;
		FPropertyBagPropertyDesc::StaticStruct()->ImportText(ImportPayload, &PropertyDesc, nullptr, PPF_None, &Errors, FPropertyBagPropertyDesc::StaticStruct()->GetName());

		if (Errors.IsEmpty())
		{
			if (DataflowAsset)
			{
				// make sure the name is unique and the GUID is invalidated to avoid copying it as is 
				PropertyDesc.Name = GenerateUniqueVariableName(*DataflowAsset, PropertyDesc.Name);
				PropertyDesc.ID.Invalidate();

				const FText TransactionName = FText::Format(LOCTEXT("PasteDataflowVariable", "Paste Dataflow Variable: {0}"), FText::FromName(PropertyDesc.Name));

				auto PasteVariableInternal =
					[&PropertyDesc](UDataflow& DataflowAsset) -> EChangeResult
					{
						DataflowAsset.Variables.AddProperties(MakeConstArrayView(&PropertyDesc, 1));
						FDataflowAssetDelegates::OnVariablesChanged.Broadcast(&DataflowAsset, PropertyDesc.Name);
						return EChangeResult::Changed;
					};

				ChangeDataflowAssetWithTransaction(DataflowAsset, TransactionName, PasteVariableInternal, GET_MEMBER_NAME_CHECKED(UDataflow, Variables));
				return PropertyDesc.Name;
			}
		}
		return NAME_None;
	}

	//////////////////////////////////////////////////////////////////////////////////////////////////////////
	// 
	// SUBGRAPHS API
	// 
	//////////////////////////////////////////////////////////////////////////////////////////////////////////
	FName FEditAssetUtils::AddNewSubGraph(UDataflow* DataflowAsset, FName BaseName)
	{
		using namespace EditAssetUtils::Private;

		const FText TransactionName{ LOCTEXT("AddNewDataflowSubGraph", "Add New Dataflow SubGraph") };

		FName UniqueSubGraphName;

		auto AddNewVariableInternal =
			[&BaseName, &UniqueSubGraphName](UDataflow& DataflowAsset) -> EChangeResult
			{
				UniqueSubGraphName = GenerateUniqueObjectName(&DataflowAsset, BaseName);
				UDataflowSubGraph* NewSubGraph = NewObject<UDataflowSubGraph>(&DataflowAsset, UniqueSubGraphName);
				ensure(NewSubGraph->GetFName().IsEqual(UniqueSubGraphName));
				NewSubGraph->Schema = UDataflowSchema::StaticClass();
				NewSubGraph->SetFlags(RF_Transactional);

				DataflowAsset.AddSubGraph(NewSubGraph);

				FDataflowAssetDelegates::OnSubGraphsChanged.Broadcast(&DataflowAsset, NewSubGraph->GetSubGraphGuid(), UE::Dataflow::ESubGraphChangedReason::Created);
				return EChangeResult::Changed;
			};

		ChangeDataflowAssetWithTransaction(DataflowAsset, TransactionName, AddNewVariableInternal, NAME_None);

		return UniqueSubGraphName;
	}

	void FEditAssetUtils::RenameSubGraph(UDataflow* DataflowAsset, FName OldSubGraphName, FName NewSubGraphName)
	{
		using namespace EditAssetUtils::Private;

		if (DataflowAsset && IsUniqueDataflowSubObjectName(DataflowAsset, NewSubGraphName))
		{
			if (UDataflowSubGraph* SubGraphToRename = DataflowAsset->FindSubGraphByName(OldSubGraphName))
			{
				const FText TransactionName{ LOCTEXT("RenameDataflowSubGraph", "Rename a Dataflow SubGraph") };

				auto RenameSubGraphInternal =
					[&SubGraphToRename, &NewSubGraphName](UDataflow& DataflowAsset) -> EChangeResult
					{
						if (SubGraphToRename->Rename(*NewSubGraphName.ToString()))
						{
							// rename the call nodes using it 
							RenameSubGraphCallNodes(DataflowAsset, SubGraphToRename->GetSubGraphGuid(), NewSubGraphName);

							FDataflowAssetDelegates::OnSubGraphsChanged.Broadcast(&DataflowAsset, SubGraphToRename->GetSubGraphGuid(), UE::Dataflow::ESubGraphChangedReason::Renamed);
							return EChangeResult::Changed;
						}
						return EChangeResult::Cancel;
					};

				ChangeDataflowAssetWithTransaction(DataflowAsset, TransactionName, RenameSubGraphInternal, NAME_None);
			}
		}
	}

	void FEditAssetUtils::DeleteSubGraph(UDataflow* DataflowAsset, FGuid SubGraphGuid)
	{
		using namespace EditAssetUtils::Private;

		if (DataflowAsset)
		{
			if (UDataflowSubGraph* SubGraphToDelete = DataflowAsset->FindSubGraphByGuid(SubGraphGuid))
			{
				const FText TransactionName{ LOCTEXT("DeleteDataflowSubGraph", "Delete a Dataflow SubGraph") };

				auto DeleteSubGraphInternal =
					[&SubGraphToDelete, SubGraphGuid](UDataflow& DataflowAsset) -> EChangeResult
					{
						FDataflowAssetDelegates::OnSubGraphsChanged.Broadcast(&DataflowAsset, SubGraphGuid, UE::Dataflow::ESubGraphChangedReason::Deleting);

						// make a copy so that we don't mofdy the array while iterating through it 
						const TArray<UEdGraphNode*> NodesToDelete = SubGraphToDelete->Nodes;
						DeleteNodesNoTransaction(SubGraphToDelete, NodesToDelete);

						// delete the Subgraph 
						DataflowAsset.RemoveSubGraph(SubGraphToDelete);

						FDataflowAssetDelegates::OnSubGraphsChanged.Broadcast(&DataflowAsset, SubGraphGuid, UE::Dataflow::ESubGraphChangedReason::Deleted);
						return EChangeResult::Changed;
					};

				ChangeDataflowAssetWithTransaction(DataflowAsset, TransactionName, DeleteSubGraphInternal, NAME_None);
			}
		}
	}

}

#undef LOCTEXT_NAMESPACE
