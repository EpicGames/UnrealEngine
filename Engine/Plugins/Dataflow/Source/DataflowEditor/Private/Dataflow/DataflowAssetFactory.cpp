// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowAssetFactory.h"

#include "Dataflow/DataflowObject.h"
#include "HAL/IConsoleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowAssetFactory)



UDataflowAssetFactory::UDataflowAssetFactory()
{
	SupportedClass = UDataflow::StaticClass();
}

bool UDataflowAssetFactory::CanCreateNew() const
{
	return true;
}

bool UDataflowAssetFactory::FactoryCanImport(const FString& Filename)
{
	return false;
}

UObject* UDataflowAssetFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags,
												   UObject* Context, FFeedbackContext* Warn)
{
	return NewObject<UDataflow>(InParent, InClass, InName, Flags);
}

bool UDataflowAssetFactory::ShouldShowInNewMenu() const
{
	return true;
}

bool UDataflowAssetFactory::ConfigureProperties()
{
	return true;
}

