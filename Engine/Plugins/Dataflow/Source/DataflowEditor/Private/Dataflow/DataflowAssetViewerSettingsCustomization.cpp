// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataflowAssetViewerSettingsCustomization.h"

#include "AssetViewerSettings.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Delegates/Delegate.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Fonts/SlateFontInfo.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformFileManager.h"
#include "IDetailGroup.h"
#include "IDetailPropertyRow.h"
#include "Internationalization/Internationalization.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "PropertyHandle.h"
#include "SSettingsEditorCheckoutNotice.h"
#include "ScopedTransaction.h"
#include "UObject/NameTypes.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealType.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SEditableTextBox.h"

#define LOCTEXT_NAMESPACE "AssetViewerSettingsCustomizations"

FDataflowAssetViewerSettingsCustomization::FDataflowAssetViewerSettingsCustomization(TSharedPtr<FDataflowPreviewProfileController::IProfileIndexStorage> ProfileIndexStorage) :
	ProfileIndexStorage(ProfileIndexStorage)
{
}

void FDataflowAssetViewerSettingsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	ViewerSettings = UAssetViewerSettings::Get();

	// Create the watcher widget for the default config file (checks file status / SCC state)
	FileWatcherWidget =
		SNew(SSettingsEditorCheckoutNotice)
		.ConfigFilePath(this, &FDataflowAssetViewerSettingsCustomization::SharedProfileConfigFilePath)
		.Visibility(this, &FDataflowAssetViewerSettingsCustomization::ShowFileWatcherWidget);

	// Find profiles array property handle and hide it from settings view
	TSharedPtr<IPropertyHandle> ProfileHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UAssetViewerSettings, Profiles));
	check(ProfileHandle->IsValidHandle());
	ProfileHandle->MarkHiddenByCustomization();

	// Create new category
	IDetailCategoryBuilder& CategoryBuilder = DetailBuilder.EditCategory("Settings", LOCTEXT("AssetViewerSettingsCategory", "Settings"), ECategoryPriority::Important);

	check(ViewerSettings != nullptr);
	ProfileIndex = 0;
	if (ProfileIndexStorage)
	{
		ProfileIndex = ViewerSettings->Profiles.IsValidIndex(ProfileIndexStorage->RetrieveProfileIndex()) ? ProfileIndexStorage->RetrieveProfileIndex() : 0;
	}

	// Add current active profile property child nodes (rest of profiles remain hidden)
	TSharedPtr<IPropertyHandle> ProfilePropertyHandle = ProfileHandle->GetChildHandle(ProfileIndex);
	checkf(ProfileHandle->IsValidHandle(), TEXT("Invalid indexing into profiles child properties"));	
	uint32 PropertyCount = 0;
	ProfilePropertyHandle->GetNumChildren(PropertyCount);

	TMap<FName, IDetailGroup*> SubcategoryToGroupMap;

	FName NamePropertyName = GET_MEMBER_NAME_CHECKED(FPreviewSceneProfile, ProfileName);
	FName SharedProfilePropertyName = GET_MEMBER_NAME_CHECKED(FPreviewSceneProfile, bSharedProfile);
	for (uint32 PropertyIndex = 0; PropertyIndex < PropertyCount; ++PropertyIndex)
	{
		TSharedPtr<IPropertyHandle> ProfileProperty = ProfilePropertyHandle->GetChildHandle(PropertyIndex);
				
		if (ProfileProperty->GetProperty()->GetFName() == NamePropertyName)
		{
			NameProperty = ProfileProperty;
			CategoryBuilder.AddCustomRow(LOCTEXT("PreviewSceneProfileDetails_ProfileNameLabel", "Profile Name"))
					.NameContent()
					[
						ProfileProperty->CreatePropertyNameWidget()
					]
					.ValueContent()
					.MaxDesiredWidth(250.0f)
					[
						// PropertyHandle->CreatePropertyValueWidget()
						SAssignNew(NameEditTextBox, SEditableTextBox)
						.IsEnabled_Lambda([this]()
						{
							return !ViewerSettings->Profiles[ProfileIndex].bIsEngineDefaultProfile;
						})
						.Text(this, &FDataflowAssetViewerSettingsCustomization::OnGetProfileName)
						.OnTextChanged(this, &FDataflowAssetViewerSettingsCustomization::OnProfileNameChanged)
						.OnTextCommitted(this, &FDataflowAssetViewerSettingsCustomization::OnProfileNameCommitted)
						.Font(IDetailLayoutBuilder::GetDetailFont())
					];
		}
		else if (ProfileProperty->GetProperty()->GetFName() == SharedProfilePropertyName)
		{
			IDetailPropertyRow& Row = CategoryBuilder.AddProperty(ProfileProperty);
			Row.EditCondition(TAttribute<bool>(this, &FDataflowAssetViewerSettingsCustomization::CanSetSharedProfile), nullptr);
			
			CategoryBuilder.AddCustomRow(LOCTEXT("PreviewSceneProfileName_CheckoutRow", "Checkout Default Config"))
			.Visibility(TAttribute<EVisibility>(this, &FDataflowAssetViewerSettingsCustomization::ShowFileWatcherWidget))
			[
				FileWatcherWidget->AsShared()
			];
			
		}
		else
		{
			FName DefaultCategoryName = ProfileProperty->GetDefaultCategoryName();

			IDetailGroup*& Group = SubcategoryToGroupMap.FindOrAdd(DefaultCategoryName);

			if (!Group)
			{
				Group = &CategoryBuilder.AddGroup(DefaultCategoryName, ProfileProperty->GetDefaultCategoryText());
			}

			Group->AddPropertyRow(ProfileProperty.ToSharedRef());
		}
	}
}

FText FDataflowAssetViewerSettingsCustomization::OnGetProfileName() const
{
	return FText::FromString(ViewerSettings->Profiles[ProfileIndex].ProfileName);
}

void FDataflowAssetViewerSettingsCustomization::OnProfileNameChanged(const FText& InNewText)
{	
	bValidProfileName = IsProfileNameValid(InNewText.ToString());
	if (!bValidProfileName)
	{
		NameEditTextBox->SetError(LOCTEXT("PreviewSceneProfileName_NotValid", "This name is already in use"));
	}
	else
	{
		NameEditTextBox->SetError(FText::GetEmpty());
	}
}

void FDataflowAssetViewerSettingsCustomization::OnProfileNameCommitted(const FText& InNewText, ETextCommit::Type InTextCommit)
{
	if (bValidProfileName && InTextCommit != ETextCommit::OnCleared)
	{
		const FScopedTransaction Transaction(LOCTEXT("RenameProfile", "Rename Profile"));
		ViewerSettings->Modify();

		ViewerSettings->Profiles[ProfileIndex].ProfileName = InNewText.ToString();		
		FPropertyChangedEvent PropertyEvent(NameProperty->GetProperty());
		ViewerSettings->PostEditChangeProperty(PropertyEvent);
	}

	bValidProfileName = false;
	NameEditTextBox->SetError(FText::GetEmpty());
}

const bool FDataflowAssetViewerSettingsCustomization::IsProfileNameValid(const FString& NewName)
{
	for (int32 CheckProfileIndex = 0; CheckProfileIndex < ViewerSettings->Profiles.Num(); ++CheckProfileIndex)
	{
		if (ProfileIndex != CheckProfileIndex && ViewerSettings->Profiles[CheckProfileIndex].ProfileName == NewName)
		{
			return false;
		}
	}

	return true;
}

bool FDataflowAssetViewerSettingsCustomization::CanSetSharedProfile() const
{
	const bool bIsEngineDefaultProfile = ViewerSettings->Profiles[ProfileIndex].bIsEngineDefaultProfile;
	const bool bIsConfigWriteable = !FPlatformFileManager::Get().GetPlatformFile().IsReadOnly(*SharedProfileConfigFilePath());
	return bIsConfigWriteable && !bIsEngineDefaultProfile;
}

EVisibility FDataflowAssetViewerSettingsCustomization::ShowFileWatcherWidget() const
{
	return CanSetSharedProfile() ? EVisibility::Collapsed : EVisibility::Visible;
}

FString FDataflowAssetViewerSettingsCustomization::SharedProfileConfigFilePath() const
{
	if (ViewerSettings != nullptr)
	{
		return ViewerSettings->GetDefaultConfigFilename();
	}

	return FString();
}

#undef LOCTEXT_NAMESPACE 
