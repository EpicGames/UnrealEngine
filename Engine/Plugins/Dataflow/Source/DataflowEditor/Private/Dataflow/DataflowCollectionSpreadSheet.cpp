// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowCollectionSpreadSheet.h"

#include "Containers/Ticker.h"
#include "Dataflow/DataflowObjectInterface.h"
#include "Dataflow/DataflowCollectionSpreadSheetWidget.h"
#include "Dataflow/DataflowContent.h"
#include "Dataflow/DataflowSelection.h"
#include "Templates/EnableIf.h"

//#include "Dataflow/DataflowEdNode.h"


FDataflowCollectionSpreadSheet::FDataflowCollectionSpreadSheet(TObjectPtr<UDataflowBaseContent> InContent)
	: FDataflowNodeView(InContent)
{

}

void FDataflowCollectionSpreadSheet::SetSupportedOutputTypes()
{
	GetSupportedOutputTypes().Empty();

	GetSupportedOutputTypes().Add("FManagedArrayCollection");
}

void FDataflowCollectionSpreadSheet::UpdateViewData()
{
	TWeakObjectPtr<UDataflowEdNode> WeakSelectedNode = GetSelectedNode();
	TWeakObjectPtr<UDataflowBaseContent> WeakEditorContent = GetEditorContent();

	ExecuteOnGameThread(TEXT("DataflowCollectionSpreadSheetRefresh"), [CollectionSpreadSheet = CollectionSpreadSheet, WeakSelectedNode, WeakEditorContent]() {

		FString NodeName;
		if (CollectionSpreadSheet && CollectionSpreadSheet->GetCollectionTable())
		{
			CollectionSpreadSheet->GetCollectionTable()->GetCollectionInfoMap().Empty();

			if (TStrongObjectPtr<UDataflowEdNode> SelectedEdNode = WeakSelectedNode.Pin())
			{
				if (SelectedEdNode->IsBound())
				{
					if (TSharedPtr<FDataflowNode> DataflowNode = SelectedEdNode->GetDataflowNode())
					{
						TArray<FDataflowOutput*> Outputs = DataflowNode->GetOutputs();
						if (TStrongObjectPtr<UDataflowBaseContent> Content = WeakEditorContent.Pin())
						{
							if (TSharedPtr<UE::Dataflow::FEngineContext> Context = Content->GetDataflowContext())
							{
								for (FDataflowOutput* Output : Outputs)
								{
									if (Output->GetType() == TEXT("FManagedArrayCollection"))
									{
										const FName Name = Output->GetName();
										const FManagedArrayCollection DefaultCollection;
										const FManagedArrayCollection& Value = Output->ReadValue(*Context, DefaultCollection);
										TSharedPtr<const FManagedArrayCollection> CopiedCollectionPtr = MakeShared<const FManagedArrayCollection>(Value);
										CollectionSpreadSheet->GetCollectionTable()->GetCollectionInfoMap().Add(Name.ToString(), { CopiedCollectionPtr });
									}
								}
							}
						}
						NodeName = SelectedEdNode->GetName();
					}
				}
			}
			CollectionSpreadSheet->SetData(NodeName);
			CollectionSpreadSheet->RefreshWidget();
		}
	});
}


void FDataflowCollectionSpreadSheet::SetCollectionSpreadSheet(TSharedPtr<SCollectionSpreadSheetWidget>& InCollectionSpreadSheet)
{
	ensure(!CollectionSpreadSheet);

	CollectionSpreadSheet = InCollectionSpreadSheet;

	if (CollectionSpreadSheet)
	{
		OnPinnedDownChangedDelegateHandle = CollectionSpreadSheet->GetOnPinnedDownChangedDelegate().AddRaw(this, &FDataflowCollectionSpreadSheet::OnPinnedDownChanged);
		OnRefreshLockedChangedDelegateHandle = CollectionSpreadSheet->GetOnRefreshLockedChangedDelegate().AddRaw(this, &FDataflowCollectionSpreadSheet::OnRefreshLockedChanged);
	}
}


FDataflowCollectionSpreadSheet::~FDataflowCollectionSpreadSheet()
{
	if (CollectionSpreadSheet)
	{
		CollectionSpreadSheet->GetOnPinnedDownChangedDelegate().Remove(OnPinnedDownChangedDelegateHandle);
		CollectionSpreadSheet->GetOnRefreshLockedChangedDelegate().Remove(OnRefreshLockedChangedDelegateHandle);
	}
}
