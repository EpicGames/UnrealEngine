// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Dataflow/DataflowView.h"

class UDataflowEditor;
class UPrimitiveComponent;
class SCollectionSpreadSheetWidget;

/**
*
* Class to handle the SelectionView widget
*
*/
class FDataflowCollectionSpreadSheet : public FDataflowNodeView
{
public:

	FDataflowCollectionSpreadSheet(TObjectPtr<UDataflowBaseContent> InContent = nullptr);
	~FDataflowCollectionSpreadSheet();

	virtual void SetSupportedOutputTypes() override;
	virtual void UpdateViewData() override;
	virtual void ConstructionViewSelectionChanged(const TArray<UPrimitiveComponent*>& SelectedComponents, const TArray<FDataflowBaseElement*>& SelectedElements) override {};
	virtual void SimulationViewSelectionChanged(const TArray<UPrimitiveComponent*>& SelectedComponents, const TArray<FDataflowBaseElement*>& SelectedElements) override {};
	
	void SetCollectionSpreadSheet(TSharedPtr<SCollectionSpreadSheetWidget>& InCollectionSpreadSheet);

private:
	TSharedPtr<SCollectionSpreadSheetWidget> CollectionSpreadSheet;

	FDelegateHandle OnPinnedDownChangedDelegateHandle;
	FDelegateHandle OnRefreshLockedChangedDelegateHandle;
};

