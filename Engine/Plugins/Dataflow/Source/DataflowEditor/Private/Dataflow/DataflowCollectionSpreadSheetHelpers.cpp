// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowCollectionSpreadSheetHelpers.h"
#include "GeometryCollection/GeometryCollection.h"
#include "Dataflow/DataflowSettings.h"
#include "Components/HorizontalBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Styling/SlateTypes.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Layout/Margin.h"

namespace UE::Dataflow::CollectionSpreadSheetHelpers
{
	template<typename T>
	FString AttributeValueToString(const T& Value)
	{
		return Value.ToString();
	}

	template<typename T>
	FString AttributeValueToString(const TArray<T>& Array)
	{
		FString Out;

		for (int32 Idx = 0; Idx < Array.Num(); ++Idx)
		{
			Out += AttributeValueToString(Array[Idx]);

			if (Idx != Array.Num() - 1)
			{
				Out += "; ";
			}
		}

		return Out;
	}

	template<typename T>
	FString AttributeValueToString(const FManagedArrayCollection& InCollection, const FName& InAttributeName, const FName& InGroupName, int32 InIdxColumn)
	{
		const TManagedArray<T>* const Array = InCollection.FindAttributeTyped<T>(InAttributeName, InGroupName);
		if (Array == nullptr)
		{
			return FString("<Unknown Attribute>");
		}

		if (InIdxColumn < 0 || Array->Num() <= InIdxColumn)
		{
			return FString("<Index out of bounds>");
		}

		return AttributeValueToString((*Array)[InIdxColumn]);
	}

	FString AttributeValueToString(float Value)
	{
		return FString::SanitizeFloat(Value, 2);
	}

	FString AttributeValueToString(double Value)
	{
		return FString::SanitizeFloat(Value, 2);
	}

	FString AttributeValueToString(uint8 Value)
	{
		return FString::FromInt((int32)Value);
	}

	FString AttributeValueToString(int32 Value)
	{
		return FString::FromInt(Value);
	}

	FString AttributeValueToString(const FString& Value)
	{
		return Value;
	}

	FString AttributeValueToString(const FName& Value)
	{
		return Value.ToString();
	}

	FString AttributeValueToString(const FLinearColor& Value)
	{
		return FString::Printf(TEXT("(R=%.2f G=%.2f B=%.2f A=%.2f)"), Value.R, Value.G, Value.B, Value.A);
	}

	FString AttributeValueToString(const FVector& Value)
	{
		return FString::Printf(TEXT("(X=%2.2f Y=%2.2f Z=%2.2f)"), Value.X, Value.Y, Value.Z);
	}

	FString AttributeValueToString(bool Value)
	{
		return Value ? FString("true") : FString("false");
	}

	FString AttributeValueToString(const FConstBitReference& Value)
	{
		return Value ? FString("true") : FString("false");
	}

	FString AttributeValueToString(const TSet<int32>& Value)
	{
		TArray<int32> Array = Value.Array();
		FString Out;

		for (int32 Idx = 0; Idx < Array.Num(); ++Idx)
		{
			Out += AttributeValueToString(Array[Idx]);

			if (Idx != Array.Num() - 1)
			{
				Out += " ";
			}
		}

		return Out;
	}

	FString AttributeValueToString(const FTransform3f& Value)
	{
		const FVector3f Translation = Value.GetTranslation();
		const FVector3f Rotation = Value.GetRotation().Euler();
		const FVector3f Scale = Value.GetScale3D();

		return FString::Printf(TEXT("T:(%s) R:(%s) S:(%s)"), *Translation.ToString(), *Rotation.ToString(), *Scale.ToString());
	}

	FString AttributeValueToString(const FTransform& Value)
	{
		const FVector Translation = Value.GetTranslation();
		const FVector Rotation = Value.GetRotation().Euler();
		const FVector Scale = Value.GetScale3D();

		return FString::Printf(TEXT("T:(%s) R:(%s) S:(%s)"), *Translation.ToString(), *Rotation.ToString(), *Scale.ToString());
	}

	FString AttributeValueToString(const FBox& Value)
	{
		FVector Center, Extents;
		Value.GetCenterAndExtents(Center, Extents);
		return FString::Printf(TEXT("Center:(%s) Extents:(%s)"), *Center.ToString(), *Extents.ToString());
	}

	FString AttributeValueToString(const FVector4f& Value)
	{
		return Value.ToString();
	}

	FString AttributeValueToString(const FVector3f& Value)
	{
		return Value.ToString();
	}

	FString AttributeValueToString(const FVector2f& Value)
	{
		return Value.ToString();
	}

	FString AttributeValueToString(const FIntVector& Value)
	{
		return FString::Printf(TEXT("%d %d %d"), Value.X, Value.Y, Value.Z);
	}

	FString AttributeValueToString(const FIntVector2& Value)
	{
		return FString::Printf(TEXT("%d %d"), Value.X, Value.Y);
	}

	FString AttributeValueToString(const FIntVector4& Value)
	{
		return FString::Printf(TEXT("%d %d %d %d"), Value[0], Value[1], Value[2], Value[3]);
	}

	FString AttributeValueToString(const FGuid& Value)
	{
		return Value.ToString();
	}

	FString AttributeValueToString(const FSoftObjectPath& Value)
	{
		return Value.ToString();
	}

	FString AttributeValueToString(const TObjectPtr<UObject>& Value)
	{
		static const FString NullObjectStr(TEXT("<Null Object>"));
		return Value ? Value.GetName() : NullObjectStr;
	}

	FString AttributeValueToString(const Chaos::FConvexPtr& Value)
	{
		if (Value)
		{
			const int32 NumVertices= Value->NumVertices();
			const int32 NumPlanes = Value->NumPlanes();
			return FString::Printf(TEXT("Vertices:(%d), Planes:(%d)"), NumVertices, NumPlanes);
		}
		return FString(TEXT("(null)"));
	}

	FString AttributeValueToString(const FManagedArrayCollection& InCollection, const FName& InAttributeName, const FName& InGroupName, int32 InIdxColumn)
	{
		const FManagedArrayCollection::EArrayType ArrayType = InCollection.GetAttributeType(InAttributeName, InGroupName);

		FString ValueAsString;

		switch (ArrayType)
		{
		case FManagedArrayCollection::EArrayType::FFloatType:
			ValueAsString = AttributeValueToString<float>(InCollection, InAttributeName, InGroupName, InIdxColumn);
			break;

		case FManagedArrayCollection::EArrayType::FDoubleType:
			ValueAsString = AttributeValueToString<double>(InCollection, InAttributeName, InGroupName, InIdxColumn);
			break;

		case FManagedArrayCollection::EArrayType::FUInt8Type:
			ValueAsString = AttributeValueToString<uint8>(InCollection, InAttributeName, InGroupName, InIdxColumn);
			break;

		case FManagedArrayCollection::EArrayType::FInt32Type:
			ValueAsString = AttributeValueToString<int32>(InCollection, InAttributeName, InGroupName, InIdxColumn);
			break;

		case FManagedArrayCollection::EArrayType::FBoolType:
			ValueAsString = AttributeValueToString<bool>(InCollection, InAttributeName, InGroupName, InIdxColumn);
			break;

		case FManagedArrayCollection::EArrayType::FStringType:
			ValueAsString = AttributeValueToString<FString>(InCollection, InAttributeName, InGroupName, InIdxColumn);
			break;

		case FManagedArrayCollection::EArrayType::FNameType:
			ValueAsString = AttributeValueToString<FName>(InCollection, InAttributeName, InGroupName, InIdxColumn);
			break;

		case FManagedArrayCollection::EArrayType::FSoftObjectPathType:
			ValueAsString = AttributeValueToString<FSoftObjectPath>(InCollection, InAttributeName, InGroupName, InIdxColumn);
			break;

		case FManagedArrayCollection::EArrayType::FLinearColorType:
			ValueAsString = AttributeValueToString<FLinearColor>(InCollection, InAttributeName, InGroupName, InIdxColumn);
			break;

		case FManagedArrayCollection::EArrayType::FVectorType:
			ValueAsString = AttributeValueToString<FVector3f>(InCollection, InAttributeName, InGroupName, InIdxColumn);
			break;

		case FManagedArrayCollection::EArrayType::FVector2DType:
			ValueAsString = AttributeValueToString<FVector2f>(InCollection, InAttributeName, InGroupName, InIdxColumn);
			break;

		case FManagedArrayCollection::EArrayType::FVector3dType:
			ValueAsString = AttributeValueToString<FVector2f>(InCollection, InAttributeName, InGroupName, InIdxColumn);
			break;

		case FManagedArrayCollection::EArrayType::FVector4fType:
			ValueAsString = AttributeValueToString<FVector4f>(InCollection, InAttributeName, InGroupName, InIdxColumn);
			break;

		case FManagedArrayCollection::EArrayType::FIntVectorType:
			ValueAsString = AttributeValueToString<FIntVector>(InCollection, InAttributeName, InGroupName, InIdxColumn);
			break;

		case FManagedArrayCollection::EArrayType::FIntVector2Type:
			ValueAsString = AttributeValueToString<FIntVector2>(InCollection, InAttributeName, InGroupName, InIdxColumn);
			break;

		case FManagedArrayCollection::EArrayType::FFVectorArrayType:
			ValueAsString = AttributeValueToString<TArray<FVector3f>>(InCollection, InAttributeName, InGroupName, InIdxColumn);
			break;

		case FManagedArrayCollection::EArrayType::FIntVector3ArrayType:
			ValueAsString = AttributeValueToString<TArray<FIntVector3>>(InCollection, InAttributeName, InGroupName, InIdxColumn);
			break;

		case FManagedArrayCollection::EArrayType::FIntVector2ArrayType:
			ValueAsString = AttributeValueToString<TArray<FIntVector2>>(InCollection, InAttributeName, InGroupName, InIdxColumn);
			break;

		case FManagedArrayCollection::EArrayType::FVector4fArrayType:
			ValueAsString = AttributeValueToString<TArray<FVector4f>>(InCollection, InAttributeName, InGroupName, InIdxColumn);
			break;

		case FManagedArrayCollection::EArrayType::FTransformType:
			ValueAsString = AttributeValueToString<FTransform>(InCollection, InAttributeName, InGroupName, InIdxColumn);
			break;

		case FManagedArrayCollection::EArrayType::FVector2DArrayType:
			ValueAsString = AttributeValueToString<TArray<FVector2f>>(InCollection, InAttributeName, InGroupName, InIdxColumn);
			break;

		case FManagedArrayCollection::EArrayType::FIntArrayType:
			ValueAsString = AttributeValueToString<TSet<int32>>(InCollection, InAttributeName, InGroupName, InIdxColumn);
			break;

		case FManagedArrayCollection::EArrayType::FInt32ArrayType:
			ValueAsString = AttributeValueToString<TArray<int32>>(InCollection, InAttributeName, InGroupName, InIdxColumn);
			break;

		case FManagedArrayCollection::EArrayType::FFloatArrayType:
			ValueAsString = AttributeValueToString<TArray<float>>(InCollection, InAttributeName, InGroupName, InIdxColumn);
			break;

		case FManagedArrayCollection::EArrayType::FBoxType:
			ValueAsString = AttributeValueToString<FBox>(InCollection, InAttributeName, InGroupName, InIdxColumn);
			break;

		case FManagedArrayCollection::EArrayType::FTransform3fType:
			ValueAsString = AttributeValueToString<FTransform3f>(InCollection, InAttributeName, InGroupName, InIdxColumn);
			break;

		case FManagedArrayCollection::EArrayType::FIntVector4Type:
			ValueAsString = AttributeValueToString<FIntVector4>(InCollection, InAttributeName, InGroupName, InIdxColumn);
			break;

		case FManagedArrayCollection::EArrayType::FGuidType:
			ValueAsString = AttributeValueToString<FGuid>(InCollection, InAttributeName, InGroupName, InIdxColumn);
			break;

		case FManagedArrayCollection::EArrayType::FFConvexRefCountedPtrType:
			ValueAsString = AttributeValueToString<Chaos::FConvexPtr>(InCollection, InAttributeName, InGroupName, InIdxColumn);
			break;

		case FManagedArrayCollection::EArrayType::FUObjectArrayType:
			ValueAsString = AttributeValueToString<TObjectPtr<UObject>>(InCollection, InAttributeName, InGroupName, InIdxColumn);
			break;

		default:
			//ensure(false);
			ValueAsString = "<Unknown Data Type>";
		}

		// Clip really long strings so slate doesn't die.
		constexpr int32 MaxStringLength = 10000;
		if (ValueAsString.Len() > MaxStringLength)
		{
			ValueAsString.LeftInline(MaxStringLength);
			ValueAsString += "...";
		}
		return ValueAsString;
	}

	FColor GetColorPerDepth(uint32 Depth)
	{
		const UDataflowSettings* DataflowSettings = GetDefault<UDataflowSettings>();

		const uint32 ColorPerDepthCount = DataflowSettings->TransformLevelColors.LevelColors.Num();
		return DataflowSettings->TransformLevelColors.LevelColors[Depth % ColorPerDepthCount].ToFColor(true);
	}

	FSlateColor UpdateItemColorFromCollection(const TSharedPtr<const FManagedArrayCollection> InCollection, const FName InGroup, const int32 InItemIndex)
	{
		constexpr FLinearColor InvalidColor(0.1f, 0.1f, 0.1f);

		int32 BoneIndex = InItemIndex;
		if (InGroup == FGeometryCollection::VerticesGroup)
		{
			if (InCollection->HasAttribute("BoneMap", FGeometryCollection::VerticesGroup))
			{
				const TManagedArray<int32>& BoneMap = InCollection->GetAttribute<int32>("BoneMap", FGeometryCollection::VerticesGroup);
				BoneIndex = BoneMap[InItemIndex];
			}
			else
			{
				return InvalidColor;
			}
		}
		else if (InGroup == FGeometryCollection::FacesGroup)
		{
			if (InCollection->HasAttribute("Indices", FGeometryCollection::FacesGroup) &&
				InCollection->HasAttribute("BoneMap", FGeometryCollection::VerticesGroup))
			{
				const TManagedArray<FIntVector>& Indices = InCollection->GetAttribute<FIntVector>("Indices", FGeometryCollection::FacesGroup);
				const int32 VertexIndex = Indices[InItemIndex][0];

				const TManagedArray<int32>& BoneMap = InCollection->GetAttribute<int32>("BoneMap", FGeometryCollection::VerticesGroup);
				BoneIndex = BoneMap[VertexIndex];
			}
			else
			{
				return InvalidColor;
			}
		}

		if (InCollection->HasAttribute("Level", FGeometryCollection::TransformGroup))
		{
			const TManagedArray<int32>& Level = InCollection->GetAttribute<int32>("Level", FGeometryCollection::TransformGroup);
			return GetColorPerDepth((uint32)Level[BoneIndex]);
		}
		else if (InCollection->HasAttribute("SimulationType", FGeometryCollection::TransformGroup))
		{
			const TManagedArray<int32>& SimulationType = InCollection->GetAttribute<int32>("SimulationType", FGeometryCollection::TransformGroup);
			switch (SimulationType[BoneIndex])
			{
			case FGeometryCollection::ESimulationTypes::FST_None:
				return FLinearColor::Green;

			case FGeometryCollection::ESimulationTypes::FST_Rigid:
			{
				bool IsVisible = true;
				if (InCollection->HasAttribute("Visible", FGeometryCollection::TransformGroup))
				{
					IsVisible = InCollection->GetAttribute<bool>("Visible", FGeometryCollection::TransformGroup)[BoneIndex];
				}

				return IsVisible ? FSlateColor::UseForeground() : InvalidColor;
			}
			case FGeometryCollection::ESimulationTypes::FST_Clustered:
				return FColor::Cyan;

			default:
				ensureMsgf(false, TEXT("Invalid Geometry Collection simulation type encountered."));
				return InvalidColor;
			}
		}

		return InvalidColor;
	}

	TSharedRef<SWidget> MakeColumnWidget(const TSharedPtr<const FManagedArrayCollection> InCollection,
		const FName InGroup,
		const FName InAttr,
		const int32 InItemIndex,
		const FSlateColor InItemColor)
	{
		FName AttrType = UE::Dataflow::CollectionSpreadSheetHelpers::GetArrayTypeString(InCollection->GetAttributeType(InAttr, InGroup));

		if (InAttr == "Index")
		{
			return SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(STextBlock)
						.Text(FText::AsNumber(InItemIndex))
						.ColorAndOpacity(InItemColor)
				];
		}
		else if (InAttr == "SimulationType")
		{
			const TArray<FString> SimTypes = { "None", "Rigid", "Clustered" };	
			FString AttrValueStr = SimTypes[InCollection->GetAttribute<int32>("SimulationType", InGroup)[InItemIndex]];

			return SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(STextBlock)
					.Text(FText::FromString(AttrValueStr))
					.ColorAndOpacity(InItemColor)
					.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
				];
		}

		if (AttrType == "LinearColor")
		{
			FLinearColor AttrValue = InCollection->GetAttribute<FLinearColor>(InAttr, InGroup)[InItemIndex];

			return SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SColorBlock)
						.Color(AttrValue)
						.Size(FVector2D(48, 16))
						.CornerRadius(2.f)
				];
		}
		else if (AttrType == "Vector2D")
		{
			FVector2f AttrValue = InCollection->GetAttribute<FVector2f>(InAttr, InGroup)[InItemIndex];

			return SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(FMargin(1.f, 0.f))
				.MinWidth(60)
				.MaxWidth(60)
				[
					SNew(STextBlock)
					.Text(FText::AsNumber(AttrValue.X))
					.ColorAndOpacity(InItemColor)
					.Justification(ETextJustify::Right)
				]

				+ SHorizontalBox::Slot()
				.Padding(FMargin(1.f, 0.f))
				.MinWidth(60)
				.MaxWidth(60)
				[
					SNew(STextBlock)
					.Text(FText::AsNumber(AttrValue.Y))
					.ColorAndOpacity(InItemColor)
					.Justification(ETextJustify::Right)
				];
		}
		else if (AttrType == "Vector")
		{
			FVector3f AttrValue = InCollection->GetAttribute<FVector3f>(InAttr, InGroup)[InItemIndex];

			return SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(FMargin(1.f, 0.f))
				.MinWidth(60)
				.MaxWidth(60)
				[
					SNew(STextBlock)
						.Text(FText::AsNumber(AttrValue.X))
						.ColorAndOpacity(InItemColor)
						.Justification(ETextJustify::Right)
				]

				+ SHorizontalBox::Slot()
				.Padding(FMargin(1.f, 0.f))
				.MinWidth(60)
				.MaxWidth(60)
				[
					SNew(STextBlock)
						.Text(FText::AsNumber(AttrValue.Y))
						.ColorAndOpacity(InItemColor)
						.Justification(ETextJustify::Right)
				]

				+ SHorizontalBox::Slot()
				.Padding(FMargin(1.f, 0.f))
				.MinWidth(60)
				.MaxWidth(60)
				[
					SNew(STextBlock)
						.Text(FText::AsNumber(AttrValue.Z))
						.ColorAndOpacity(InItemColor)
						.Justification(ETextJustify::Right)
				];
		}
		else if (AttrType == "IntVector")
		{
			FIntVector AttrValue = InCollection->GetAttribute<FIntVector>(InAttr, InGroup)[InItemIndex];

			return SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(FMargin(1.f, 0.f))
				.MinWidth(60)
				.MaxWidth(60)
				[
					SNew(STextBlock)
					.Text(FText::AsNumber(AttrValue.X))
					.ColorAndOpacity(InItemColor)
					.Justification(ETextJustify::Right)
				]

				+ SHorizontalBox::Slot()
				.Padding(FMargin(1.f, 0.f))
				.MinWidth(60)
				.MaxWidth(60)
				[
					SNew(STextBlock)
					.Text(FText::AsNumber(AttrValue.Y))
					.ColorAndOpacity(InItemColor)
					.Justification(ETextJustify::Right)
				]

				+ SHorizontalBox::Slot()
				.Padding(FMargin(1.f, 0.f))
				.MinWidth(60)
				.MaxWidth(60)
				[
					SNew(STextBlock)
					.Text(FText::AsNumber(AttrValue.Z))
					.ColorAndOpacity(InItemColor)
					.Justification(ETextJustify::Right)
				];
		}
		else if (AttrType == "Transform3f")
		{
			constexpr float TextFieldWidth = 50.f;
			constexpr int32 FractionalDigits = 2;

			struct FTextDisplayInfo
			{
				FText Text;
				ETextJustify::Type Justify;
				float Width = 0.f;
			};

			FTransform3f AttrValue = InCollection->GetAttribute<FTransform3f>(InAttr, InGroup)[InItemIndex];

			const FVector3f Translation = AttrValue.GetTranslation();
			const FVector3f Rotation = AttrValue.GetRotation().Euler();
			const FVector3f Scale = AttrValue.GetScale3D();

			FNumberFormattingOptions FormattingOptions;
			FormattingOptions.SetMinimumFractionalDigits(FractionalDigits);
			FormattingOptions.SetMaximumFractionalDigits(FractionalDigits);

			const FTextDisplayInfo TextDisplayArr[] =
			{
				{ FText::FromString(TEXT("T:[")), ETextJustify::Right, 20 },
				{ FText::AsNumber(Translation.X, &FormattingOptions), ETextJustify::Right, TextFieldWidth },
				{ FText::AsNumber(Translation.Y, &FormattingOptions), ETextJustify::Right, TextFieldWidth },
				{ FText::AsNumber(Translation.Z, &FormattingOptions), ETextJustify::Right, TextFieldWidth },
				{ FText::FromString(TEXT("]")), ETextJustify::Left, 15 },
				{ FText::FromString(TEXT("R:[")), ETextJustify::Right, 20 },
				{ FText::AsNumber(Rotation.X, &FormattingOptions), ETextJustify::Right, TextFieldWidth },
				{ FText::AsNumber(Rotation.Y, &FormattingOptions), ETextJustify::Right, TextFieldWidth },
				{ FText::AsNumber(Rotation.Z, &FormattingOptions), ETextJustify::Right, TextFieldWidth },
				{ FText::FromString(TEXT("]")), ETextJustify::Left, 15 },
				{ FText::FromString(TEXT("S:[")), ETextJustify::Right, 20 },
				{ FText::AsNumber(Scale.X, &FormattingOptions), ETextJustify::Right, TextFieldWidth },
				{ FText::AsNumber(Scale.Y, &FormattingOptions), ETextJustify::Right, TextFieldWidth },
				{ FText::AsNumber(Scale.Z, &FormattingOptions), ETextJustify::Right, TextFieldWidth },
				{ FText::FromString(TEXT("]")), ETextJustify::Left, 5 }
			};

			constexpr int32 TextArrSize = sizeof(TextDisplayArr) / sizeof(FTextDisplayInfo);

			TSharedRef<SHorizontalBox> Box = SNew(SHorizontalBox);

			for (int32 Idx = 0; Idx < TextArrSize; ++Idx)
			{
				Box->AddSlot()
					.Padding(FMargin(1.f, 0.f))
					.MinWidth(TextDisplayArr[Idx].Width)
					.MaxWidth(TextDisplayArr[Idx].Width)
					[
						SNew(STextBlock)
						.Text(TextDisplayArr[Idx].Text)
						.ColorAndOpacity(InItemColor)
						.Justification(TextDisplayArr[Idx].Justify)
					];
			}

			return Box;
		}
		else
		{
			FString AttrValueStr = UE::Dataflow::CollectionSpreadSheetHelpers::AttributeValueToString(*InCollection.Get(), InAttr, InGroup, InItemIndex);

			return SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(STextBlock)
						.Text(FText::FromString(AttrValueStr))
						.ColorAndOpacity(InItemColor)
						.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
				];
		}
	}

}