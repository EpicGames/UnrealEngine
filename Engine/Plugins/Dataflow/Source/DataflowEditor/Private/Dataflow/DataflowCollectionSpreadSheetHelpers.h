// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GeometryCollection/ManagedArrayCollection.h"
#include "Styling/SlateColor.h"

class SWidget;

struct FSoftObjectPath;

namespace UE::Dataflow::CollectionSpreadSheetHelpers
{
	struct FAttrInfo
	{
		FName Name;
		FString Type;
	};

	static TMap<FString, int32> AttrTypeWidthMap =
	{
		{ TEXT("Transform"),	600 },
		{ TEXT("Transform3f"),	600 },
		{ TEXT("String"),		200 },
		{ TEXT("Name"),			200 },
		{ TEXT("LinearColor"),	80 },
		{ TEXT("int32"),		100 },
		{ TEXT("IntArray"),		200 },
		{ TEXT("Vector"),		250 },
		{ TEXT("Vector2D"),		160 },
		{ TEXT("Float"),		150 },
		{ TEXT("Double"),		150 },
		{ TEXT("IntVector"),	220 },
		{ TEXT("Bool"),			75 },
		{ TEXT("Box"),			550 },
		{ TEXT("MeshSection"),	100 },
		{ TEXT("UInt8"),		100 },
		{ TEXT("Guid"),			350 },
	};

	FString AttributeValueToString(float Value);

	FString AttributeValueToString(double Value);

	FString AttributeValueToString(uint8 Value);

	FString AttributeValueToString(int32 Value);

	FString AttributeValueToString(const FString& Value);

	FString AttributeValueToString(const FName& Value);

	FString AttributeValueToString(const FLinearColor& Value);

	FString AttributeValueToString(const FVector& Value);

	FString AttributeValueToString(bool Value);

	FString AttributeValueToString(const FConstBitReference& Value);

	FString AttributeValueToString(const TSet<int32>& Value);

	FString AttributeValueToString(const FTransform3f& Value);

	FString AttributeValueToString(const FTransform& Value);

	FString AttributeValueToString(const FBox& Value);

	FString AttributeValueToString(const FVector4f& Value);

	FString AttributeValueToString(const FVector3f& Value);
	
	FString AttributeValueToString(const FVector2f& Value);

	FString AttributeValueToString(const FIntVector& Value);

	FString AttributeValueToString(const FIntVector2& Value);

	FString AttributeValueToString(const FIntVector4& Value);

	FString AttributeValueToString(const FGuid& Value);

	FString AttributeValueToString(const FSoftObjectPath& Value);

	FString AttributeValueToString(const TObjectPtr<UObject>& Value);

	FString AttributeValueToString(const Chaos::FConvexPtr& Value);

	FString AttributeValueToString(const FManagedArrayCollection& InCollection, const FName& InAttributeName, const FName& InGroupName, int32 InIdxColumn);

	inline FName GetArrayTypeString(FManagedArrayCollection::EArrayType ArrayType)
	{
		switch (ArrayType)
		{
#define MANAGED_ARRAY_TYPE(a,A)	case EManagedArrayType::F##A##Type:\
		return FName(#A);
#include "GeometryCollection/ManagedArrayTypeValues.inl"
#undef MANAGED_ARRAY_TYPE
		}
		return FName();
	}

	FColor GetColorPerDepth(uint32 Depth);

	FSlateColor UpdateItemColorFromCollection(const TSharedPtr<const FManagedArrayCollection> InCollection, const FName InGroup, const int32 InItemIndex);

	TSharedRef<SWidget> MakeColumnWidget(const TSharedPtr<const FManagedArrayCollection> InCollection,
		const FName InGroup,
		const FName InAttr,
		const int32 InItemIndex,
		const FSlateColor InItemColor);
}


