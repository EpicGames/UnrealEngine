// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowColorRampCustomization.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "Dataflow/DataflowColorRamp.h"
#include "SColorGradientEditor.h"
#include "Dataflow/DataflowGraphEditor.h"
#include "DetailWidgetRow.h"


namespace UE::Dataflow
{
	TSharedRef<IPropertyTypeCustomization> FDataflowColorRampCustomization::MakeInstance()
	{
		return MakeShareable(new FDataflowColorRampCustomization);
	}

	void FDataflowColorRampCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (const FStructProperty* const StructProperty = CastField<FStructProperty>(PropertyHandle->GetProperty()))
		{
			if (StructProperty->Struct == FDataflowColorRamp::StaticStruct())
			{
				void* Data = nullptr;
				if (PropertyHandle->GetValueData(Data) == FPropertyAccess::Success)
				{
					FDataflowColorRamp* ColorRamp = reinterpret_cast<FDataflowColorRamp*>(Data);

					TSharedPtr<SColorGradientEditor> GradientEditor;

					HeaderRow
						[
							SAssignNew(GradientEditor, SColorGradientEditor)
								.ViewMinInput(0.0f)
								.ViewMaxInput(1.0f)
								.ClampStopsToViewRange(true)
						];

					GradientEditor->SetCurveOwner(&ColorRamp->ColorRamp);
				}
			}
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	TSharedRef<IPropertyTypeCustomization> FDataflowLinearColorRampCustomization::MakeInstance()
	{
		return MakeShareable(new FDataflowLinearColorRampCustomization);
	}

	void FDataflowLinearColorRampCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (const FStructProperty* const StructProperty = CastField<FStructProperty>(PropertyHandle->GetProperty()))
		{
			if (StructProperty->Struct)
			{
				void* Data = nullptr;

				if (PropertyHandle->GetValueData(Data) == FPropertyAccess::Success)
				{
					FDataflowLinearColorRamp* ColorRamp = reinterpret_cast<FDataflowLinearColorRamp*>(Data);

					TSharedPtr<SColorGradientEditor> GradientEditor;

					HeaderRow
						[
							SAssignNew(GradientEditor, SColorGradientEditor)
								.ViewMinInput(0.0f)
								.ViewMaxInput(1.0f)
								.ClampStopsToViewRange(true)
						];

					GradientEditor->SetCurveOwner(ColorRamp);
				}
			}
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
}  





