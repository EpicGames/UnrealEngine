// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowComponentToolTarget.h"

#include "ConversionUtils/DynamicMeshViaMeshDescriptionUtil.h"
#include "Dataflow/DataflowComponent.h"
#include "DynamicMeshToMeshDescription.h"
#include "GeometryCollection/Facades/CollectionRenderingFacade.h"
#include "MeshDescriptionToDynamicMesh.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "Materials/MaterialInterface.h"
#include "StaticMeshAttributes.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowComponentToolTarget)

using namespace UE::Geometry;

namespace UE
{
	namespace Conversion
	{
		// Convert a dataflow component to a dynamic mesh
		void DataflowComponentToDynamicMesh(const UDataflowComponent* DataflowComponent, FDynamicMesh3& DynamicMesh)
		{
			const FManagedArrayCollection& RenderingCollection = DataflowComponent->GetRenderingCollection();
			const GeometryCollection::Facades::FRenderingFacade Facade(RenderingCollection);

			if(Facade.CanRenderSurface())
			{
				const int32 NumTriangles = Facade.NumTriangles();
				const int32 NumVertices = Facade.NumVertices();
				
				const TManagedArray<FIntVector>& Indices = Facade.GetIndices();
				const TManagedArray<FVector3f>& Positions = Facade.GetVertices();
				const TManagedArray<FVector3f>& Normals = Facade.GetNormals();
				const TManagedArray<FLinearColor>& Colors = Facade.GetVertexColor();

				DynamicMesh.Clear();
				for(int32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
				{
					DynamicMesh.AppendVertex(FVertexInfo(FVector3d(Positions[VertexIndex]), Normals[VertexIndex],
						FVector3f(Colors[VertexIndex].R, Colors[VertexIndex].G, Colors[VertexIndex].B)));
				}
				for(int32 TriangleIndex = 0; TriangleIndex < NumTriangles; ++TriangleIndex)
				{
					DynamicMesh.AppendTriangle(FIndex3i(Indices[TriangleIndex].X, Indices[TriangleIndex].Y, Indices[TriangleIndex].Z));
				}
			}
		}
		
		// Convert a dynamic mesh to a dataflow component
		void DynamicMeshToDataflowComponent(const FDynamicMesh3& DynamicMesh, UDataflowComponent* DataflowComponent)
		{
			FManagedArrayCollection& RenderingCollection = DataflowComponent->ModifyRenderingCollection();
			GeometryCollection::Facades::FRenderingFacade Facade(RenderingCollection);

			if(Facade.CanRenderSurface())
			{
				const int32 NumTriangles = Facade.NumTriangles();
				const int32 NumVertices = Facade.NumVertices();

				// We can only override vertices attributes (position, normals, colors)
				if((NumTriangles == DynamicMesh.TriangleCount()) && (NumVertices == DynamicMesh.VertexCount()))
				{
					TManagedArray<FVector3f>& Positions = Facade.ModifyVertices();
                    TManagedArray<FVector3f>& Normals = Facade.ModifyNormals();
                    TManagedArray<FLinearColor>& Colors = Facade.ModifyVertexColor();

					for(int32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
					{
						Positions[VertexIndex] = FVector3f(DynamicMesh.GetVertex(VertexIndex));
						Normals[VertexIndex] = DynamicMesh.GetVertexNormal(VertexIndex);
						Colors[VertexIndex] = DynamicMesh.GetVertexColor(VertexIndex);
					}
				}
			}
		}
	}
}

//
// UDataflowComponentReadOnlyToolTarget
//

bool UDataflowComponentReadOnlyToolTarget::IsValid() const
{
	if (!UPrimitiveComponentToolTarget::IsValid())
	{
		return false;
	}
	UDataflowComponent* DataflowComponent = Cast<UDataflowComponent>(Component);
	if (DataflowComponent == nullptr)
	{
		return false;
	}
	const UDataflow* Dataflow = DataflowComponent->GetDataflow();

	return Dataflow != nullptr;
}

int32 UDataflowComponentReadOnlyToolTarget::GetNumMaterials() const
{
	return ensure(IsValid()) ? Component->GetNumMaterials() : 0;
}

UMaterialInterface* UDataflowComponentReadOnlyToolTarget::GetMaterial(int32 MaterialIndex) const
{
	return ensure(IsValid()) ? Component->GetMaterial(MaterialIndex) : nullptr;
}

void UDataflowComponentReadOnlyToolTarget::GetMaterialSet(FComponentMaterialSet& MaterialSetOut, bool bPreferAssetMaterials) const
{
	if (!ensure(IsValid())) return;

	const int32 NumMaterials = Component->GetNumMaterials();
	MaterialSetOut.Materials.SetNum(NumMaterials);
	for (int32 k = 0; k < NumMaterials; ++k)
	{
		MaterialSetOut.Materials[k] = Component->GetMaterial(k);
	}
}

bool UDataflowComponentReadOnlyToolTarget::CommitMaterialSetUpdate(const FComponentMaterialSet& MaterialSet, bool bApplyToAsset)
{
	// filter out any Engine materials that we don't want to be permanently assigning
	TArray<UMaterialInterface*> FilteredMaterials = MaterialSet.Materials;
	for (int32 k = 0; k < FilteredMaterials.Num(); ++k)
	{
		FString AssetPath = FilteredMaterials[k]->GetPathName();
		if (AssetPath.StartsWith(TEXT("/MeshModelingToolsetExp/")))
		{
			FilteredMaterials[k] = UMaterial::GetDefaultMaterial(MD_Surface);
		}
	}

	UDataflowComponent* DynamicMeshComponent = Cast<UDataflowComponent>(Component);
	const int32 NumMaterialsGiven = FilteredMaterials.Num();

	DynamicMeshComponent->Modify();
	for (int32 k = 0; k < NumMaterialsGiven; ++k)
	{
		DynamicMeshComponent->SetMaterial(k, FilteredMaterials[k]);
	}

	return true;
}

const FMeshDescription* UDataflowComponentReadOnlyToolTarget::GetMeshDescription(const FGetMeshParameters& GetMeshParams)
{
	if (ensure(IsValid()) && !DataflowMeshDescription.IsValid())
	{
		DataflowMeshDescription = MakeUnique<FMeshDescription>();
		FStaticMeshAttributes Attributes(*DataflowMeshDescription);
		Attributes.Register();

		const FDynamicMesh3 DynamicMesh = GetDynamicMesh();
		
		FDynamicMeshToMeshDescription Converter;
		Converter.Convert(&DynamicMesh, *DataflowMeshDescription, true);
	}
	return DataflowMeshDescription.Get();
}

FMeshDescription UDataflowComponentReadOnlyToolTarget::GetEmptyMeshDescription()
{
	FMeshDescription EmptyMeshDescription;
	FStaticMeshAttributes MeshAttributes(EmptyMeshDescription);
	MeshAttributes.Register();

	return EmptyMeshDescription;
}

FDynamicMesh3 UDataflowComponentReadOnlyToolTarget::GetDynamicMesh()
{
	FDynamicMesh3 DynamicMesh;
	if( const UDataflowComponent* DataflowComponent = Cast<UDataflowComponent>(Component))
	{
		UE::Conversion::DataflowComponentToDynamicMesh(DataflowComponent, DynamicMesh);
	}
	return DynamicMesh;
}

//
// UDataflowComponentToolTarget
//


void UDataflowComponentToolTarget::CommitMeshDescription(const FCommitter& Committer, const FCommitMeshParameters& CommitMeshParams)
{
	if (ensure(IsValid()) == false) return;

	// Let the user fill our mesh description with the Committer
	if (!DataflowMeshDescription.IsValid())
	{
		DataflowMeshDescription = MakeUnique<FMeshDescription>();
		FStaticMeshAttributes Attributes(*DataflowMeshDescription);
		Attributes.Register();
	}
	FCommitterParams CommitParams;
	CommitParams.MeshDescriptionOut = DataflowMeshDescription.Get();
	Committer(CommitParams);

	// The conversion we have right now is from dynamic mesh to volume, so we convert
	// to dynamic mesh first.
	FDynamicMesh3 DynamicMesh;
	FMeshDescriptionToDynamicMesh Converter;
	Converter.Convert(DataflowMeshDescription.Get(), DynamicMesh);

	CommitDynamicMesh(DynamicMesh);
}

void UDataflowComponentToolTarget::CommitDynamicMesh(const FDynamicMesh3& DynamicMesh, const FDynamicMeshCommitInfo& CommitInfo)
{
	if(UDataflowComponent* DataflowComponent = Cast<UDataflowComponent>(Component))
	{
		UE::Conversion::DynamicMeshToDataflowComponent(DynamicMesh, DataflowComponent);
	}
}

// Factory

bool UDataflowComponentReadOnlyToolTargetFactory::CanBuildTarget(UObject* SourceObject, const FToolTargetTypeRequirements& Requirements) const
{
	// We are using an exact cast here to prevent subclasses, which might not meet all
	// requirements for functionality such as the deprecated DestructibleMeshComponent, from 
	// being caught up as valid targets.
	// If you want to make the tool target work with some subclass of UDataflowComponent,
	// just add another factory that allows that class specifically(but make sure that
	// GetMeshDescription and such work properly)

	return Cast<UDataflowComponent>(SourceObject) && Cast<UDataflowComponent>(SourceObject)->GetDataflow() &&
		ExactCast<UDataflow>(Cast<UDataflowComponent>(SourceObject)->GetDataflow()) &&
		!ExactCast<UDataflow>(Cast<UDataflowComponent>(SourceObject)->GetDataflow())->GetOutermost()->bIsCookedForEditor &&
		Requirements.AreSatisfiedBy(UDataflowComponentReadOnlyToolTarget::StaticClass());
}

UToolTarget* UDataflowComponentReadOnlyToolTargetFactory::BuildTarget(UObject* SourceObject, const FToolTargetTypeRequirements& Requirements)
{
	UDataflowComponentReadOnlyToolTarget* Target = NewObject<UDataflowComponentReadOnlyToolTarget>();
	Target->InitializeComponent(Cast<UDataflowComponent>(SourceObject));
	checkSlow(Target->Component.IsValid() && Requirements.AreSatisfiedBy(Target));

	return Target;
}

bool UDataflowComponentToolTargetFactory::CanBuildTarget(UObject* SourceObject, const FToolTargetTypeRequirements& Requirements) const
{
	// We are using an exact cast here to prevent subclasses, which might not meet all
	// requirements for functionality such as the deprecated DestructibleMeshComponent, from 
	// being caught up as valid targets.
	// If you want to make the tool target work with some subclass of UDataflowComponent,
	// just add another factory that allows that class specifically(but make sure that
	// GetMeshDescription and such work properly)

	return Cast<UDataflowComponent>(SourceObject) && Cast<UDataflowComponent>(SourceObject)->GetDataflow() &&
		ExactCast<UDataflow>(Cast<UDataflowComponent>(SourceObject)->GetDataflow()) &&
		!ExactCast<UDataflow>(Cast<UDataflowComponent>(SourceObject)->GetDataflow())->GetOutermost()->bIsCookedForEditor &&
		Requirements.AreSatisfiedBy(UDataflowComponentToolTarget::StaticClass());
}

UToolTarget* UDataflowComponentToolTargetFactory::BuildTarget(UObject* SourceObject, const FToolTargetTypeRequirements& Requirements)
{
	UDataflowComponentToolTarget* Target = NewObject<UDataflowComponentToolTarget>();
	Target->InitializeComponent(Cast<UDataflowComponent>(SourceObject));
	checkSlow(Target->Component.IsValid() && Requirements.AreSatisfiedBy(Target));

	return Target;
}

