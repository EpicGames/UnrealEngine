// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "TargetInterfaces/DynamicMeshCommitter.h"
#include "TargetInterfaces/DynamicMeshProvider.h"
#include "TargetInterfaces/MaterialProvider.h"
#include "TargetInterfaces/MeshDescriptionCommitter.h"
#include "TargetInterfaces/MeshDescriptionProvider.h"
#include "ToolTargets/PrimitiveComponentToolTarget.h"

#include "DataflowComponentToolTarget.generated.h"

#define UE_API DATAFLOWEDITOR_API

/**
 * A tool target backed by a read-only dataflow component that can provide and take a mesh
 * description.
 */
UCLASS(MinimalAPI, Transient)
class UDataflowComponentReadOnlyToolTarget :
	public UPrimitiveComponentToolTarget,
	public IMeshDescriptionProvider,
	public IDynamicMeshProvider, 
	public IMaterialProvider
{
	GENERATED_BODY()

public:

	// UToolTarget implementation
	UE_API virtual bool IsValid() const override;

	// IMeshDescriptionProvider implementation
	UE_API virtual const FMeshDescription* GetMeshDescription(const FGetMeshParameters& GetMeshParams = FGetMeshParameters()) override;
	UE_API virtual FMeshDescription GetEmptyMeshDescription() override;

	// IMaterialProvider implementation
	UE_API virtual int32 GetNumMaterials() const override;
	UE_API virtual UMaterialInterface* GetMaterial(int32 MaterialIndex) const override;
	UE_API virtual void GetMaterialSet(FComponentMaterialSet& MaterialSetOut, bool bPreferAssetMaterials) const override;
	UE_API virtual bool CommitMaterialSetUpdate(const FComponentMaterialSet& MaterialSet, bool bApplyToAsset) override;

	// IDynamicMeshProvider
	UE_API virtual UE::Geometry::FDynamicMesh3 GetDynamicMesh() override;

protected:
	// So that the tool target factory can poke into Component.
	friend class UDataflowComponentReadOnlyToolTargetFactory;

	// Until UDataflow stores its internal representation as FMeshDescription, we need to
	// retain the storage here to cover the lifetime of the pointer returned by GetMeshDescription(). 
	TUniquePtr<FMeshDescription> DataflowMeshDescription;
};

/**
 * A tool target backed by a dataflow component that can provide and take a mesh
 * description.
 */
UCLASS(MinimalAPI, Transient)
class UDataflowComponentToolTarget :
	public UDataflowComponentReadOnlyToolTarget,
	public IMeshDescriptionCommitter,
	public IDynamicMeshCommitter
{
	GENERATED_BODY()

public:
	// IMeshDescriptionCommitter implementation
	UE_API virtual void CommitMeshDescription(const FCommitter& Committer, const FCommitMeshParameters& CommitParams = FCommitMeshParameters()) override;
	using IMeshDescriptionCommitter::CommitMeshDescription; // unhide the other overload

	// IDynamicMeshCommitter
	UE_API virtual void CommitDynamicMesh(const UE::Geometry::FDynamicMesh3& Mesh, const FDynamicMeshCommitInfo& CommitInfo) override;
	using IDynamicMeshCommitter::CommitDynamicMesh; // unhide the other overload

protected:
	// So that the tool target factory can poke into Component.
	friend class UDataflowComponentToolTargetFactory;
};


/** Factory for UDataflowComponentReadOnlyToolTarget to be used by the target manager. */
UCLASS(MinimalAPI, Transient)
class UDataflowComponentReadOnlyToolTargetFactory : public UToolTargetFactory
{
	GENERATED_BODY()

public:

	// UToolTargetFactory implementation
	UE_API virtual bool CanBuildTarget(UObject* SourceObject, const FToolTargetTypeRequirements& TargetTypeInfo) const override;
	UE_API virtual UToolTarget* BuildTarget(UObject* SourceObject, const FToolTargetTypeRequirements& TargetTypeInfo) override;
};


/** Factory for UDataflowComponentToolTarget to be used by the target manager. */
UCLASS(MinimalAPI, Transient)
class UDataflowComponentToolTargetFactory : public UToolTargetFactory
{
	GENERATED_BODY()

public:

	// UToolTargetFactory implementation
	UE_API virtual bool CanBuildTarget(UObject* SourceObject, const FToolTargetTypeRequirements& TargetTypeInfo) const override;
	UE_API virtual UToolTarget* BuildTarget(UObject* SourceObject, const FToolTargetTypeRequirements& TargetTypeInfo) override;
};

#undef UE_API
