// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowConstructionScene.h"

#include "AssetEditorModeManager.h"
#include "Dataflow/CollectionRenderingPatternUtility.h"
#include "Dataflow/DataflowDebugDrawComponent.h"
#include "Dataflow/DataflowEditorCollectionComponent.h"
#include "Dataflow/DataflowEditor.h"
#include "Dataflow/DataflowEditorToolkit.h"
#include "Dataflow/DataflowObject.h"
#include "Dataflow/DataflowEditorStyle.h"
#include "Dataflow/DataflowRenderingFactory.h"
#include "Dataflow/DataflowEditorUtil.h"
#include "DataflowRendering/DataflowDebugMeshRenderableType.h"
#include "DataflowRendering/DataflowRenderableComponents.h"
#include "DataflowRendering/DataflowRenderableTypeInstance.h"
#include "Debug/DebugDrawComponent.h"
#include "Drawing/MeshElementsVisualizer.h"
#include "Elements/Framework/EngineElementsLibrary.h"
#include "Selection.h"
#include "AssetViewerSettings.h"
#include "DataflowEditorOptions.h"
#include "Dataflow/DataflowPrimitiveNode.h"
#include "GeometryCollection/Facades/CollectionRenderingFacade.h"
#include "Selection/GeometrySelector.h"

#define LOCTEXT_NAMESPACE "FDataflowConstructionScene"

//
// Construction Scene
//

FDataflowConstructionScene::FDataflowConstructionScene(FPreviewScene::ConstructionValues ConstructionValues, UDataflowEditor* InEditor)
	: FDataflowPreviewSceneBase(ConstructionValues, InEditor, FName("Construction Components"))
{}

FDataflowConstructionScene::~FDataflowConstructionScene()
{
	ResetWireframeMeshElementsVisualizer();
	ResetSceneComponents();
}

TArray<TObjectPtr<UDynamicMeshComponent>> FDataflowConstructionScene::GetDynamicMeshComponents() const
{
	TArray<TObjectPtr<UDynamicMeshComponent>> OutValues;
	DynamicMeshComponents.GenerateValueArray(OutValues);
	return MoveTemp(OutValues);
}

TArray<TObjectPtr<UDebugDrawComponent>> FDataflowConstructionScene::GetDebugDrawComponents() const
{
	TArray<TObjectPtr<UDebugDrawComponent>> DebugDrawComponents;
	for (TObjectPtr<UPrimitiveComponent> PrimitiveComponent : PrimitiveComponents)
	{
		if (UDebugDrawComponent* DebugDrawComponentToAdd = Cast<UDebugDrawComponent>(PrimitiveComponent))
		{
			DebugDrawComponents.Add(DebugDrawComponentToAdd);
		}
	}
	if (UDebugDrawComponent* DebugDrawComponentToAdd = Cast<UDebugDrawComponent>(GetDebugDrawComponent()))
	{
		DebugDrawComponents.Add(DebugDrawComponentToAdd);
	}
	return DebugDrawComponents;
}

TArray<UDataflowRenderableTypeSettings*> FDataflowConstructionScene::GetRenderableTypeSettings() const
{
	TArray<UDataflowRenderableTypeSettings*> RenderSettingsArray;
	// make sure we do not add debug mesh render settings if there's no instances
	if (RenderableTypeInstances.Num() > 0)
	{
		for (const UE::Dataflow::FRenderableTypeInstance& RenderableInstance : RenderableTypeInstances)
		{
			for (UDataflowRenderableTypeSettings* RenderSettings : RenderableInstance.GetAllRenderSettings())
			{
				RenderSettingsArray.AddUnique(RenderSettings);
			}
		}
		if (DebugMeshRenderSettings)
		{
			if (DynamicMeshComponents.Num() > 0)
			{
				RenderSettingsArray.AddUnique(DebugMeshRenderSettings);
			}
		}
	}
	return RenderSettingsArray;
}

/** Hide all or a single component */
void FDataflowConstructionScene::SetVisibility(bool bVisibility, UActorComponent* InComponent)
{
	auto SetCollectionVisiblity = [](bool bVisibility,TObjectPtr<UDataflowEditorCollectionComponent> Component) {
		Component->SetVisibility(bVisibility);
		if (Component->WireframeComponent)
		{
			Component->WireframeComponent->SetVisibility(bVisibility);
		}
	};

	for (FRenderElement& RenderElement : DynamicMeshComponents)
	{
		if (TObjectPtr<UDataflowEditorCollectionComponent> DynamicMeshComponent = Cast<UDataflowEditorCollectionComponent>(RenderElement.Value))
		{
			if (InComponent != nullptr)
			{
				if (InComponent == DynamicMeshComponent.Get())
				{
					SetCollectionVisiblity(bVisibility,DynamicMeshComponent);
				}
			}
			else
			{
				SetCollectionVisiblity(bVisibility,DynamicMeshComponent);
			}
		}
	}
}


void FDataflowConstructionScene::AddReferencedObjects(FReferenceCollector& Collector)
{
	FDataflowPreviewSceneBase::AddReferencedObjects(Collector);

	Collector.AddReferencedObjects(DynamicMeshComponents);
	Collector.AddReferencedObjects(WireframeElements);
	Collector.AddReferencedObject(DebugMeshRenderSettings);
}

void FDataflowConstructionScene::TickDataflowScene(const float DeltaSeconds)
{
	if (const TObjectPtr<UDataflowBaseContent>& EditorContent = GetEditorContent())
	{
		if (const UDataflow* Dataflow = EditorContent->GetDataflowAsset())
		{
			if (Dataflow->GetDataflow())
			{
				UE::Dataflow::FTimestamp SystemTimestamp = UE::Dataflow::FTimestamp::Invalid;
				bool bMustUpdateConstructionScene = false;
				for (TObjectPtr<const UDataflowBaseContent> DataflowBaseContent : GetTerminalContents())
				{
					const FName DataflowTerminalName(DataflowBaseContent->GetDataflowTerminal());
					if (TSharedPtr<const FDataflowNode> DataflowTerminalNode = Dataflow->GetDataflow()->FindBaseNode(DataflowTerminalName))
					{
						SystemTimestamp = DataflowTerminalNode->GetTimestamp();
					}

					if (LastRenderedTimestamp < SystemTimestamp)
					{
						LastRenderedTimestamp = SystemTimestamp;
						bMustUpdateConstructionScene = true;
					}
				}
				if (bMustUpdateConstructionScene || EditorContent->IsConstructionDirty())
				{
					UpdateConstructionScene();
				}
			}
		}
	}
	for (TObjectPtr<UInteractiveToolPropertySet>& Propset : PropertyObjectsToTick)
	{
		if (Propset)
		{
			if (Propset->IsPropertySetEnabled())
			{
				Propset->CheckAndUpdateWatched();
			}
			else
			{
				Propset->SilentUpdateWatched();
			}
		}
	}

	for (FRenderWireElement Elem : WireframeElements)
	{
		Elem.Value->OnTick(DeltaSeconds);
	}
}

void FDataflowConstructionScene::UpdateDynamicMeshComponents()
{
	using namespace UE::Geometry;//FDynamicMesh3

	if (!DebugMeshRenderSettings)
	{
		DebugMeshRenderSettings = NewObject<UDataflowDebugMeshRenderSettings>();
	}

	// The preview scene for the construction view will be
	// cleared and rebuilt from scratch. This will generate a
	// list of UPrimitiveComponents for rendering.

	// Views are always read-only: they consume cached values only and never trigger graph evaluation.
	// Evaluation is the toolkit's responsibility (on node selection in Automatic mode, on invalidation via OnNodeInvalidated).
	const bool bEvaluateOutputs = false;

	if (const TObjectPtr<UDataflowBaseContent>& EditorContent = GetEditorContent())
	{
		const TObjectPtr<UDataflow>& DataflowAsset = EditorContent->GetDataflowAsset();
		const TSharedPtr<UE::Dataflow::FEngineContext>& DataflowContext = EditorContent->GetDataflowContext();
		if(DataflowAsset && DataflowContext)
		{
			for (TObjectPtr<const UDataflowEdNode> Target : DataflowAsset->GetRenderTargets())
			{
				UpdateComponentsFromNode(Target, /*bWireframe*/false);
			}

			MeshComponentsForWireframeRendering.Reset();
			for (TObjectPtr<const UDataflowEdNode> Target : DataflowAsset->GetWireframeRenderTargets())
			{
				UpdateComponentsFromNode(Target, /*bWireframe*/true);
			}

			// If we have a single mesh component in the scene, select it
			if (DynamicMeshComponents.Num() == 1 && DataflowModeManager)
			{
				if (USelection* const SelectedComponents = DataflowModeManager->GetSelectedComponents())
				{
					UDynamicMeshComponent* const DynamicMeshComponent = DynamicMeshComponents.CreateIterator()->Value;
					SelectedComponents->Select(DynamicMeshComponent);
					DynamicMeshComponent->PushSelectionToProxy();
				}
			}
		}
	}

	UpdateFloorVisibility();

	bPreviewSceneDirty = true;
}

void FDataflowConstructionScene::UpdatePrimitiveComponents()
{
	for(TObjectPtr<UPrimitiveComponent> PrimitiveComponent : PrimitiveComponents)
	{
		PrimitiveComponent->SelectionOverrideDelegate = UPrimitiveComponent::FSelectionOverride::CreateRaw(this, &FDataflowPreviewSceneBase::IsComponentSelected);
		PrimitiveComponent->UpdateBounds();

		AddComponent(PrimitiveComponent, PrimitiveComponent->GetRelativeTransform());	
		AddSceneObject(PrimitiveComponent, true);
	}
	bPreviewSceneDirty = true;
}

void FDataflowConstructionScene::RemoveSceneComponent(USelection* SelectedComponents, UPrimitiveComponent* PrimitiveComponent)
{
	if(PrimitiveComponent)
	{
		PrimitiveComponent->SelectionOverrideDelegate.Unbind();
		if (SelectedComponents->IsSelected(PrimitiveComponent))
		{
			SelectedComponents->Deselect(PrimitiveComponent);
			PrimitiveComponent->PushSelectionToProxy();
		}
		RemoveSceneObject(PrimitiveComponent);
		RemoveComponent(PrimitiveComponent);
		PrimitiveComponent->DestroyComponent();
	}
}

void FDataflowConstructionScene::ResetSceneComponents()
{
	USelection* SelectedComponents = DataflowModeManager->GetSelectedComponents();
	for (FRenderElement RenderElement : DynamicMeshComponents)
	{
		RemoveSceneComponent(SelectedComponents, RenderElement.Value);
	}
	for (TObjectPtr<UPrimitiveComponent> PrimitiveComponent : PrimitiveComponents)
	{
		RemoveSceneComponent(SelectedComponents, PrimitiveComponent);
	}
	DynamicMeshComponents.Reset();
	PrimitiveComponents.Reset();
	RenderableTypeInstances.Reset();
	bPreviewSceneDirty = true;

	RemoveSceneObject(RootSceneActor);
}

TObjectPtr<UDynamicMeshComponent>& FDataflowConstructionScene::AddDynamicMeshComponent(FDataflowRenderKey InKey, const FString& MeshName, UE::Geometry::FDynamicMesh3&& DynamicMesh, const TArray<UMaterialInterface*>& MaterialSet, const FTransform& InMeshTransform)
{
	const FName UniqueObjectName = MakeUniqueObjectName(RootSceneActor, UDataflowEditorCollectionComponent::StaticClass(), FName(MeshName));
	TObjectPtr<UDataflowEditorCollectionComponent> DynamicMeshComponent = NewObject<UDataflowEditorCollectionComponent>(RootSceneActor, UniqueObjectName);

	DynamicMeshComponent->MeshIndex = InKey.Value;
	DynamicMeshComponent->Node = InKey.Key;
	DynamicMeshComponent->SetMesh(MoveTemp(DynamicMesh));
	
	if (MaterialSet.Num() == 0)
	{
		const TObjectPtr<UDataflowBaseContent>& EditorContent = GetEditorContent();
		if (EditorContent && EditorContent->GetDataflowAsset() && EditorContent->GetDataflowAsset()->Material)
		{
			DynamicMeshComponent->ConfigureMaterialSet({ EditorContent->GetDataflowAsset()->Material });
		}
		else
		{
			ensure(FDataflowEditorStyle::Get().DefaultTwoSidedMaterial);
			DynamicMeshComponent->SetOverrideRenderMaterial(FDataflowEditorStyle::Get().DefaultTwoSidedMaterial);
			DynamicMeshComponent->SetCastShadow(false);
		}
	}
	else
	{
		DynamicMeshComponent->ConfigureMaterialSet(MaterialSet);
	}

	DynamicMeshComponent->SelectionOverrideDelegate = UPrimitiveComponent::FSelectionOverride::CreateRaw(this, &FDataflowPreviewSceneBase::IsComponentSelected);
	DynamicMeshComponent->UpdateBounds();

	// Fix up any triangles without valid material IDs
	int32 DefaultMaterialID = INDEX_NONE;
	for (const int32 TriID : DynamicMeshComponent->GetMesh()->TriangleIndicesItr())
	{
		const int32 MaterialID = DynamicMeshComponent->GetMesh()->Attributes()->GetMaterialID()->GetValue(TriID);
		if (!DynamicMeshComponent->GetMaterial(MaterialID))
		{
			if (DefaultMaterialID == INDEX_NONE)
			{
				DefaultMaterialID = DynamicMeshComponent->GetNumMaterials();
				DynamicMeshComponent->SetMaterial(DefaultMaterialID, FDataflowEditorStyle::Get().VertexMaterial);
			}
			DynamicMeshComponent->GetMesh()->Attributes()->GetMaterialID()->SetValue(TriID, DefaultMaterialID);
		}
	}

	AddComponent(DynamicMeshComponent, InMeshTransform);
	DynamicMeshComponents.Emplace(InKey, DynamicMeshComponent);
	AddSceneObject(DynamicMeshComponent, true);
	return DynamicMeshComponents[InKey];
}

void FDataflowConstructionScene::AddWireframeMeshElementsVisualizer()
{
	ensure(WireframeElements.Num() == 0);
	for (UDynamicMeshComponent* Elem : MeshComponentsForWireframeRendering)
	{
		if (TObjectPtr<UDataflowEditorCollectionComponent> DynamicMeshComponent = Cast<UDataflowEditorCollectionComponent>(Elem))
		{
			// Set up the wireframe display of the rest space mesh.

			TObjectPtr<UMeshElementsVisualizer> WireframeDraw = NewObject<UMeshElementsVisualizer>(RootSceneActor);
			WireframeElements.Add(DynamicMeshComponent, WireframeDraw);

			WireframeDraw->CreateInWorld(GetWorld(), DynamicMeshComponent->GetComponentToWorld());
			checkf(WireframeDraw->Settings, TEXT("Expected UMeshElementsVisualizer::Settings to exist after CreateInWorld"));

			WireframeDraw->Settings->DepthBias = 2.0;
			WireframeDraw->Settings->bAdjustDepthBiasUsingMeshSize = false;
			WireframeDraw->Settings->bShowWireframe = true;
			WireframeDraw->Settings->bShowBorders = true;
			WireframeDraw->Settings->bShowUVSeams = false;
			WireframeDraw->WireframeComponent->BoundaryEdgeThickness = 2;
			DynamicMeshComponent->WireframeComponent = WireframeDraw->WireframeComponent;

			WireframeDraw->SetMeshAccessFunction([DynamicMeshComponent](UMeshElementsVisualizer::ProcessDynamicMeshFunc ProcessFunc)
				{
					ProcessFunc(*DynamicMeshComponent->GetMesh());
				});

			for (FRenderElement RenderElement : DynamicMeshComponents)
			{
				RenderElement.Value->OnMeshChanged.Add(FSimpleMulticastDelegate::FDelegate::CreateLambda([WireframeDraw, this]()
					{
						WireframeDraw->NotifyMeshChanged();
					}));
			}

			WireframeDraw->Settings->bVisible = false;
			PropertyObjectsToTick.Add(WireframeDraw->Settings);
		}
	}
}

void FDataflowConstructionScene::ResetWireframeMeshElementsVisualizer()
{
	for (FRenderWireElement Elem : WireframeElements)
	{
		Elem.Value->Disconnect();
	}
	WireframeElements.Empty();
}

void FDataflowConstructionScene::UpdateWireframeMeshElementsVisualizer()
{
	ResetWireframeMeshElementsVisualizer();
	AddWireframeMeshElementsVisualizer();
}

bool FDataflowConstructionScene::HasRenderableGeometry()
{
	for (FRenderElement RenderElement : DynamicMeshComponents)
	{
		if (RenderElement.Value->GetMesh()->TriangleCount() > 0)
		{
			return true;
		}
	}
	for (const UPrimitiveComponent* PrimitiveComponent : PrimitiveComponents)
	{
		if (PrimitiveComponent->Bounds.SphereRadius > UE_SMALL_NUMBER)
		{
			return true;
		}
	}
	return false;
}

void FDataflowConstructionScene::UpdateFloorVisibility()
{
	// Hide the floor in orthographic view modes
	if (const TObjectPtr<UDataflowBaseContent>& EditorContent = GetEditorContent())
	{
		if (const UE::Dataflow::IDataflowConstructionViewMode* ConstructionViewMode = EditorContent->GetConstructionViewMode())
		{
			if (!ConstructionViewMode->IsPerspective())
			{
				constexpr bool bDontModifyProfile = true;
				SetFloorVisibility(false, bDontModifyProfile);
			}
			else
			{
				// Restore visibility from profile settings
				const int32 ProfileIndex = GetCurrentProfileIndex();
				if (DefaultSettings->Profiles.IsValidIndex(ProfileIndex))
				{
					const bool bProfileSetting = DefaultSettings->Profiles[CurrentProfileIndex].bShowFloor;
					constexpr bool bDontModifyProfile = true;
					SetFloorVisibility(bProfileSetting, bDontModifyProfile);
				}
			}
		}
	}
}

void FDataflowConstructionScene::ResetConstructionScene()
{
	// The ModeManagerss::USelection will hold references to Components, but 
	// does not report them to the garbage collector. We need to clear the
	// saved selection when the scene is rebuilt. @todo(Dataflow) If that 
	// selection needs to persist across render resets, we will also need to
	// buffer the names of the selected objects so they can be reselected.
	if (GetDataflowModeManager())
	{
		if (USelection* SelectedComponents = GetDataflowModeManager()->GetSelectedComponents())
		{
			SelectedComponents->DeselectAll();
		}
	}

	// Some objects, like the UMeshElementsVisualizer and Settings Objects
	// are not part of a tool, so they won't get ticked.This member holds
	// ticked objects that get rebuilt on Update
	PropertyObjectsToTick.Empty();

	ResetWireframeMeshElementsVisualizer();

	ResetSceneComponents();
}

void FDataflowConstructionScene::UpdateConstructionScene()
{
	ResetConstructionScene();

	// Add root actor to TEDS
	AddSceneObject(RootSceneActor, true);

	// The preview scene for the construction view will be
	// cleared and rebuilt from scratch. This will generate a 
	// list of UPrimitiveComponents for rendering.
	UpdateDynamicMeshComponents();
	
	// Attach a wireframe renderer to the DynamicMeshComponents
	UpdateWireframeMeshElementsVisualizer();

	// Update all the primitive components potentially added by the selected node
	UpdatePrimitiveComponents();

	for (const UDynamicMeshComponent* const DynamicMeshComponent : MeshComponentsForWireframeRendering)
	{
		WireframeElements[DynamicMeshComponent]->Settings->bVisible = true;
	}

	if (const TObjectPtr<UDataflowBaseContent>& EditorContent = GetEditorContent())
	{
		EditorContent->SetConstructionDirty(false);
	}

	for(const TObjectPtr<UDataflowBaseContent>& TerminalContent : GetTerminalContents())
	{
		TerminalContent->SetConstructionDirty(false);
	}

	if (OnConstructionSceneUpdated.IsBound())
	{
		OnConstructionSceneUpdated.Broadcast();
	}
}

void FDataflowConstructionScene::SelectNodeComponents(const UDataflowEdNode* EdNode)
{
	TArray<AActor*> FoundActors;
	if (USelection* SelectedComponents = GetDataflowModeManager()->GetSelectedComponents())
	{
		SelectedComponents->Modify();
		SelectedComponents->BeginBatchSelectOperation();

		TArray<TWeakObjectPtr<UObject>> SelectedObjects;
		const int32 NumSelected = SelectedComponents->GetSelectedObjects(SelectedObjects);
		for (TWeakObjectPtr<UObject> WeakObject : SelectedObjects)
		{
			if (WeakObject.IsValid())
			{
				if (UDataflowEditorCollectionComponent* ActorComponent = Cast< UDataflowEditorCollectionComponent>(WeakObject.Get()))
				{
					SelectedComponents->Deselect(ActorComponent);
					ActorComponent->PushSelectionToProxy();
				}
			}
		}

		if (TObjectPtr<AActor> RootActor = GetRootActor())
		{
			for (UActorComponent* ActorComponent : RootActor->GetComponents())
			{
				if (UDataflowEditorCollectionComponent* Component = Cast<UDataflowEditorCollectionComponent>(ActorComponent))
				{
					if (Component->Node == EdNode)
					{
						SelectedComponents->Select(Component);
						Component->PushSelectionToProxy();
					}
				}
			}
		}
		SelectedComponents->EndBatchSelectOperation();
	}
}

TArray<UMaterialInterface*> FDataflowConstructionScene::GetMaterialsFromRenderingFacade(const GeometryCollection::Facades::FRenderingFacade& RenderingFacade, int32 MeshIndex)
{
	TArray<UMaterialInterface*> Materials;

	if (MeshIndex >= 0 && MeshIndex < RenderingFacade.NumGeometry())
	{
		const TManagedArray<FString>& MaterialPaths = RenderingFacade.GetMaterialPaths();
		const int32 MaterialStart = RenderingFacade.GetMaterialStart()[MeshIndex];
		const int32 MaterialCount = RenderingFacade.GetMaterialCount()[MeshIndex];

		for (int32 MaterialIndex = MaterialStart; MaterialIndex < MaterialStart + MaterialCount; ++MaterialIndex)
		{
			const FString& Path = MaterialPaths[MaterialIndex];
			UMaterialInterface* const Material = LoadObject<UMaterialInterface>(nullptr, *Path);
			Materials.Add(Material);
		}
	}
	return Materials;
}

void FDataflowConstructionScene::UpdateComponentsFromNode(const UDataflowEdNode* EdNode, bool bWireframe)
{
	if (EdNode == nullptr)
	{
		return;
	}

	using namespace UE::Geometry;//FDynamicMesh3

	// Views are always read-only: they consume cached values only and never trigger graph evaluation.
	// Evaluation is the toolkit's responsibility (on node selection in Automatic mode, on invalidation via OnNodeInvalidated).
	const bool bEvaluateOutputs = false;

	const TObjectPtr<UDataflowBaseContent>& EditorContent = GetEditorContent();
	const TSharedPtr<UE::Dataflow::FEngineContext> DataflowContext = EditorContent ? EditorContent->GetDataflowContext() : TSharedPtr<UE::Dataflow::FEngineContext>();

	if (EditorContent && DataflowContext)
	{
		const int32 TotalWork = 4.f;
		UE::Dataflow::FScopedProgressNotification ProgressNotification(LOCTEXT("FDataflowConstructionScene_UpdateCOmponentFromNode_Progress", "Generating construction view components..."), TotalWork);

		const UE::Dataflow::IDataflowConstructionViewMode* ConstructionViewMode = EditorContent->GetConstructionViewMode();

		// step 1 : create rendering facade
		ProgressNotification.SetProgress(0);

		TSharedPtr<FManagedArrayCollection> RenderCollection(new FManagedArrayCollection);
		GeometryCollection::Facades::FRenderingFacade RenderingFacade(*RenderCollection);
		RenderingFacade.DefineSchema();

		// do not check debug draw as this is orthogonal to the old/new rendering system 
		const bool bCanRenderOutput = (ConstructionViewMode)
			? UE::Dataflow::CanRenderNodeOutput(*EdNode, *EditorContent, *ConstructionViewMode, /*bCheckDebugDraw*/false)
			: false;

		// step 2 : Render nodes that have explictly specifiy how to be rendered ( Old path )
		ProgressNotification.AddProgress(1.f);
		bool bHasRenderCollectionPrimitives = false;
		if (bCanRenderOutput)
		{
			bHasRenderCollectionPrimitives = UE::Dataflow::RenderNodeOutput(RenderingFacade, *EdNode, *EditorContent, bEvaluateOutputs);
			if (EdNode == EditorContent->GetSelectedNode())
			{
				EditorContent->SetRenderCollection(RenderCollection);
			}
		}

		// step 3 : Collect primitive from nodes that explicly want to generate them 
		ProgressNotification.AddProgress(1.f);

		// Dataflow primitive nodes node explicily provide its own components 
		// We do not check for bHasRenderCollectionPrimitives because some nodes actualy set HasRenderCollectionPrimitives to false while still implementing AddPrimitiveComponents
		// and we need to support that for the time being 
		if (TSharedPtr<const FDataflowNode> DataflowNode = EdNode->GetDataflowNode())
		{
			// sadly AddPrimitiveComponents is not const , so we need to const_cast the pointer 
			// this code path should potentially be deprecated if we adopt the new way of rendering where the rendering callback are making components directly
			if (FDataflowPrimitiveNode* PrimitiveDataflowNode = const_cast<FDataflowPrimitiveNode*>(DataflowNode->AsType<FDataflowPrimitiveNode>()))
			{
				PrimitiveDataflowNode->AddPrimitiveComponents(*DataflowContext, EditorContent->GetRenderCollection(), DataflowContext->Owner, RootSceneActor, PrimitiveComponents);
			}
		}

		// step 4 : convert render collection results in dynamic meshes or get the components from the new rendering system  
		ProgressNotification.AddProgress(1.f);

		if (!bHasRenderCollectionPrimitives)
		{
			if (bCanRenderOutput) // Path where the node has defined it's way of rendering the output using a MACRO on the node definition itself 
			{
				const int32 NumGeometry = RenderingFacade.NumGeometry();
				const float GeometryProgress = (NumGeometry > 0) ? (1.f / (float)NumGeometry) : 0;

				for (int32 MeshIndex = 0; MeshIndex < NumGeometry; ++MeshIndex)
				{
					FDataflowRenderKey DynamicMeshKey{ EdNode, MeshIndex };
					if (bWireframe && DynamicMeshComponents.Contains(DynamicMeshKey))
					{
						UDynamicMeshComponent* const ExistingMeshComponent = DynamicMeshComponents[DynamicMeshKey];
						MeshComponentsForWireframeRendering.Add(ExistingMeshComponent);
					}
					else
					{
						FDynamicMesh3 DynamicMesh;
						UE::Dataflow::Conversion::RenderingFacadeToDynamicMesh(RenderingFacade, MeshIndex, DynamicMesh);

						if (DynamicMesh.VertexCount())
						{
							const FString MeshName = RenderingFacade.GetGeometryName()[MeshIndex];
							const FTransform& MeshTransform = RenderingFacade.GetGeometryTransform()[MeshIndex];
							if (bWireframe)
							{
								// Add hidden DynamicMeshComponents for any targets that we want to render in wireframe
								// 
								// Note: UMeshElementsVisualizers need source meshes to pull from. We add invisible dynamic mesh components to the existing DynamicMeshComponents collection
								// for this purpose, but could have instead created a separate collection of meshes for wireframe rendering. We are choosing to keep all the scene DynamicMeshComponents 
								// in one place and using separate structures to dictate how they are used (MeshComponentsForWireframeRendering in this case), in case visualization requirements 
								// change in the future.
								//
								const FString UniqueObjectName = MakeUniqueObjectName(RootSceneActor, UDataflowEditorCollectionComponent::StaticClass(), FName(MeshName)).ToString();
								UDynamicMeshComponent* const NewDynamicMeshComponent = AddDynamicMeshComponent(DynamicMeshKey, UniqueObjectName, MoveTemp(DynamicMesh), {}, MeshTransform);
								NewDynamicMeshComponent->SetVisibility(false);
								MeshComponentsForWireframeRendering.Add(NewDynamicMeshComponent);
							}
							else
							{
								const TArray<UMaterialInterface*> Materials = GetMaterialsFromRenderingFacade(RenderingFacade, MeshIndex);
								AddDynamicMeshComponent(DynamicMeshKey, MeshName, MoveTemp(DynamicMesh), Materials, MeshTransform);
							}
						}
					}

					ProgressNotification.AddProgress(GeometryProgress);
				}
			}
			else // New method where rendering callbacks generate primitive components directly by output types
			{
				TArray<UE::Dataflow::FRenderableTypeInstance> NodeRenderableTypeInstances;
				if (ConstructionViewMode)
				{
					if (TSharedPtr<const FDataflowNode> DataflowNode = EdNode->GetDataflowNode())
					{
						UE::Dataflow::FRenderableTypeInstance::GetRenderableInstancesForNode(*DataflowContext, *ConstructionViewMode, *DataflowNode, bEvaluateOutputs, NodeRenderableTypeInstances);
					}
				}
				RenderableTypeInstances.Append(NodeRenderableTypeInstances);

				// extract components from instances
				const float RenderableTypeProgress = (NodeRenderableTypeInstances.Num() > 0) ? (1.f / (float)NodeRenderableTypeInstances.Num()) : 0;

				UE::Dataflow::FRenderableComponents NodeRenderableComponents(RootSceneActor);
				TArray<UPrimitiveComponent*> ComponentsFromOutputTypes;
				for (UE::Dataflow::FRenderableTypeInstance& Instance : NodeRenderableTypeInstances)
				{
					// todo : filter by option ? - what option should be enmabled by default ?
					Instance.GetPrimitiveComponents(NodeRenderableComponents);

					ProgressNotification.AddProgress(RenderableTypeProgress);
				}

				if (DebugMeshRenderSettings)
				{
					UE::Dataflow::Private::AddDebugComponents(NodeRenderableComponents, *DebugMeshRenderSettings);
				}

				PrimitiveComponents.Append(NodeRenderableComponents.GetComponents());

				int32 MeshIndex = 0;
				const TArray<UPrimitiveComponent*>& NodeComponents = NodeRenderableComponents.GetComponents();
				for (UPrimitiveComponent* PrimitiveComponent : NodeComponents)
				{
					if (UDynamicMeshComponent* DynamicMeshComponent = Cast<UDynamicMeshComponent>(PrimitiveComponent))
					{
						FDataflowRenderKey DynamicMeshKey{ EdNode, MeshIndex };
						DynamicMeshComponents.Emplace(DynamicMeshKey, DynamicMeshComponent);
						++MeshIndex;
					}
				}
				
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE

