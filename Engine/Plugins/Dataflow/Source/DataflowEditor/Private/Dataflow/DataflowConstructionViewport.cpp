// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowConstructionViewport.h"

#include "DataflowSceneProfileIndexStorage.h"
#include "Dataflow/DataflowActor.h"
#include "Dataflow/DataflowEditorCommands.h"
#include "Dataflow/DataflowEditorMode.h"
#include "Dataflow/DataflowConstructionViewportClient.h"
#include "Dataflow/DataflowEditorToolkit.h"
#include "EditorModeManager.h"
#include "Dataflow/DataflowContent.h"
#include "Dataflow/DataflowConstructionScene.h"
#include "Dataflow/DataflowEditorPreviewSceneBase.h"
#include "Dataflow/DataflowRenderingViewMode.h"
#include "Dataflow/DataflowSimulationPanel.h"
#include "ToolMenus.h"
#include "Dataflow/DataflowConstructionVisualization.h"
#include "ViewportToolbar/UnrealEdViewportToolbar.h"
#include "Widgets/Text/SRichTextBlock.h"
#include "EditorViewportCommands.h"

#define LOCTEXT_NAMESPACE "SDataflowConstructionViewport"

SDataflowConstructionViewport::SDataflowConstructionViewport()
{
}

void SDataflowConstructionViewport::Construct(const FArguments& InArgs, const FAssetEditorViewportConstructionArgs& InViewportConstructionArgs)
{
	SAssetEditorViewport::FArguments ParentArgs;
	ParentArgs._EditorViewportClient = InArgs._ViewportClient;
	DataflowClient = InArgs._ViewportClient;
	SAssetEditorViewport::Construct(ParentArgs, InViewportConstructionArgs);
	Client->VisibilityDelegate.BindSP(this, &SDataflowConstructionViewport::IsVisible);
}

TWeakPtr<FDataflowConstructionScene> SDataflowConstructionViewport::GetConstructionScene() const
{
	if (DataflowClient)
	{
		if (const TSharedPtr<FDataflowEditorToolkit> Toolkit = DataflowClient->GetDataflowEditorToolkit().Pin())
		{
			return Toolkit->GetConstructionScene();
		}
	}
	return nullptr;
}

void SDataflowConstructionViewport::BuildVisualizationMenu(FToolMenuSection& MenuSection)
{
	if (UUnrealEdViewportToolbarContext* Context = MenuSection.FindContext<UUnrealEdViewportToolbarContext>())
	{
		MenuSection.AddSubMenu(
			"Visualization",
			LOCTEXT("VisualizationSubmenuLabel", "Visualization"),
			LOCTEXT("VisualizationSubmenuTooltip", "Controls visualization options"),
			FNewMenuDelegate::CreateLambda([WeakViewport = Context->Viewport](FMenuBuilder& Menu)
				{
					if (const TSharedPtr<SDataflowConstructionViewport> Viewport = StaticCastSharedPtr<SDataflowConstructionViewport>(WeakViewport.Pin()))
					{
						if (const TSharedPtr<FDataflowConstructionViewportClient> DataflowClient = Viewport->GetDataflowClient())
						{
							for (const TPair<FName, TUniquePtr<UE::Dataflow::IDataflowConstructionVisualization>>& Visualization : UE::Dataflow::FDataflowConstructionVisualizationRegistry::GetInstance().GetVisualizations())
							{
								Visualization.Value->ExtendViewportShowMenu(DataflowClient, Menu);
							}
						}
					}
				}),
			false,
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "DetailsView.ViewOptions")
		);
	}
}

void SDataflowConstructionViewport::BuildViewMenu(FToolMenuSection& MenuSection)
{
	if (UUnrealEdViewportToolbarContext* Context = MenuSection.FindContext<UUnrealEdViewportToolbarContext>())
	{
		// Camera / View Options
		const TAttribute<FText> Label = TAttribute<FText>::CreateLambda([WeakViewport = Context->Viewport]
			{
				if (const TSharedPtr<SDataflowConstructionViewport> Viewport = StaticCastSharedPtr<SDataflowConstructionViewport>(WeakViewport.Pin()))
				{
					if (UDataflowEditorMode* EditorMode = Viewport->GetEdMode())
					{
						return EditorMode->GetConstructionViewMode()->GetButtonText();
					}
				}

				return LOCTEXT("DataflowViewLabel", "View");
			});

		MenuSection.AddSubMenu(
			"DataflowView",
			Label,
			LOCTEXT("DataflowViewTooltip", "Display view options for the construction viewport."),
			FNewToolMenuDelegate::CreateLambda([WeakViewport = Context->Viewport](UToolMenu* Menu)
				{
					FToolMenuSection& SimulationSection = Menu->AddSection("Simulation", LOCTEXT("SimulationSection", "Simulation"));

					if (const TSharedPtr<SDataflowConstructionViewport> Viewport = StaticCastSharedPtr<SDataflowConstructionViewport>(WeakViewport.Pin()))
					{
						if (UDataflowEditorMode* EditorMode = Viewport->GetEdMode())
						{
							const bool bIsAnyNodeSelected = EditorMode->IsAnyNodeSelected();

							for (const TPair<FName, TSharedPtr<FUICommandInfo>>& NameAndCommand : FDataflowEditorCommandsImpl::Get().SetConstructionViewModeCommands)
							{
								if (!bIsAnyNodeSelected || EditorMode->CanChangeConstructionViewModeTo(NameAndCommand.Key))
								{
									SimulationSection.AddMenuEntry(NameAndCommand.Value);
								}
							}
						}
					}
				}),
			false,
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "EditorViewport.Perspective")
		);
	}
}

TSharedPtr<SWidget> SDataflowConstructionViewport::BuildViewportToolbar()
{
	const FName ToolbarName = "Dataflow.ConstructionViewportToolbar";
	
	if (!UToolMenus::Get()->IsMenuRegistered(ToolbarName))
	{
		UToolMenu* Menu = UToolMenus::Get()->RegisterMenu(ToolbarName, NAME_None, EMultiBoxType::SlimHorizontalToolBar);
		Menu->StyleName = "ViewportToolbar";
		
		FToolMenuSection& LeftSection = Menu->AddSection("Left");
		{
			//LeftSection.AddEntry(UE::UnrealEd::CreateTransformsSubmenu());
			LeftSection.AddEntry(UE::UnrealEd::CreateSnappingSubmenu());
		}

		FToolMenuSection& RightSection = Menu->AddSection("Right");
		{
			RightSection.Alignment = EToolMenuSectionAlign::Last;

			RightSection.AddEntry(UE::UnrealEd::CreateCameraSubmenu(
				UE::UnrealEd::FViewportCameraMenuOptions()
				.ShowCameraMovement()
				.ShowLensControls()
			));
			RightSection.AddEntry(UE::UnrealEd::CreateViewModesSubmenu());
			RightSection.AddEntry(UE::UnrealEd::CreateDefaultShowSubmenu());
			RightSection.AddEntry(UE::UnrealEd::CreateAssetViewerProfileSubmenu());
			{
				RightSection.AddDynamicEntry("ViewAndVisualization", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& Section)
				{
					BuildVisualizationMenu(Section);
					BuildViewMenu(Section);			
				}));
			}
		}
	}
	
	FToolMenuContext Context;
	{
		Context.AppendCommandList(GetCommandList());
		Context.AddExtender(GetExtenders());
		
		UUnrealEdViewportToolbarContext* ContextObject = UE::UnrealEd::CreateViewportToolbarDefaultContext(SharedThis(this));
		Context.AddObject(ContextObject);
	}
	
	return UToolMenus::Get()->GenerateWidget(ToolbarName, Context);
}

TSharedPtr<IPreviewProfileController> SDataflowConstructionViewport::CreatePreviewProfileController()
{
	TSharedPtr<FDataflowConstructionSceneProfileIndexStorage> ProfileIndexStorage = MakeShared<FDataflowConstructionSceneProfileIndexStorage>(GetConstructionScene());
	return MakeShared<FDataflowPreviewProfileController>(ProfileIndexStorage);
}

void SDataflowConstructionViewport::OnFocusViewportToSelection()
{
	FocusViewportToSelection(/*bInstant*/false);
}

void SDataflowConstructionViewport::FocusViewportToSelection(bool bInstant)
{
	const UDataflowEditorMode* const DataflowEdMode = GetEdMode();

	if (DataflowEdMode && DataflowClient)
	{
		FBox BoundingBox = DataflowEdMode->SelectionBoundingBox();
		if (BoundingBox.IsValid && !(BoundingBox.Min == FVector::Zero() && BoundingBox.Max == FVector::Zero()))
		{
			if (DataflowClient->IsOrtho())
			{
				const FVector Center = BoundingBox.GetCenter();
				const FVector Extent = BoundingBox.GetExtent() * 0.5;
				BoundingBox = FBox(Center - Extent, Center + Extent);
			}

			DataflowClient->FocusViewportOnBoundingBox(BoundingBox, bInstant);
		}
	}
}

UDataflowEditorMode* SDataflowConstructionViewport::GetEdMode() const
{
	if (const FEditorModeTools* const EditorModeTools = Client->GetModeTools())
	{
		if (UDataflowEditorMode* const DataflowEdMode = Cast<UDataflowEditorMode>(EditorModeTools->GetActiveScriptableMode(UDataflowEditorMode::EM_DataflowEditorModeId)))
		{
			return DataflowEdMode;
		}
	}
	return nullptr;
}

void SDataflowConstructionViewport::BindCommands()
{
	SAssetEditorViewport::BindCommands();

	const FDataflowEditorCommandsImpl& CommandInfos = FDataflowEditorCommands::Get();

	for (const TPair<FName, TSharedPtr<FUICommandInfo>>& SetViewModeCommand : CommandInfos.SetConstructionViewModeCommands)
	{
		CommandList->MapAction(
			SetViewModeCommand.Value,
			FExecuteAction::CreateLambda([this, ViewModeName = SetViewModeCommand.Key]()
			{
				if (UDataflowEditorMode* const EdMode = GetEdMode())
				{
					EdMode->SetConstructionViewMode(ViewModeName);
				}
			}),
			FCanExecuteAction::CreateLambda([this, ViewModeName = SetViewModeCommand.Key]()
			{ 
				if (const UDataflowEditorMode* const EdMode = GetEdMode())
				{
					const bool bIsAnyNodeSelected = EdMode->IsAnyNodeSelected();
					return !bIsAnyNodeSelected || EdMode->CanChangeConstructionViewModeTo(ViewModeName);
				}
				return false; 
			}),
			FIsActionChecked::CreateLambda([this, ViewModeName = SetViewModeCommand.Key]()
			{
				if (const UDataflowEditorMode* const EdMode = GetEdMode())
				{
					return EdMode->GetConstructionViewMode()->GetName() == ViewModeName;
				}
				return false;
			})
		);

		FUIAction InvalidAction;
		InvalidAction.IsActionVisibleDelegate.BindLambda([]() { return false; });
	}

	// Now bind the visibility of the camnera view commands to the selected dataflow view mode perspective flag 
	auto OverrideCameraViewVisibility = [this](const TSharedPtr<FUICommandInfo>& Command, ELevelViewportType ViewportType)
		{
			TWeakPtr<SDataflowConstructionViewport> WeakSelf = StaticCastWeakPtr<SDataflowConstructionViewport>(AsWeak());
			if (CommandList)
			{
				if (const FUIAction* CameraViewAction = CommandList->GetActionForCommand(Command))
				{
					FUIAction OverridenCameraViewAction = *CameraViewAction;
					OverridenCameraViewAction.CanExecuteAction = FCanExecuteAction::CreateLambda(
						[ViewportType, WeakSelf]() -> bool
						{
							if (const TSharedPtr<SDataflowConstructionViewport> Viewport = WeakSelf.Pin())
							{
								if (UDataflowEditorMode* EditorMode = Viewport->GetEdMode())
								{
									if (const UE::Dataflow::IDataflowConstructionViewMode* CurrentViewMode = EditorMode->GetConstructionViewMode())
									{
										const ELevelViewportType ViewModeViewportType = CurrentViewMode->GetViewportType();
										// perspective view mode can use any camera view type
										// ortho view modes are constrained by their unique camera veiw type
										return (ViewModeViewportType == ELevelViewportType::LVT_Perspective || ViewModeViewportType == ViewportType);
									}
								}
							}
							return true;
						});
					CommandList->MapAction(Command, OverridenCameraViewAction);
				}
			}
		};
	
	OverrideCameraViewVisibility(FEditorViewportCommands::Get().Perspective, ELevelViewportType::LVT_Perspective);
	OverrideCameraViewVisibility(FEditorViewportCommands::Get().Front, ELevelViewportType::LVT_OrthoFront);
	OverrideCameraViewVisibility(FEditorViewportCommands::Get().Left, ELevelViewportType::LVT_OrthoLeft);
	OverrideCameraViewVisibility(FEditorViewportCommands::Get().Top, ELevelViewportType::LVT_OrthoTop);
	OverrideCameraViewVisibility(FEditorViewportCommands::Get().Back, ELevelViewportType::LVT_OrthoBack);
	OverrideCameraViewVisibility(FEditorViewportCommands::Get().Right, ELevelViewportType::LVT_OrthoRight);
	OverrideCameraViewVisibility(FEditorViewportCommands::Get().Bottom, ELevelViewportType::LVT_OrthoBottom);
}

bool SDataflowConstructionViewport::IsVisible() const
{
	// Intentionally not calling SEditorViewport::IsVisible because it will return false if our simulation is more than 250ms.
	return ViewportWidget.IsValid();
}

TSharedRef<class SEditorViewport> SDataflowConstructionViewport::GetViewportWidget()
{
	return SharedThis(this);
}

TSharedPtr<FExtender> SDataflowConstructionViewport::GetExtenders() const
{
	TSharedPtr<FExtender> Result(MakeShareable(new FExtender));
	return Result;
}

void SDataflowConstructionViewport::OnFloatingButtonClicked()
{
}

void SDataflowConstructionViewport::PopulateViewportOverlays(TSharedRef<SOverlay> Overlay)
{
	SEditorViewport::PopulateViewportOverlays(Overlay);

	Overlay->AddSlot()
		.VAlign(VAlign_Top)
		.HAlign(HAlign_Left)
		.Padding(6.0f)
		[
			SNew(SBorder)
				.BorderImage(FAppStyle::Get().GetBrush("FloatingBorder"))
				.Padding(4.f)
				[
					SNew(SRichTextBlock)
					.Text(this, &SDataflowConstructionViewport::GetOverlayText)
				]
		];

	Overlay->AddSlot()
		.VAlign(VAlign_Top)
		.HAlign(HAlign_Fill)
		[
			SNew(SBorder)
			.BorderBackgroundColor(FColor::Orange)
			.BorderImage(FCoreStyle::Get().GetBrush("ErrorReporting.Box"))
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.Padding(FMargin(8, 8))
			.Visibility(this, &SDataflowConstructionViewport::GetEditorModeMessageVisibility)
			[
				SNew(STextBlock)
				.ColorAndOpacity(FColor::Black)
				.Text(this, &SDataflowConstructionViewport::GetEditorModeMessage)
			]
		];

	// this widget will display the current viewed feature level
	Overlay->AddSlot()
		.VAlign(VAlign_Bottom)
		.HAlign(HAlign_Right)
		.Padding(5.0f)
		[
			BuildShaderPlatformWidget()
		];
}

bool SDataflowConstructionViewport::HasEditorModeMessage() const
{
	if (const UDataflowEditorMode* const DataflowEdMode = GetEdMode())
	{
		return DataflowEdMode->HasUserMessage();
	}
	return false;
}

EVisibility SDataflowConstructionViewport::GetEditorModeMessageVisibility() const
{
	return HasEditorModeMessage()
		? EVisibility::Visible
		: EVisibility::Collapsed;
}

FText SDataflowConstructionViewport::GetEditorModeMessage() const
{
	if (const UDataflowEditorMode* const DataflowEdMode = GetEdMode())
	{
		return DataflowEdMode->GetUserMessage();
	}
	return {};
}

FText SDataflowConstructionViewport::GetOverlayText() const
{
	if (DataflowClient)
	{
		return FText::FromString(DataflowClient->GetOverlayString());
	}

	return FText();
}

#undef LOCTEXT_NAMESPACE
