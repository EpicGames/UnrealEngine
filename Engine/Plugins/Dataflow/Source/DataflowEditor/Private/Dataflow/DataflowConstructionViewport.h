// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "SBaseCharacterFXEditorViewport.h"
#include "SCommonEditorViewportToolbarBase.h"
#include "Widgets/Notifications/SErrorText.h"

class UDataflowEditorMode;
class FDataflowConstructionScene;
class FDataflowConstructionViewportClient;
class SRichTextBlock;
class SErrorText;

struct FToolMenuSection;

// ----------------------------------------------------------------------------------

class SDataflowConstructionViewport : public SBaseCharacterFXEditorViewport, public ICommonEditorViewportToolbarInfoProvider
{
public:
	SLATE_BEGIN_ARGS(SDataflowConstructionViewport) {}
	SLATE_ARGUMENT(TSharedPtr<FDataflowConstructionViewportClient>, ViewportClient)
	SLATE_END_ARGS()

	SDataflowConstructionViewport();

	void Construct(const FArguments& InArgs, const FAssetEditorViewportConstructionArgs& InViewportConstructionArgs);

	// SEditorViewport
	virtual void BindCommands() override;
	virtual TSharedPtr<SWidget> BuildViewportToolbar() override;
	virtual TSharedPtr<IPreviewProfileController> CreatePreviewProfileController() override;
	virtual bool IsVisible() const override;
	virtual void OnFocusViewportToSelection() override;

	// ICommonEditorViewportToolbarInfoProvider
	virtual TSharedRef<class SEditorViewport> GetViewportWidget() override;
	virtual TSharedPtr<FExtender> GetExtenders() const override;
	virtual void OnFloatingButtonClicked() override;
	virtual void PopulateViewportOverlays(TSharedRef<SOverlay> Overlay) override;

	FText GetOverlayText() const;
	bool HasEditorModeMessage() const;
	FText GetEditorModeMessage() const;
	EVisibility GetEditorModeMessageVisibility() const;
	void FocusViewportToSelection(bool bInstant);

	TSharedPtr<FDataflowConstructionViewportClient> GetDataflowClient() const { return DataflowClient; }

private:
	static void BuildVisualizationMenu(FToolMenuSection& MenuSection);
	static void BuildViewMenu(FToolMenuSection& MenuSection);

	TWeakPtr<FDataflowConstructionScene> GetConstructionScene() const;

	TSharedPtr<FDataflowConstructionViewportClient> DataflowClient;

	/** Pointer to the box into which the overlay text items are added */
	TSharedPtr<SRichTextBlock> OverlayText;
	TSharedPtr<SErrorText> MessageWidget;

	UDataflowEditorMode* GetEdMode() const;
};

