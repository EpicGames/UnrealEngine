// Copyright Epic Games, Inc. All Rights Reserved.
#include "Dataflow/DataflowConstructionViewportClient.h"

#include "AssetEditorModeManager.h"
#include "Dataflow/DataflowObject.h"
#include "Dataflow/DataflowEditorMode.h"
#include "Dataflow/DataflowEditorOptions.h"
#include "Dataflow/DataflowEditorToolkit.h"
#include "Dataflow/DataflowEditorCollectionComponent.h"
#include "Dataflow/DataflowEngineSceneHitProxies.h"
#include "Dataflow/DataflowGraphEditor.h"
#include "Dataflow/DataflowConstructionScene.h"
#include "Dataflow/DataflowConstructionVisualization.h"
#include "Dataflow/DataflowEditorPreviewSceneBase.h"
#include "Dataflow/DataflowRenderingViewMode.h"
#include "EditorModeManager.h"
#include "Framework/Application/SlateApplication.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "GraphEditor.h"
#include "PreviewScene.h"
#include "Selection.h"
#include "SGraphPanel.h"
#include "SNodePanel.h"
#include "BaseBehaviors/ClickDragBehavior.h"
#include "BaseBehaviors/MouseWheelBehavior.h"
#include "Behaviors/2DViewportBehaviorTargets.h"

#define LOCTEXT_NAMESPACE "DataflowConstructionViewportClient"

FDataflowConstructionViewportClient::FDataflowConstructionViewportClient(FEditorModeTools* InModeTools,
                                                             TWeakPtr<FDataflowConstructionScene> InConstructionScene, const bool bCouldTickScene,
                                                             const TWeakPtr<SEditorViewport> InEditorViewportWidget)
	: FDataflowEditorViewportClientBase(InModeTools, InConstructionScene, bCouldTickScene, InEditorViewportWidget)
	, ConstructionScene(InConstructionScene)
{
	// We want our near clip plane to be quite close so that we can zoom in further.
	OverrideNearClipPlane(KINDA_SMALL_NUMBER);

	// Allow focusing on small objects
	MinimumFocusRadius = 0.1f;

	EngineShowFlags.SetSelectionOutline(true);
	EngineShowFlags.EnableAdvancedFeatures();

	bEnableSceneTicking = bCouldTickScene;

	UpdateFromSettings();

	if (UDataflowEditorOptions* const MutableOptions = GetMutableDefault<UDataflowEditorOptions>())
	{
		OnSettingsChangeHandle = MutableOptions->OnSettingChanged().AddLambda([this](UObject*, FPropertyChangedEvent& Event) { UpdateFromSettings(); });

		SetCameraSpeedSettings(MutableOptions->ConstructionCameraSpeedSettings);
	}

	WidgetMode = UE::Widget::WM_None;
	if (ModeTools)
	{
		ModeTools->SetWidgetMode(WidgetMode);
		ModeTools->SetCoordSystem(COORD_Local);
	}
}

bool FDataflowConstructionViewportClient::ShouldOrbitCamera() const
{
	if (ConstructionViewMode && ConstructionViewMode->IsPerspective())
	{
		return FEditorViewportClient::ShouldOrbitCamera();
	}
	return false;
}


void FDataflowConstructionViewportClient::UpdateFromSettings()
{
	BehaviorsFor2DMode.Reset();
	OrthoScrollBehaviorTarget = MakeUnique<FEditor2DScrollBehaviorTarget>(this);

	bool bUseRightMouseButton = true;
	bool bUseMiddleMouseButton = true;
	if (const UDataflowEditorOptions* const Options = GetDefault<UDataflowEditorOptions>())
	{
		bUseRightMouseButton = (Options->ConstructionViewportMousePanButton == EDataflowConstructionViewportMousePanButton::Right || Options->ConstructionViewportMousePanButton == EDataflowConstructionViewportMousePanButton::RightOrMiddle);
		bUseMiddleMouseButton = (Options->ConstructionViewportMousePanButton == EDataflowConstructionViewportMousePanButton::Middle || Options->ConstructionViewportMousePanButton == EDataflowConstructionViewportMousePanButton::RightOrMiddle);
	}

	// We'll have the priority of our viewport behaviors be lower (i.e. higher numerically) than both the gizmo default and the tool default
	constexpr int ViewportBehaviorPriority = FMath::Max(FInputCapturePriority::DEFAULT_GIZMO_PRIORITY, FInputCapturePriority::DEFAULT_TOOL_PRIORITY) + 3;

	if (bUseRightMouseButton)
	{
		UClickDragInputBehavior* const RightMouseClickDragInputBehavior = NewObject<UClickDragInputBehavior>();
		RightMouseClickDragInputBehavior->Initialize(OrthoScrollBehaviorTarget.Get());
		RightMouseClickDragInputBehavior->SetDefaultPriority(ViewportBehaviorPriority);
		RightMouseClickDragInputBehavior->SetUseRightMouseButton();
		BehaviorsFor2DMode.Add(RightMouseClickDragInputBehavior);
	}

	if (bUseMiddleMouseButton)
	{
		UClickDragInputBehavior* const MiddleMouseClickDragInputBehavior = NewObject<UClickDragInputBehavior>();
		MiddleMouseClickDragInputBehavior->Initialize(OrthoScrollBehaviorTarget.Get());
		MiddleMouseClickDragInputBehavior->SetDefaultPriority(ViewportBehaviorPriority);
		MiddleMouseClickDragInputBehavior->SetUseMiddleMouseButton();
		BehaviorsFor2DMode.Add(MiddleMouseClickDragInputBehavior);
	}

	ZoomBehaviorTarget = MakeUnique<FEditor2DMouseWheelZoomBehaviorTarget>(this);
	{
		constexpr float CameraFarPlaneWorldZ = -10.0f;
		constexpr float CameraNearPlaneProportionZ = 0.8f;
		ZoomBehaviorTarget->SetCameraFarPlaneWorldZ(CameraFarPlaneWorldZ);
		ZoomBehaviorTarget->SetCameraNearPlaneProportionZ(CameraNearPlaneProportionZ);
		ZoomBehaviorTarget->SetZoomLimits(0.001, 100000);
		UMouseWheelInputBehavior* const ZoomBehavior = NewObject<UMouseWheelInputBehavior>();
		ZoomBehavior->Initialize(ZoomBehaviorTarget.Get());
		ZoomBehavior->SetDefaultPriority(ViewportBehaviorPriority);
		BehaviorsFor2DMode.Add(ZoomBehavior);
	}

	if (const UDataflowEditorOptions* const Options = UDataflowEditorOptions::StaticClass()->GetDefaultObject<UDataflowEditorOptions>())
	{
		FOVAngle = Options->ConstructionViewFOV;
		ViewFOV = FOVAngle;
		ExposureSettings.bFixed = Options->bConstructionViewFixedExposure;
	}

	if (ConstructionViewMode != nullptr)
	{
		SetConstructionViewMode(ConstructionViewMode);
	}
}

FDataflowConstructionViewportClient::~FDataflowConstructionViewportClient()
{
	if (UDataflowEditorOptions* const MutableOptions = GetMutableDefault<UDataflowEditorOptions>())
	{
		MutableOptions->OnSettingChanged().Remove(OnSettingsChangeHandle);
	}

	if (UDataflowEditorOptions* const Options = UDataflowEditorOptions::StaticClass()->GetDefaultObject<UDataflowEditorOptions>())
	{
		Options->ConstructionViewFOV = FOVAngle;
		Options->bConstructionViewFixedExposure = ExposureSettings.bFixed;
		Options->ConstructionCameraSpeedSettings = GetCameraSpeedSettings();
		Options->SaveConfig();
	}
}

UE::Widget::EWidgetMode FDataflowConstructionViewportClient::GetWidgetMode() const
{
	// No transform gizmo yet available in the viewport
	return UE::Widget::WM_None;
}

bool FDataflowConstructionViewportClient::CanSetWidgetMode(UE::Widget::EWidgetMode NewMode) const
{
	return	NewMode == UE::Widget::EWidgetMode::WM_Translate
			|| NewMode == UE::Widget::EWidgetMode::WM_Scale
			|| NewMode == UE::Widget::EWidgetMode::WM_Rotate;
}

void FDataflowConstructionViewportClient::SetWidgetMode(UE::Widget::EWidgetMode NewMode)
{
	if (ModeTools)
	{
		ModeTools->SetWidgetMode(NewMode);
	}
	WidgetMode = NewMode;
}

void FDataflowConstructionViewportClient::SetDataflowEditorToolkit(TWeakPtr<FDataflowEditorToolkit> InDataflowEditorToolkitPtr)
{
	DataflowEditorToolkitPtr = InDataflowEditorToolkitPtr;
}

void FDataflowConstructionViewportClient::SetToolCommandList(TWeakPtr<FUICommandList> InToolCommandList)
{
	ToolCommandList = InToolCommandList;
}

void FDataflowConstructionViewportClient::Tick(float DeltaSeconds)
{
	Super::Tick(DeltaSeconds);
	if (const TSharedPtr<FDataflowConstructionScene> DataflowConstructionScenePtr = ConstructionScene.Pin())
	{
		DataflowConstructionScenePtr->TickDataflowScene(DeltaSeconds);
	}
}

void FDataflowConstructionViewportClient::ShowMeshEdges(bool bShow)
{
	EngineShowFlags.SetMeshEdges(bShow);
	Invalidate();
}

USelection* FDataflowConstructionViewportClient::GetSelectedComponents() const 
{
	if (const TSharedPtr<FDataflowConstructionScene> DataflowConstructionScenePtr = ConstructionScene.Pin())
	{
		if (USelection* const SceneSelection = DataflowConstructionScenePtr->GetSelectedComponents())
		{
			return SceneSelection;
		}
	}
	return ModeTools? ModeTools->GetSelectedComponents(): nullptr;
}

bool FDataflowConstructionViewportClient::InputKey(const FInputKeyEventArgs& EventArgs)
{
	// See if any tool commands want to handle the key event
	const TSharedPtr<FUICommandList> PinnedToolCommandList = ToolCommandList.Pin();
	if (EventArgs.Event != IE_Released && PinnedToolCommandList.IsValid())
	{
		const FModifierKeysState KeyState = FSlateApplication::Get().GetModifierKeys();
		if (PinnedToolCommandList->ProcessCommandBindings(EventArgs.Key, KeyState, (EventArgs.Event == IE_Repeat)))
		{
			return true;
		}
	}
	// Only route input keys to the super class if we are in perspective view mode
	// This is to avoid being able to orbit the camera in 2D views
	if (ConstructionViewMode && ConstructionViewMode->IsPerspective())
	{
		return FDataflowEditorViewportClientBase::InputKey(EventArgs);
	}

	// We'll support disabling input like our base class, even if it does not end up being used.
	if (bDisableInput)
	{
		return true;
	}

	// Our viewport manipulation is placed in the input router that ModeTools manages
	return ModeTools
		? ModeTools->InputKey(this, EventArgs.Viewport, EventArgs.Key, EventArgs.Event)
		: false;
}


void FDataflowConstructionViewportClient::ProcessClick(FSceneView& View, HHitProxy* HitProxy, FKey Key, EInputEvent Event, uint32 HitX, uint32 HitY)
{
	Super::ProcessClick(View, HitProxy, Key, Event, HitX, HitY);
	OnViewportClicked(HitProxy);
}

void FDataflowConstructionViewportClient::OnViewportClicked(HHitProxy* HitProxy)
{
	auto EnableToolForSelectedNode = [&](USelection* SelectedComponents)
	{
		if (TSharedPtr<FDataflowEditorToolkit> DataflowEditorToolkit = DataflowEditorToolkitPtr.Pin())
		{
			const TSharedPtr<FDataflowConstructionScene> ConstructionScenePtr = ConstructionScene.Pin();
			if (ConstructionScenePtr && ConstructionScenePtr->GetDataflowModeManager())
			{
				if (UDataflowEditorMode* DataflowMode = Cast<UDataflowEditorMode>(ConstructionScenePtr->GetDataflowModeManager()->GetActiveScriptableMode(UDataflowEditorMode::EM_DataflowEditorModeId)))
				{
					if (TSharedPtr<SDataflowGraphEditor> GraphEditor = DataflowEditorToolkit->GetDataflowGraphEditor())
					{
						if (UEdGraphNode* SelectedNode = GraphEditor->GetSingleSelectedNode())
						{
							if (SelectedComponents && SelectedComponents->Num() == 1)
							{
								DataflowMode->StartToolForSelectedNode(SelectedNode);
							}
						}
					}
				}
			}
		}
	};

	auto UpdateSelectedComponentInViewport = [&](USelection* SelectedComponents)
	{
		TArray<UPrimitiveComponent*> PreviouslySelectedComponents;
		SelectedComponents->GetSelectedObjects<UPrimitiveComponent>(PreviouslySelectedComponents);

		SelectedComponents->Modify();
		SelectedComponents->BeginBatchSelectOperation();

		SelectedComponents->DeselectAll();

		if (HitProxy && HitProxy->IsA(HActor::StaticGetType()))
		{
			const HActor* ActorProxy = static_cast<HActor*>(HitProxy);
			if (ActorProxy && ActorProxy->PrimComponent && ActorProxy->Actor)
			{
				UPrimitiveComponent* Component = const_cast<UPrimitiveComponent*>(ActorProxy->PrimComponent.Get());
				SelectedComponents->Select(Component);
				Component->PushSelectionToProxy();
			}
		}

		SelectedComponents->EndBatchSelectOperation();

		for (UPrimitiveComponent* const Component : PreviouslySelectedComponents)
		{
			Component->PushSelectionToProxy();
		}
	};

	auto SelectSingleNodeInGraph = [&](TObjectPtr<const UDataflowEdNode> Node)
	{
		if (TSharedPtr<FDataflowEditorToolkit> DataflowEditorToolkit = DataflowEditorToolkitPtr.Pin())
		{
			if (TSharedPtr<SDataflowGraphEditor> GraphEditor = DataflowEditorToolkit->GetDataflowGraphEditor())
			{
				GraphEditor->GetGraphPanel()->SelectionManager.SelectSingleNode((UObject*)Node.Get());
			}
		}
	};
	
	auto IsInteractiveToolActive = [&]()
	{
		if (const TSharedPtr<FDataflowConstructionScene> ConstructionScenePtr = ConstructionScene.Pin())
		{
			if (UDataflowEditorMode* const DataflowMode = Cast<UDataflowEditorMode>(
				ConstructionScenePtr->GetDataflowModeManager()->GetActiveScriptableMode(UDataflowEditorMode::EM_DataflowEditorModeId)))
			{
				if (UEditorInteractiveToolsContext* const ToolsContext = DataflowMode->GetInteractiveToolsContext())
				{
					return ToolsContext->HasActiveTool();
				}
			}
		}
		return false;
	};

	TArray<UPrimitiveComponent*> CurrentlySelectedComponents;
	if (!IsInteractiveToolActive())
	{
		if (USelection* SelectedComponents = GetSelectedComponents())
		{
			UpdateSelectedComponentInViewport(SelectedComponents);

			if (bool bIsAltKeyDown = Viewport->KeyState(EKeys::LeftAlt) || Viewport->KeyState(EKeys::RightAlt))
			{
				if (UDataflowEditorCollectionComponent* DataflowComponent
					= SelectedComponents->GetBottom<UDataflowEditorCollectionComponent>())
				{
					SelectSingleNodeInGraph(DataflowComponent->Node);
				}
			}

			EnableToolForSelectedNode(SelectedComponents);

			SelectedComponents->GetSelectedObjects<UPrimitiveComponent>(CurrentlySelectedComponents);
		}
	}
	
	// Get all the scene selected elements 
	TArray<FDataflowBaseElement*> DataflowElements;
	GetSelectedElements(HitProxy, DataflowElements);
	
	OnSelectionChangedMulticast.Broadcast(CurrentlySelectedComponents, DataflowElements);
}

void FDataflowConstructionViewportClient::FocusViewportOnBoundingBox(const FBox& BoundingBox, bool bInstant)
{
	FDataflowEditorViewportClientBase::FocusViewportOnBox(BoundingBox, bInstant);

	// make sure we adjust the near and far planes for 2D since the focus function above may move the camera so that the zero plane is no longer visible
	if (ConstructionViewMode && !ConstructionViewMode->IsPerspective())
	{
		const double AbsZ = FMath::Abs(ViewTransformPerspective.GetLocation().Z);
		constexpr double CameraFarPlaneWorldZ = -10.0;
		constexpr double CameraNearPlaneProportionZ = 0.8;
		OverrideFarClipPlane(static_cast<float>(AbsZ - CameraFarPlaneWorldZ));
		OverrideNearClipPlane(static_cast<float>(AbsZ * (1.0 - CameraNearPlaneProportionZ)));
	}
}

void FDataflowConstructionViewportClient::SetConstructionViewMode(const UE::Dataflow::IDataflowConstructionViewMode* InViewMode)
{
	checkf(InViewMode, TEXT("SetConstructionViewMode received null IDataflowConstructionViewMode pointer"));

	if (ConstructionViewMode)
	{
		SavedInactiveViewTransforms.FindOrAdd(ConstructionViewMode->GetName()) = GetViewTransform();
	}

	ConstructionViewMode = InViewMode;

	if (ConstructionViewMode && ConstructionViewMode->IsPerspective())
	{
		OverrideFarClipPlane(0);
		OverrideNearClipPlane(UE_KINDA_SMALL_NUMBER);
	}
	else
	{
		const double AbsZ = FMath::Abs(ViewTransformPerspective.GetLocation().Z);
		constexpr double CameraFarPlaneWorldZ = -10.0;
		constexpr double CameraNearPlaneProportionZ = 0.8;
		OverrideFarClipPlane(static_cast<float>(AbsZ - CameraFarPlaneWorldZ));
		OverrideNearClipPlane(static_cast<float>(AbsZ * (1.0 - CameraNearPlaneProportionZ)));
	}

	BehaviorSet->RemoveAll();

	if (InViewMode->IsPerspective())
	{
		for (UInputBehavior* const Behavior : BaseBehaviors)
		{
			BehaviorSet->Add(Behavior);
		}
	}
	else
	{
		for (UInputBehavior* const Behavior : BehaviorsFor2DMode)
		{
			BehaviorSet->Add(Behavior);
		}
	}

	if (ModeTools)
	{
		ModeTools->GetInteractiveToolsContext()->InputRouter->DeregisterSource(this);
		ModeTools->GetInteractiveToolsContext()->InputRouter->RegisterSource(this);
	}

	if (const FViewportCameraTransform* const FoundPreviousTransform = SavedInactiveViewTransforms.Find(InViewMode->GetName()))
	{
		ViewTransformPerspective = *FoundPreviousTransform;
	}
	else
	{
		// TODO: Default view transform
	}

	bDrawAxes = ConstructionViewMode->IsPerspective();
	//EngineShowFlags.SetGrid(ConstructionViewMode->IsPerspective());

	if (TSharedPtr<FDataflowEditorToolkit> DataflowEditorToolkit = DataflowEditorToolkitPtr.Pin())
	{
		DataflowEditorToolkit->OnConstructionViewModeChanged(ConstructionViewMode);
	}

	Invalidate();
}

void FDataflowConstructionViewportClient::Draw(const FSceneView* View, FPrimitiveDrawInterface* PDI)
{
	FDataflowEditorViewportClientBase::Draw(View, PDI);

	if (const TSharedPtr<FDataflowConstructionScene> ConstructionScenePtr = ConstructionScene.Pin())
	{
		using namespace UE::Dataflow;
		for (const TPair<FName, TUniquePtr<IDataflowConstructionVisualization>>& Visualization : FDataflowConstructionVisualizationRegistry::GetInstance().GetVisualizations())
		{
			Visualization.Value->Draw(ConstructionScenePtr.Get(), PDI, View);
		}
	}
}

void FDataflowConstructionViewportClient::DrawCanvas(FViewport& InViewport, FSceneView& View, FCanvas& Canvas)
{
	FEditorViewportClient::DrawCanvas(InViewport, View, Canvas);

	if (const TSharedPtr<FDataflowConstructionScene> ConstructionScenePtr = ConstructionScene.Pin())
	{
		using namespace UE::Dataflow;
		for (const TPair<FName, TUniquePtr<IDataflowConstructionVisualization>>& Visualization : FDataflowConstructionVisualizationRegistry::GetInstance().GetVisualizations())
		{
			Visualization.Value->DrawCanvas(ConstructionScenePtr.Get(), &Canvas, &View);
		}
	}

	if (const TSharedPtr<FDataflowEditorToolkit> DataflowEditorToolkit = DataflowEditorToolkitPtr.Pin())
	{
		DataflowEditorToolkit->DrawPropertyGizmoLabels(&Canvas, &View);
	}
}

float FDataflowConstructionViewportClient::GetMinimumOrthoZoom() const
{
	// Ignore ULevelEditorViewportSettings::MinimumOrthographicZoom in this viewport client
	return 1.0f;
}

void FDataflowConstructionViewportClient::AddReferencedObjects(FReferenceCollector& Collector)
{
	FDataflowEditorViewportClientBase::AddReferencedObjects(Collector);
	Collector.AddReferencedObjects(BehaviorsFor2DMode);
}

FString FDataflowConstructionViewportClient::GetOverlayString() const
{
	if (TSharedPtr<FDataflowEditorToolkit> DataflowEditorToolkit = DataflowEditorToolkitPtr.Pin())
	{
		return DataflowEditorToolkit->GetDebugDrawOverlayString();
	}

	return {};
}

#undef LOCTEXT_NAMESPACE 
