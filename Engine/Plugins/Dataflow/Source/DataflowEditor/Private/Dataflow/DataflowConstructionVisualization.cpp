// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowConstructionVisualization.h"
#include "Misc/LazySingleton.h"
#include "ChaosLog.h"

#define LOCTEXT_NAMESPACE "DataflowConstructionVisualization"

namespace UE::Dataflow
{
	FDataflowConstructionVisualizationRegistry& FDataflowConstructionVisualizationRegistry::GetInstance()
	{
		return TLazySingleton<FDataflowConstructionVisualizationRegistry>::Get();
	}

	void FDataflowConstructionVisualizationRegistry::TearDown()
	{
		return TLazySingleton<FDataflowConstructionVisualizationRegistry>::TearDown();
	}

	void FDataflowConstructionVisualizationRegistry::RegisterVisualization(TUniquePtr<IDataflowConstructionVisualization>&& Visualization)
	{
		const FName NewVisualizationName = Visualization->GetName();
		if (VisualizationMap.Contains(NewVisualizationName))
		{
			UE_LOGF(LogChaos, Warning, "Dataflow construction viewport visualization registration conflicts with existing visualization: %ls", *NewVisualizationName.ToString());
		}
		else
		{
			VisualizationMap.Add(NewVisualizationName, MoveTemp(Visualization));
		}
	}

	void FDataflowConstructionVisualizationRegistry::DeregisterVisualization(const FName& VisualizationName)
	{
		if (!VisualizationMap.Contains(VisualizationName))
		{
			UE_LOGF(LogChaos, Warning, "Dataflow visualization deregistration -- visualization not registered : %ls", *VisualizationName.ToString());
		}
		else
		{
			VisualizationMap.Remove(VisualizationName);
		}
	}

	const TMap<FName, TUniquePtr<IDataflowConstructionVisualization>>& FDataflowConstructionVisualizationRegistry::GetVisualizations() const
	{
		return VisualizationMap;
	}

	const IDataflowConstructionVisualization* FDataflowConstructionVisualizationRegistry::GetVisualization(const FName& Name) const
	{
		if (VisualizationMap.Contains(Name))
		{
			return VisualizationMap[Name].Get();
		}
		return nullptr;
	}

}

#undef LOCTEXT_NAMESPACE 

