// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowEditor.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "Animation/Skeleton.h"
#include "ContentBrowserModule.h"
#include "Dataflow/AssetDefinition_DataflowContext.h"
#include "Dataflow/DataflowContent.h"
#include "Dataflow/DataflowEditorToolkit.h"
#include "Dataflow/DataflowEditorUtil.h"
#include "Engine/SkeletalMesh.h"
#include "Modules/ModuleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowEditor)

DEFINE_LOG_CATEGORY(LogDataflowEditor);

UDataflowEditor::UDataflowEditor() : Super()
{}

TSharedPtr<FBaseAssetToolkit> UDataflowEditor::CreateToolkit()
{
	TSharedPtr<FDataflowEditorToolkit> DataflowToolkit = MakeShared<FDataflowEditorToolkit>(this);
	return DataflowToolkit;
}


void UDataflowEditor::AddEditorSettings(const UDataflowEditorSettings* Settings)
{
	EditorSettings.Add(Settings);
}

void UDataflowEditor::Initialize(const TArray<TObjectPtr<UObject>>& InObjects, const TSubclassOf<AActor>& InPreviewClass)
{
	if(!InObjects.IsEmpty())
	{
		const TObjectPtr<UObject> ContentOwner = InObjects[0];
		TArray<TObjectPtr<UObject>> RequiredObjects = { ContentOwner };
		
		if(!EditorContent)
		{
			UDataflow* DataflowAsset = Cast<UDataflow>(ContentOwner);
			if(DataflowAsset)
			{
				EditorContent = UE::DataflowContextHelpers::CreateNewDataflowContent<UDataflowBaseContent>(ContentOwner);
				EditorContent->SetDataflowOwner(DataflowAsset);
				EditorContent->SetDataflowAsset(DataflowAsset);
			}
			else
			{
				if(IDataflowContentOwner* EditorContentOwner = Cast<IDataflowContentOwner>(ContentOwner))
				{
					EditorContent = EditorContentOwner->BuildDataflowContent();
					if (EditorContent)
					{
						if (EditorContent->IsSaved())
						{
							// Setup an asset that lives in the content broswer.
							FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
							FAssetRegistryModule::AssetCreated(EditorContent);
						}

						// Only register standalone Dataflow assets in OpenedAssets. Embedded sub-objects (no RF_Standalone, outer is not the package) 
						// must not be registered here: during package reload HandlePackageReloaded would add their repainted counterpart to PendingAssetsToOpen,
						// which triggers OpenAssets() → ShowModal() and blocks the gamethread while Tab A's deferred RequestCloseTab() is pending,
						// causing the editor to freeze and two tabs to remain visible simultaneously.
						DataflowAsset = EditorContent->GetDataflowAsset();
						if (DataflowAsset && DataflowAsset->IsAsset())
						{
							RequiredObjects.Add(DataflowAsset);
						}
					}
				}
			}
		}
		if(EditorContent && InPreviewClass)
		{
			EditorContent->SetPreviewClass(InPreviewClass);
		}
		RequiredObjects.Add(EditorContent);

		// Update the editor datas (skeleton information for the viewer)
		UpdateEditorContent();

		// Update and build the terminal contents
		UpdateTerminalContents(UE::Dataflow::FTimestamp::Invalid);
		
		// Potentially we could add additional objects to edit here (fields, meshes....)
		// If these objects have a matching factory we would be able to use geometry tools
		UBaseCharacterFXEditor::Initialize(RequiredObjects);
	}
}

void UDataflowEditor::UpdateEditorContent()
{
	if(EditorContent && EditorContent->GetDataflowAsset())
	{
		EditorContent->GetDataflowAsset()->Schema = UDataflowSchema::StaticClass();
	}
}

void UDataflowEditor::RemoveTerminalContents(const TSharedPtr<UE::Dataflow::FGraph>& DataflowGraph, ValidTerminalsType& ValidTerminals)
{
	for(int32 ContentIndex = TerminalContents.Num()-1; ContentIndex >= 0; --ContentIndex)
	{
		if(TSharedPtr<FDataflowNode> TerminalNode = DataflowGraph->FindFilteredNode(FDataflowTerminalNode::StaticType(),
			FName(TerminalContents[ContentIndex]->GetDataflowTerminal())))
		{
			if(Cast<IDataflowContentOwner>(TerminalNode->AsType<FDataflowTerminalNode>()->GetTerminalAsset()))
			{
				ValidTerminals.Add(TerminalNode, TerminalContents[ContentIndex]);
			}
			else
			{
				TerminalContents.RemoveAt(ContentIndex);
			}
		}
		else
		{
			// Removal of all the contents with invalid nodes
			TerminalContents.RemoveAt(ContentIndex);
		}
	}
}

void UDataflowEditor::AddTerminalContents(const TSharedPtr<UE::Dataflow::FGraph>& DataflowGraph, ValidTerminalsType& ValidTerminals)
{
	for(const TSharedPtr<FDataflowNode>& DataflowNode : DataflowGraph->GetFilteredNodes(FDataflowTerminalNode::StaticType()))
	{
		if(const FDataflowTerminalNode* TerminalNode = DataflowNode->AsType<FDataflowTerminalNode>())
		{
			if(IDataflowContentOwner* TerminalOwner = Cast<IDataflowContentOwner>(TerminalNode->GetTerminalAsset()))
			{
				const TObjectPtr<UDataflowBaseContent>* TerminalContent = ValidTerminals.Find(DataflowNode);
				if(!TerminalContent)
				{
					TerminalContents.Add(TerminalOwner->BuildDataflowContent());
					TerminalContent = &TerminalContents.Last();
					(*TerminalContent)->SetDataflowTerminal(TerminalNode->GetName().ToString());
					(*TerminalContent)->SetDataflowContext(EditorContent->GetDataflowContext());
					
					(*TerminalContent)->SetLastModifiedTimestamp(EditorContent->GetLastModifiedTimestamp());
				}
				if(TerminalNode->GetTerminalAsset() != (*TerminalContent)->GetTerminalAsset())
				{
					(*TerminalContent)->SetTerminalAsset(TerminalNode->GetTerminalAsset());
				}
			}
		}
	}
}

void UDataflowEditor::UpdateTerminalContents(const UE::Dataflow::FTimestamp TimeStamp)
{
	// update of the terminal contents only if no terminal asset on the main editor content
	if(EditorContent && !EditorContent->GetTerminalAsset() && EditorContent->GetDataflowAsset())
	{
		if(const TSharedPtr<UE::Dataflow::FGraph> DataflowGraph = EditorContent->GetDataflowAsset()->GetDataflow())
		{
			ValidTerminalsType ValidTerminals;

			// Remove invalid terminals
			RemoveTerminalContents(DataflowGraph, ValidTerminals);

			// Add valid terminals
			AddTerminalContents(DataflowGraph, ValidTerminals);
		}
	}
}

