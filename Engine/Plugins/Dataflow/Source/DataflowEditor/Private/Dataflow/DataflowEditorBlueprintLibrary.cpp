// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowEditorBlueprintLibrary.h"

#include "Dataflow/DataflowObjectInterface.h"
#include "Dataflow/DataflowAssetEditUtils.h"
#include "Dataflow/DataflowEdNode.h"

#include "HAL/PlatformApplicationMisc.h"
#include "Kismet2/BlueprintEditorUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowEditorBlueprintLibrary)

FName UDataflowEditorBlueprintLibrary::AddDataflowNode(UDataflow* Dataflow, FName NodeTypeName, FName BaseName, FVector2D Location)
{
	FName NewNodeName;
	if (Dataflow)
	{
		UDataflowEdNode* NewNode = UE::Dataflow::FEditAssetUtils::AddNewNode(Dataflow, Location, BaseName, NodeTypeName, /*FromPin*/ nullptr);
		if (NewNode)
		{
			NewNodeName = NewNode->GetFName();
		}
	}
	return NewNodeName;
}

bool UDataflowEditorBlueprintLibrary::ConnectDataflowNodes(UDataflow* Dataflow, FName FromNodeName, FName OutputName, FName ToNodeName, FName InputName)
{
	bool bSuccess = false;
	if (Dataflow)
	{
		if (TSharedPtr<UE::Dataflow::FGraph> Graph = Dataflow->GetDataflow())
		{
			TSharedPtr<FDataflowNode> FromNode = Graph->FindBaseNode(FromNodeName);
			TSharedPtr<FDataflowNode> ToNode = Graph->FindBaseNode(ToNodeName);
			if (FromNode && ToNode)
			{
				FDataflowOutput* Output = FromNode->FindOutput(OutputName);
				FDataflowInput* Input = ToNode->FindInput(InputName);
				if (Output && Input)
				{
					Dataflow->Modify();
					if (Graph->Connect(*Output, *Input))
					{
						Dataflow->RefreshEdNodeByGuid(FromNode->GetGuid());
						Dataflow->RefreshEdNodeByGuid(ToNode->GetGuid());
						bSuccess = true;
					}
				}
			}
		}
	}
	return bSuccess;
}

bool UDataflowEditorBlueprintLibrary::AddDataflowFromClipboardContent(UDataflow* Dataflow, const FString& ClipboardContent, FVector2D Location)
{
	bool bSuccess = false;
	if (Dataflow)
	{
		FPlatformApplicationMisc::ClipboardCopy(*ClipboardContent);
		TArray<UEdGraphNode*> PastedNodes;
		UE::Dataflow::FEditAssetUtils::PasteNodesFromClipboard(Dataflow, Location, PastedNodes);
		bSuccess = (PastedNodes.Num() > 0);
	}
	return bSuccess;
}

bool UDataflowEditorBlueprintLibrary::SetDataflowNodeProperty(UDataflow* Dataflow, FName NodeName, FName PropertyName, FString Propertyvalue)
{
	bool bSuccess = false;
	if (Dataflow)
	{
		if (TSharedPtr<UE::Dataflow::FGraph> Graph = Dataflow->GetDataflow())
		{
			if (TSharedPtr<FDataflowNode> Node = Graph->FindBaseNode(NodeName))
			{
				if (const FProperty* Property = Node->TypedScriptStruct()->FindPropertyByName(PropertyName))
				{
					TUniquePtr<FStructOnScope> StructOnScope(Node->NewStructOnScope());
					bSuccess = FBlueprintEditorUtils::PropertyValueFromString(Property, Propertyvalue, StructOnScope->GetStructMemory());
				}
			}
		}
	}
	return bSuccess;
}
			










