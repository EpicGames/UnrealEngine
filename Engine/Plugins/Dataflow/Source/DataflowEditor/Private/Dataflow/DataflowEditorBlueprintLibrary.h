// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "Engine/EngineTypes.h"
#include "Dataflow/DataflowObject.h"

#include "DataflowEditorBlueprintLibrary.generated.h"

UCLASS(MinimalAPI)
class UDataflowEditorBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/**
	* Add a specific node , return the node name
	*/
	UFUNCTION(BlueprintCallable, Category = "DataflowEditor", meta = (Keywords = "Dataflow Editor Graph Node"))
	static DATAFLOWEDITOR_API FName AddDataflowNode(UDataflow* Dataflow, FName NodeTypeName, FName BaseName, FVector2D Location);

	/**
	* Connect the output oif a node to the input of another 
	*/
	UFUNCTION(BlueprintCallable, Category = "DataflowEditor", meta = (Keywords = "Dataflow Editor Graph Node"))
	static DATAFLOWEDITOR_API bool ConnectDataflowNodes(UDataflow* Dataflow, FName FromNodeName, FName OutputName, FName ToNodeName, FName InputName);

	UFUNCTION(BlueprintCallable, Category = "DataflowEditor", meta = (Keywords = "Dataflow Editor Graph Node"))
	static DATAFLOWEDITOR_API bool AddDataflowFromClipboardContent(UDataflow* Dataflow, const FString& ClipboardContent, FVector2D Location);

	UFUNCTION(BlueprintCallable, Category = "DataflowEditor", meta = (Keywords = "Dataflow Editor Graph Node"))
	static DATAFLOWEDITOR_API bool SetDataflowNodeProperty(UDataflow* Dataflow, FName NodeName, FName PropertyName, FString Propertyvalue);


};
