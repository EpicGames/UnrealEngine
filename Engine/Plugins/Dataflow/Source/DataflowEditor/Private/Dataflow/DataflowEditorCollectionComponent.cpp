// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowEditorCollectionComponent.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowEditorCollectionComponent)

DEFINE_LOG_CATEGORY_STATIC(LogDataflowEditorCollectionComponentInternal, Log, All);

UDataflowEditorCollectionComponent::UDataflowEditorCollectionComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

