// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once


#include "Components/DynamicMeshComponent.h"
#include "UObject/ObjectMacros.h"

#include "DataflowEditorCollectionComponent.generated.h"

class UDataflowEdNode;
class UMeshWireframeComponent;
/**
*	FleshComponent
*/
UCLASS(MinimalAPI)
class UDataflowEditorCollectionComponent : public UDynamicMeshComponent
{
	GENERATED_UCLASS_BODY()
public:

	UPROPERTY()
	int32 MeshIndex = INDEX_NONE;

	UPROPERTY()
	TObjectPtr<const UDataflowEdNode> Node = nullptr;

	UPROPERTY()
	TObjectPtr<UMeshWireframeComponent> WireframeComponent = nullptr;


};
