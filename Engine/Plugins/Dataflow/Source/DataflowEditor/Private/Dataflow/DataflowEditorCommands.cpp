// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowEditorCommands.h"

#include "Dataflow/AssetDefinition_DataflowAsset.h"
#include "Dataflow/DataflowEdNode.h"
#include "Dataflow/DataflowContent.h"
#include "Dataflow/DataflowEditorStyle.h"
#include "Dataflow/DataflowEditorUtil.h"
#include "Dataflow/DataflowGraphEditor.h"
#include "Dataflow/DataflowNodeFactory.h"
#include "Dataflow/DataflowObject.h"
#include "Dataflow/DataflowOverrideNode.h"
#include "Dataflow/DataflowRenderingViewMode.h"
#include "Dataflow/DataflowSCommentNode.h"
#include "Dataflow/DataflowGraphSchemaAction.h"
#include "Dataflow/DataflowToolRegistry.h"
#include "Dataflow/DataflowAssetEditUtils.h"
#include "DataflowEditorTools/DataflowEditorWeightMapPaintTool.h"
#include "EdGraphNode_Comment.h"
#include "EdGraph/EdGraphNode.h"
#include "Editor.h"
#include "IDetailsView.h"
#include "IStructureDetailsView.h"
#include "IStructureDataProvider.h"
#include "Serialization/ObjectWriter.h"
#include "Serialization/ObjectReader.h"
#include "Dataflow/DataflowGraph.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"

#if WITH_EDITOR
#include "HAL/PlatformApplicationMisc.h"
#endif

#define LOCTEXT_NAMESPACE "DataflowEditorCommands"

const FString FDataflowEditorCommandsImpl::AddWeightMapNodeIdentifier = TEXT("AddWeightMapNode");

FDataflowEditorCommandsImpl::FDataflowEditorCommandsImpl()
	: TBaseCharacterFXEditorCommands<FDataflowEditorCommandsImpl>("DataflowEditor", 
		LOCTEXT("ContextDescription", "Dataflow Editor"), 
		NAME_None,
		FDataflowEditorStyle::Get().GetStyleSetName())
{
}

void FDataflowEditorCommandsImpl::RegisterCommands()
{
	TBaseCharacterFXEditorCommands::RegisterCommands();
	
	UI_COMMAND(EvaluateNode, "Evaluate", "Trigger an evaluation of the selected node.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(EvaluateGraph, "Evaluate Dataflow Graph", "Trigger an evaluation of the Dataflow graph terminal nodes.", EUserInterfaceActionType::Button, FInputChord(EKeys::F5));
	UI_COMMAND(ToggleSimulation, "Simulate Dataflow Scene", "Toggle the simulation of the registered components in the simulation scene.", EUserInterfaceActionType::RadioButton, FInputChord(EKeys::S, EModifierKey::Alt));
	UI_COMMAND(StartSimulation, "Start Dataflow Simulation", "Start the simulation of the asset", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(StopSimulation, "Stop Dataflow Simulation", "Stop the simulation of the asset.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(PauseSimulation, "Pause Dataflow Simulation", "Pause the simulation of the asset.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(StepSimulation, "Step Dataflow Simulation", "Advance the simulation of the asset by one frame.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ResetSimulation, "Reset Dataflow Simulation", "Reset the simulation of the asset.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(EvaluateGraphAutomatic, "Automatic Graph Evaluation", "Set the evaluation mode of the Dataflow graph to Automatic.", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(EvaluateGraphManual, "Manual Graph Evaluation", "Set the evaluation mode of the Dataflow graph to Manual.", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(ClearGraphCache, "Force Graph Re-evaluation", "Force the entire Dataflow graph to re-evaluate by deleting the evaluation cache and re-evaluating the terminal nodes.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(TogglePerfData, "Performance Data", "Toggle the evaluation performance data for each node.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ToggleAsyncEvaluation, "Asynchronous Evaluation (Experimental)", "Toggle asynchronous evaluation of the Dataflow graph (This is an experimental feature).", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(FreezeNodes, "FreezeNodes", "Freeze the evaluation of the selected nodes.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(UnfreezeNodes, "UnfreezeNodes", "Unfreeze the evaluation of the selected nodes.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(CreateComment, "CreateComment", "Create a Comment node.", EUserInterfaceActionType::None, FInputChord());
	UI_COMMAND(ToggleEnabledState, "ToggleEnabledState", "Toggle node between Enabled/Disabled state.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ToggleObjectSelection, "ToggleObjectSelection", "Enable object selection in editor.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ToggleFaceSelection, "ToggleFaceSelection", "Enable face selection in editor.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ToggleVertexSelection, "ToggleVertexSelection", "Enable vertex selection in editor.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(AddOptionPin, "AddOptionPin", "Add an option pin to the selected nodes.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(RemoveOptionPin, "RemoveOptionPin", "Remove the last option pin from the selected nodes.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ZoomToFitGraph, "ZoomToFitGraph", "Fit the Dataflow graph in the graph editor viewport.", EUserInterfaceActionType::None, FInputChord(EKeys::F));

	UI_COMMAND(AddWeightMapNode, "Add Weight Map", "Paint weight maps on the mesh", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(AddNewVariable, "Variable", "Adds a new variable to this Dataflow graph.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(AddNewSubGraph, "SubGraph", "Adds a new subgraph to this Dataflow graph.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ConvertToBasicSubGraph, "Convert to Basic Subgraph", "Convert the subgraph to a basic one (no loop).", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ConvertToForEachSubGraph, "Convert to For Each Subgraph", "Convert the subgraph to be able to iterate through an array.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(CollapseToSubGraph, "Collapse to Sub-graph", "Collapse the selection into a subgraph", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(ConvertToEmbeddedDataflow, "Embed Dataflow", "Embed the Dataflow graph in the current asset", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ConvertToExternalDataflow, "Extract and Reference Dataflow...", "Convert this Dataflow graph to be an external Dataflow asset that can be shared by other asset and assign it to this asset", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ToggleDataflowEditingLock, "Lock/Unlock Editing", "Toggle editing for the current Dataflow graph.", EUserInterfaceActionType::ToggleButton, FInputChord());

	for (const TPair<FName, TUniquePtr<UE::Dataflow::IDataflowConstructionViewMode>>& NameAndMode : UE::Dataflow::FRenderingViewModeFactory::GetInstance().GetViewModes())
	{
		TSharedPtr< FUICommandInfo > SetViewModeCommand;
		
		const UE::Dataflow::IDataflowConstructionViewMode* const ViewMode = NameAndMode.Value.Get();
		checkf(ViewMode, TEXT("Registered mode in FRenderingViewModeFactory has no associated IDataflowConstructionViewMode object. Registered name: %s"), *NameAndMode.Key.ToString());

		FUICommandInfo::MakeCommandInfo(
			this->AsShared(),
			SetViewModeCommand,
			ViewMode->GetName(),
			ViewMode->GetButtonText(),
			ViewMode->GetTooltipText(),
			FSlateIcon(),
			EUserInterfaceActionType::RadioButton,
			FInputChord()
		);
		SetConstructionViewModeCommands.Add(ViewMode->GetName(), SetViewModeCommand);
	}

	UE::Dataflow::FDataflowToolRegistry& ToolRegistry = UE::Dataflow::FDataflowToolRegistry::Get();
	const TArray<FName> NodeNames = ToolRegistry.GetNodeNames();
	for (const FName& NodeName : NodeNames)
	{
		FUICommandInfo::MakeCommandInfo(
			this->AsShared(),
			ToolRegistry.GetToolCommandForNode(NodeName),
			FName(NodeName.ToString() + "_Tool"),
			ToolRegistry.GetAddNodeButtonText(NodeName),					// TODO: Replace placeholder names
			FText::Format(LOCTEXT("LaunchDataflowToolTooltip", "Launch a \"{0}\" Dataflow tool"), FText::FromString(NodeName.ToString())),
			ToolRegistry.GetAddNodeButtonIcon(NodeName),
			EUserInterfaceActionType::RadioButton,
			FInputChord()
		);

		FUICommandInfo::MakeCommandInfo(
			this->AsShared(),
			ToolRegistry.GetAddNodeCommandForNode(NodeName),
			FName("Add_" + NodeName.ToString()),
			ToolRegistry.GetAddNodeButtonText(NodeName),
			FText::Format(LOCTEXT("AddDataflowNodeTooltip", "Add a \"{0}\" node to the Dataflow graph"), FText::FromString(NodeName.ToString())),
			ToolRegistry.GetAddNodeButtonIcon(NodeName),
			EUserInterfaceActionType::Button,
			FInputChord()
		);

	}

}


void FDataflowEditorCommandsImpl::UpdateToolCommandBinding(UInteractiveTool* Tool, TSharedPtr<FUICommandList> UICommandList, bool bUnbind)
{
	UE::Dataflow::FDataflowToolRegistry& ToolRegistry = UE::Dataflow::FDataflowToolRegistry::Get();
	if (bUnbind)
	{
		ToolRegistry.UnbindActiveCommands(UICommandList);
	}
	else
	{
		ToolRegistry.BindCommandsForCurrentTool(UICommandList, Tool);
	}
}

const FDataflowEditorCommandsImpl& FDataflowEditorCommands::Get()
{
	return FDataflowEditorCommandsImpl::Get();
}

void FDataflowEditorCommands::Register()
{
	return FDataflowEditorCommandsImpl::Register();
}

void FDataflowEditorCommands::Unregister()
{
	return FDataflowEditorCommandsImpl::Unregister();
}

bool FDataflowEditorCommands::IsRegistered()
{
	return FDataflowEditorCommandsImpl::IsRegistered();
}


const FDataflowNode* FDataflowEditorCommands::EvaluateNode(UE::Dataflow::FContext& Context, UE::Dataflow::FTimestamp& InOutLastNodeTimestamp,
	const UDataflow* Dataflow, const FDataflowNode* Node, const FDataflowOutput* Output, const FString& NodeName, UObject* Asset)
{
	UE_LOGF(LogChaosDataflow, VeryVerbose, "FDataflowEditorCommands::EvaluateNode(): Node [%ls], NodeName [%ls] Output [%ls]", Node ? *Node->GetName().ToString() : TEXT("nullptr"), *NodeName, Output ? *Output->GetName().ToString() : TEXT("nullptr"));

	if (!Node && Dataflow)
	{
		if (const TSharedPtr<const UE::Dataflow::FGraph> Graph = Dataflow->GetDataflow())
		{
			Node = Graph->FindBaseNode(FName(NodeName)).Get();
		}
	}
	if (Node && InOutLastNodeTimestamp < Node->GetTimestamp())
	{
		// Note: If the node is deactivated and has any outputs, then these outputs might still need to be forwarded.
		//       Therefore the Evaluate method has to be called for whichever value of bActive.
		//       This however isn't the case of SetAssetValue() for which the active state needs to be checked before the call.

		TWeakPtr<const FDataflowNode> WeakNode{ Node->AsWeak() };
		TWeakObjectPtr<UObject> WeakAsset{ Asset };
		auto OnPostEvaluation =
			[WeakNode, WeakAsset](UE::Dataflow::FContext& Context)
			{
				if (TSharedPtr<const FDataflowNode> Node = WeakNode.Pin())
				{
					if (const FDataflowTerminalNode* const TerminalNode = Node->AsType<const FDataflowTerminalNode>())
					{
						if (TerminalNode->IsActive())
						{
							TStrongObjectPtr<UObject> Asset = WeakAsset.Pin();
							UE_LOGF(LogChaosDataflow, Verbose, "FDataflowTerminalNode::SetAssetValue(): TerminalNode [%ls], Asset [%ls]", *TerminalNode->GetName().ToString(), Asset? *Asset->GetName(): TEXT("<none>"));
							TerminalNode->SetAssetValue(Asset.Get(), Context);
						}
					}
				}
			};

		Context.Evaluate(Node, Output, OnPostEvaluation);

		InOutLastNodeTimestamp = Node->GetTimestamp();
	}
	return Node;
}

void FDataflowEditorCommands::EvaluateNode(UE::Dataflow::FContext& Context, const FDataflowNode& Node, const FDataflowOutput* Output, UObject* Asset, UE::Dataflow::FTimestamp& InOutLastNodeTimestamp, UE::Dataflow::FOnPostEvaluationFunction OnEvaluationCompleted)
{
	UE_LOGF(LogChaosDataflow, VeryVerbose, "FDataflowEditorCommands::EvaluateNode() : Node [%ls], Output [%ls]", *Node.GetName().ToString(), Output ? *Output->GetName().ToString() : TEXT("nullptr"));

	if (InOutLastNodeTimestamp < Node.GetTimestamp())
	{
		// Note: If the node is deactivated and has any outputs, then these outputs might still need to be forwarded.
		//       Therefore the Evaluate method has to be called for whichever value of bActive.
		//       This however isn't the case of SetAssetValue() for which the active state needs to be checked before the call.

		TWeakPtr<const FDataflowNode> WeakNode{ Node.AsWeak() };
		TWeakObjectPtr<UObject> WeakAsset{ Asset };
		auto OnPostEvaluation =
			[WeakNode, WeakAsset, OnEvaluationCompleted](UE::Dataflow::FContext& Context)
			{
				if (TSharedPtr<const FDataflowNode> Node = WeakNode.Pin())
				{
					if (const FDataflowTerminalNode* const TerminalNode = Node->AsType<const FDataflowTerminalNode>())
					{
						if (TerminalNode->IsActive())
						{
							TStrongObjectPtr<UObject> Asset = WeakAsset.Pin();
							UE_LOGF(LogChaosDataflow, Verbose, "FDataflowTerminalNode::SetAssetValue(): TerminalNode [%ls], Asset [%ls]", *TerminalNode->GetName().ToString(), Asset ? *Asset->GetName() : TEXT("<none>"));
							TerminalNode->SetAssetValue(Asset.Get(), Context);
						}
					}
				}
				OnEvaluationCompleted(Context);
			};

		if (Output)
		{
			Context.Evaluate(&Node, Output, OnPostEvaluation);
		}
		else
		{
			Node.EvaluateNode(Context);
			OnPostEvaluation(Context);
		}

		InOutLastNodeTimestamp = Node.GetTimestamp();
	}
	else
	{
		OnEvaluationCompleted(Context);
	}
}

bool FDataflowEditorCommands::OnNodeVerifyTitleCommit(const FText& NewText, UEdGraphNode* GraphNode, FText& OutErrorMessage)
{
	const FString NewString = NewText.ToString();
	const int32 NewStringLen = NewString.Len();
	if (NewStringLen >= NAME_SIZE)
	{
		OutErrorMessage = FText::FromString(FString::Printf(TEXT("Name length is %d characters which exceeds the maximum allowed of %d characters"), NewStringLen, NAME_SIZE - 1));
		return false;
	}
	if (GraphNode)
	{
		// Comments are always valid because the text does not need to be unique
		if (GraphNode->IsA<UEdGraphNode_Comment>())
		{
			return true;
		}

		// Normal node let's make sure that the name is unique
		const FName NewNodeName(NewString);

		// check that the name is unique within the asset itself
		bool bIsUniqueSubObjectName = false;
		if (UDataflow* DataflowAsset = UDataflow::GetDataflowAssetFromEdGraph(GraphNode->GetGraph()))
		{
			bIsUniqueSubObjectName = UE::Dataflow::FEditAssetUtils::IsUniqueDataflowSubObjectName(DataflowAsset, NewNodeName);
		}

		// check if the node already exists ( technically bIsUniqueSubObjectName should be enough, but maybe a node has been renamed outside of the normal code paths )
		if (bIsUniqueSubObjectName)
		{
			if (UDataflowEdNode* DataflowNode = Cast<UDataflowEdNode>(GraphNode))
			{
				if (TSharedPtr<UE::Dataflow::FGraph> Graph = DataflowNode->GetDataflowGraph())
				{
					if (Graph->FindBaseNode(FName(NewString)).Get() == nullptr)
					{
						return true;
					}
				}
			}
		}
	}
	OutErrorMessage = FText::FromString(FString::Printf(TEXT("Non-unique name for graph node (%s)"), *NewString));
	return false;
}


void FDataflowEditorCommands::OnNodeTitleCommitted(const FText& InNewText, ETextCommit::Type InCommitType, UEdGraphNode* GraphNode)
{
	if (InCommitType == ETextCommit::OnCleared)
	{
		return;
	}

	if (GraphNode)
	{
		if (UDataflowEdNode* DataflowNode = Cast<UDataflowEdNode>(GraphNode))
		{
			if (TSharedPtr<UE::Dataflow::FGraph> Graph = DataflowNode->GetDataflowGraph())
			{
				if (TSharedPtr<FDataflowNode> Node = Graph->FindBaseNode(DataflowNode->GetDataflowNodeGuid()))
				{
					GraphNode->Rename(*InNewText.ToString());
					Node->SetName(FName(InNewText.ToString()));
				}
			}
		}
		else if (UEdGraphNode_Comment* CommentNode = Cast<UEdGraphNode_Comment>(GraphNode))
		{
			GraphNode->NodeComment = InNewText.ToString();
		}
	}
}

void FDataflowEditorCommands::OnNotifyPropertyPreChange(TSharedPtr<IStructureDetailsView> PropertiesEditor, UDataflow* Graph, class FEditPropertyChain* PropertyAboutToChange)
{
	// Find the associated UDataflowEdNode(s) and call Modify on them for Undo/Redo.
	if (PropertiesEditor && Graph)
	{
		Graph->Modify();

		if (TSharedPtr<const IStructureDataProvider> StructProvider = PropertiesEditor->GetStructureProvider())
		{
			const UStruct* const BaseStruct = StructProvider->GetBaseStructure();
			if (BaseStruct && BaseStruct->IsChildOf(FDataflowNode::StaticStruct()))
			{
				TArray<TSharedPtr<FStructOnScope>> StructData;
				StructProvider->GetInstances(StructData, FDataflowNode::StaticStruct());
				for (const TSharedPtr<FStructOnScope>& StructOnScope : StructData)
				{
					if (StructOnScope.IsValid() && StructOnScope->IsValid())
					{
						const FDataflowNode* const Node = reinterpret_cast<FDataflowNode*>(StructOnScope->GetStructMemory());
						check(Node);
						if (const TObjectPtr<UDataflowEdNode> EdNode = Graph->FindEdNodeByDataflowNodeGuid(Node->GetGuid()))
						{
							EdNode->Modify();
						}
					}
				}
			}
			else if (BaseStruct && BaseStruct->IsChildOf(UEdGraphNode_Comment::StaticClass()))
			{
				// Manually capture UObject state for undo/redo; FStructOnScope won't auto-Modify it.
				TArray<TSharedPtr<FStructOnScope>> StructData;
				StructProvider->GetInstances(StructData, UEdGraphNode_Comment::StaticClass());
				for (const TSharedPtr<FStructOnScope>& StructOnScope : StructData)
				{
					if (StructOnScope.IsValid() && StructOnScope->IsValid())
					{
						if (UEdGraphNode_Comment* const CommentNode = reinterpret_cast<UEdGraphNode_Comment*>(StructOnScope->GetStructMemory()))
						{
							CommentNode->Modify();
						}
					}
				}
			}
		}
	}
}


void FDataflowEditorCommands::OnAssetPropertyValueChanged(TObjectPtr<UDataflowBaseContent> Content, const FPropertyChangedEvent& InPropertyChangedEvent)
{
	if (Content)
	{
		const TObjectPtr<UDataflow>& DataflowAsset = Content->GetDataflowAsset();
		if (DataflowAsset)
		{
			if (InPropertyChangedEvent.ChangeType == EPropertyChangeType::ValueSet ||
				InPropertyChangedEvent.ChangeType == EPropertyChangeType::ArrayRemove ||
				InPropertyChangedEvent.ChangeType == EPropertyChangeType::ArrayClear)
			{
				if (InPropertyChangedEvent.GetPropertyName() == FName("Overrides_Key") ||
					InPropertyChangedEvent.GetPropertyName() == FName("Overrides"))
				{
					if (ensureMsgf(DataflowAsset != nullptr, TEXT("Warning : Failed to find valid graph.")))
					{
						for (const TSharedPtr<FDataflowNode>& DataflowNode : DataflowAsset->Dataflow->GetNodes())
						{
							if (DataflowNode->IsA(FDataflowOverrideNode::StaticType()))
							{
								// TODO: For now we invalidate all the FDataflowOverrideNode nodes
								// Once the Variable system will be in place only the neccessary nodes
								// will be invalidated
								DataflowNode->Invalidate();
							}
						}
					}
				}
			}
		}
	}
}

void FDataflowEditorCommands::OnPropertyValueChanged(UDataflow* OutDataflow, TSharedPtr<UE::Dataflow::FEngineContext>& Context, UE::Dataflow::FTimestamp& OutLastNodeTimestamp, const FPropertyChangedEvent& InPropertyChangedEvent, const TSet<TObjectPtr<UDataflowEdNode>>& SelectedNodes)
{
	if (InPropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive)
	{
		if (ensure(OutDataflow && InPropertyChangedEvent.Property && InPropertyChangedEvent.Property->GetOwnerUObject()))
		{
			for (UDataflowEdNode* const SelectedNode : SelectedNodes)
			{
				if (TSharedPtr<FDataflowNode> DataflowNode = SelectedNode->GetDataflowNode())
				{
					const FName PropertyName = InPropertyChangedEvent.GetPropertyName();

					// Active state update
					if (PropertyName == FDataflowNode::GetActivePropertyName())
					{
						// Reflect the active state on the drawing of the node
						constexpr bool bCheckIsActiveFlagOnly = true;
						if (DataflowNode->IsActive(bCheckIsActiveFlagOnly) != SelectedNode->IsNodeEnabled())
						{
							SelectedNode->SetEnabledState(DataflowNode->IsActive(bCheckIsActiveFlagOnly) ? ENodeEnabledState::Enabled : ENodeEnabledState::Disabled);
						}
					}
					else
					{
						// make sure the editable values are up-to-date with the changes
						SelectedNode->UpdatePinsDefaultValuesFromNode();
					}

					if (DataflowNode->ShouldInvalidateOnPropertyChanged(InPropertyChangedEvent))
					{
						// Invalidate the node and reset the editor timestamp
						DataflowNode->Invalidate();
						OutLastNodeTimestamp = UE::Dataflow::FTimestamp::Invalid;
					}
					if (Context.IsValid())
					{
						DataflowNode->OnPropertyChanged(*Context, InPropertyChangedEvent);
					}
				}
			}
		}
	}
}

void FDataflowEditorCommands::OnCommentPropertyValueChanged(const FPropertyChangedEvent& InPropertyChangedEvent, const TSet<TObjectPtr<UEdGraphNode_Comment>>& SelectedComments)
{
	if (InPropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive)
	{
		for (UEdGraphNode_Comment* const SelectedComment : SelectedComments)
		{
			if (SelectedComment)
			{
				FPropertyChangedEvent NonConstEvent = InPropertyChangedEvent;
				SelectedComment->PostEditChangeProperty(NonConstEvent);
			}
		}
	}
}

void FDataflowEditorCommands::FreezeNodes(UE::Dataflow::FContext& Context, const FGraphPanelSelectionSet& SelectedNodes)
{
	for (UObject* const SelectedNode : SelectedNodes)
	{
		if (UDataflowEdNode* const Node = Cast<UDataflowEdNode>(SelectedNode))
		{
			if (const TSharedPtr<FDataflowNode> DataflowNode = Node->GetDataflowNode())
			{
				if (!DataflowNode->IsFrozen())
				{
					DataflowNode->Freeze(Context);
				}
			}
		}
	}
}

void FDataflowEditorCommands::UnfreezeNodes(UE::Dataflow::FContext& Context, const FGraphPanelSelectionSet& SelectedNodes)
{
	for (UObject* const SelectedNode : SelectedNodes)
	{
		if (UDataflowEdNode* const Node = Cast<UDataflowEdNode>(SelectedNode))
		{
			if (const TSharedPtr<FDataflowNode> DataflowNode = Node->GetDataflowNode())
			{
				if (DataflowNode->IsFrozen())
				{
					DataflowNode->Unfreeze(Context);
				}
			}
		}
	}
}

void FDataflowEditorCommands::DeleteNodes(UEdGraph* EdGraph, const FGraphPanelSelectionSet& SelectedNodes)
{
	TArray<UEdGraphNode*> NodesToDelete;
	NodesToDelete.Reserve(SelectedNodes.Num());
	for (UObject* Node : SelectedNodes)
	{
		if (UEdGraphNode* EdNode = Cast<UEdGraphNode>(Node))
		{
			NodesToDelete.Add(EdNode);
		}
	}
	UE::Dataflow::FEditAssetUtils::DeleteNodes(EdGraph, NodesToDelete);
}

void FDataflowEditorCommands::RenameNode(const TSharedPtr<SDataflowGraphEditor>& DataflowGraphEditor, UEdGraphNode* EdNode)
{
	if (EdNode && DataflowGraphEditor.IsValid())
	{
		// GMelich: There is no direct rename function, this function can rename a node without recentering the selected node
		DataflowGraphEditor->IsNodeTitleVisible(EdNode, /*bRequestRename*/true);
	}
}

void FDataflowEditorCommands::OnSelectedNodesChanged(TSharedPtr<IStructureDetailsView> PropertiesEditor, UObject* Asset, UDataflow* Graph, const TSet<TObjectPtr<UObject> >& NewSelection)
{
	PropertiesEditor->SetStructureData(nullptr);

	if (Graph && PropertiesEditor)
	{
		if (const TSharedPtr<UE::Dataflow::FGraph> DataflowGraph = Graph->GetDataflow())
		{
			auto AsRawPointers = [](const TSet<TObjectPtr<UObject> >& NewSelection) {
				TSet<UObject*> Raw; for (UObject* Elem : NewSelection) Raw.Add(Elem);
				return Raw;
			};
			FGraphPanelSelectionSet SelectedNodes = AsRawPointers(NewSelection);
			if (SelectedNodes.Num())
			{
				TArray<TSharedPtr<FStructOnScope>> StructData;
				StructData.Reserve(SelectedNodes.Num());
				for (UObject* SelectedObject : SelectedNodes)
				{
					if (UDataflowEdNode* EdNode = Cast<UDataflowEdNode>(SelectedObject))
					{
						if (TSharedPtr<FDataflowNode> DataflowNode = DataflowGraph->FindBaseNode(EdNode->GetDataflowNodeGuid()))
						{
							FIsPropertyReadOnly IsPropertyReadOnlyLambda = FIsPropertyReadOnly::CreateLambda(
								[WeakNode = DataflowNode.ToWeakPtr()](const FPropertyAndParent& PropertyAndParent)
								{
									if (TSharedPtr<FDataflowNode> DataflowNode = WeakNode.Pin())
									{
										if (DataflowNode->MakeConnectedPropertiesReadOnly())
										{
											if (const FDataflowInput* Input = DataflowNode->FindInput(PropertyAndParent.Property.GetFName()))
											{
												if (Input->IsConnected())
												{
													return true;
												}
											}
											// let's check the parents so that we can disable cghildren properties of structures ( Vectors for example )
											for (const FProperty* ParentProperty : PropertyAndParent.ParentProperties)
											{
												if (ParentProperty)
												{
													if (const FDataflowInput* Input = DataflowNode->FindInput(ParentProperty->GetFName()))
													{
														if (Input->IsConnected())
														{
															return true;
														}
													}
												}
											}
										}
									}
									return false;
								});
							PropertiesEditor->GetDetailsView()->SetIsPropertyReadOnlyDelegate(IsPropertyReadOnlyLambda);

							StructData.Emplace(DataflowNode->NewStructOnScope());
						}
					}
					else if (UEdGraphNode_Comment* CommentNode = Cast<UEdGraphNode_Comment>(SelectedObject))
					{
						StructData.Emplace(new FStructOnScope(UEdGraphNode_Comment::StaticClass(), (uint8*)CommentNode));
					}
				}
				if (StructData.Num())
				{
					for (TSharedPtr<FStructOnScope>& StructOnScope : StructData)
					{
						StructOnScope->SetPackage(Graph->GetPackage());
					}
					PropertiesEditor->SetStructureProvider(MakeShared<FStructOnScopeStructureDataProvider>(StructData));
				}
			}
		}
	}
}

void FDataflowEditorCommands::ToggleEnabledState(UDataflow* Graph)
{
}

static UEdGraphPin* GetPin(const UDataflowEdNode* Node, const EEdGraphPinDirection Direction, const FName Name)
{
	for (UEdGraphPin* Pin : Node->GetAllPins())
	{
		if (Pin->PinName == Name && Pin->Direction == Direction)
		{
			return Pin;
		}
	}

	return nullptr;
}

static void ShowNotificationMessage(const FText& Message, const SNotificationItem::ECompletionState CompletionState)
{
	FNotificationInfo Info(Message);
	Info.ExpireDuration = 5.0f;
	Info.bUseLargeFont = false;
	Info.bUseThrobber = false;
	Info.bUseSuccessFailIcons = false;
	TSharedPtr<SNotificationItem> Notification = FSlateNotificationManager::Get().AddNotification(Info);
	if (Notification.IsValid())
	{
		Notification->SetCompletionState(CompletionState);
	}
}

void FDataflowEditorCommands::DuplicateNodes(UEdGraph* EdGraph, const TSharedPtr<SDataflowGraphEditor>& DataflowGraphEditor, const FGraphPanelSelectionSet& SelectedNodes)
{
	if (ensureMsgf(EdGraph != nullptr, TEXT("Warning : Failed to find valid graph.")))
	{
		TArray<UEdGraphNode*> SelectedEdNodes;
		for (UObject* SelectedNode : SelectedNodes)
		{
			if (UEdGraphNode* EdNode = Cast<UEdGraphNode>(SelectedNode))
			{
				SelectedEdNodes.Add(EdNode);
			}
		}
		TArray<UEdGraphNode*> DuplicatedEdNodes;
		const FVector2f PasteLocation = DataflowGraphEditor->GetPasteLocation2f();
		UE::Dataflow::FEditAssetUtils::DuplicateNodes(EdGraph, SelectedEdNodes, FDeprecateSlateVector2D(PasteLocation), DuplicatedEdNodes);

		// Update the selection in the Editor
		if (DuplicatedEdNodes.Num())
		{
			DataflowGraphEditor->ClearSelectionSet();
			for (UEdGraphNode* Node : DuplicatedEdNodes)
			{
				DataflowGraphEditor->SetNodeSelection(Node, true);
			}

			// Display message stating that nodes were duplicated
			FText MessageFormat;
			if (DuplicatedEdNodes.Num() == 1)
			{
				MessageFormat = LOCTEXT("DataflowDuplicatedNodesSingleNode", "{0} node/comment was duplicated");
			}
			else
			{
				MessageFormat = LOCTEXT("DataflowDuplicatedNodesMultipleNodes", "{0} nodes/comments were duplicated");
			}
			ShowNotificationMessage(FText::Format(MessageFormat, DuplicatedEdNodes.Num()), SNotificationItem::CS_Success);
		}
	}
}

void FDataflowEditorCommands::CopyNodes(UEdGraph* EdGraph, const TSharedPtr<SDataflowGraphEditor>& DataflowGraphEditor, const FGraphPanelSelectionSet& InSelectedNodes)
{
	if (ensureMsgf(EdGraph != nullptr, TEXT("Warning : Failed to find valid graph.")))
	{
		if (InSelectedNodes.Num() > 0)
		{
			TArray<const UEdGraphNode*> SelectedEdNodes;
			for (UObject* SelectedNode : InSelectedNodes)
			{
				if (const UEdGraphNode* EdNode = Cast<const UEdGraphNode>(SelectedNode))
				{
					SelectedEdNodes.Add(EdNode);
				}
			}

			int32 NumCopiedNodes = 0;
			UE::Dataflow::FEditAssetUtils::CopyNodesToClipboard(SelectedEdNodes, NumCopiedNodes);

			// Display message stating that nodes were copied to clipboard
			if (NumCopiedNodes > 0)
			{
				FText MessageFormat;
				if (NumCopiedNodes == 1)
				{
					MessageFormat = LOCTEXT("DataflowCopiedNodesToClipboardSingleNode", "{0} node/comment was copied to clipboard");
				}
				else
				{
					MessageFormat = LOCTEXT("DataflowCopiedNodesToClipboardMultipleNodes", "{0} nodes/comments were copied to clipboard");
				}
				ShowNotificationMessage(FText::Format(MessageFormat, NumCopiedNodes), SNotificationItem::CS_Success);
			}
		}
	}
}

void FDataflowEditorCommands::PasteNodes(UEdGraph* EdGraph, const TSharedPtr<SDataflowGraphEditor>& DataflowGraphEditor)
{
	TArray<UEdGraphNode*> PastedEdNodes;
	const FVector2f PasteLocation = DataflowGraphEditor->GetPasteLocation2f();
	UE::Dataflow::FEditAssetUtils::PasteNodesFromClipboard(EdGraph, FDeprecateSlateVector2D(PasteLocation), PastedEdNodes);

	// select the pasted nodes
	if (PastedEdNodes.Num())
	{
		DataflowGraphEditor->ClearSelectionSet();
		for (UEdGraphNode* Node : PastedEdNodes)
		{
			DataflowGraphEditor->SetNodeSelection(Node, true);
		}
	
		// Display message stating that nodes were pasted from clipboard
		FText MessageFormat;
		if (PastedEdNodes.Num() == 1)
		{
			MessageFormat = LOCTEXT("DataflowPastedNodesFromClipboardSingleNode", "{0} node/comment was pasted from clipboard");
		}
		else
		{
			MessageFormat = LOCTEXT("DataflowPastedNodesFromClipboardMultipleNodes", "{0} nodes/comments were pasted from clipboard");
		}
		ShowNotificationMessage(FText::Format(MessageFormat, PastedEdNodes.Num()), SNotificationItem::CS_Success);
	}
}

#undef LOCTEXT_NAMESPACE
