// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowEditorMode.h"

#include "AssetEditorModeManager.h"
#include "AttributeEditorTool.h"
#include "BaseGizmos/TransformGizmoUtil.h"
#include "EditorGizmos/EditorTransformGizmoUtil.h"
#include "EditorInteractiveGizmoSubsystem.h"
#include "SkeletalMeshEditorUtils.h"
#include "SkeletalMeshGizmoUtils.h"
#include "Components/DynamicMeshComponent.h"
#include "ContextObjectStore.h"
#include "DataflowEditorTools/DataflowEditorWeightMapPaintTool.h"
#include "DataflowEditorTools/DataflowEditorVertexAttributePaintTool.h"
#include "Dataflow/DataflowComponentToolTarget.h"
#include "Dataflow/DataflowCollectionAddScalarVertexPropertyNode.h"
#include "Dataflow/DataflowConstructionViewportClient.h"
#include "Dataflow/DataflowEditor.h"
#include "Dataflow/DataflowContent.h"
#include "Dataflow/DataflowEditorCommands.h"
#include "Dataflow/DataflowEditorModeToolkit.h"
#include "Dataflow/DataflowEditorUtil.h"
#include "Dataflow/DataflowGraphEditor.h"
#include "Dataflow/DataflowEditorPreviewSceneBase.h"
#include "Dataflow/DataflowConstructionScene.h"
#include "Dataflow/DataflowRenderingViewMode.h"
#include "Dataflow/DataflowSimulationScene.h"
#include "Dataflow/DataflowSimulationViewportClient.h"
#include "Dataflow/DataflowGraphSchemaAction.h"
#include "Dataflow/DataflowToolTarget.h"
#include "Dataflow/DataflowToolRegistry.h"
#include "EditorInteractiveGizmoManager.h"
#include "EditorModeManager.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "Elements/Framework/EngineElementsLibrary.h"
#include "EngineAnalytics.h"
#include "MeshSelectionTool.h"
#include "MeshVertexPaintTool.h"
#include "MeshAttributePaintTool.h"
#include "ModelingToolTargetUtil.h"
#include "TargetInterfaces/MaterialProvider.h"
#include "TargetInterfaces/DynamicMeshCommitter.h"
#include "TargetInterfaces/DynamicMeshProvider.h"
#include "ToolTargetManager.h"
#include "ToolTargets/DynamicMeshComponentToolTarget.h"
#include "ToolTargets/StaticMeshComponentToolTarget.h"
#include "ToolTargets/StaticMeshToolTarget.h"
#include "ToolTargets/SkeletalMeshComponentToolTarget.h"
#include "ToolTargets/SkeletalMeshToolTarget.h"
#include "Tools/UEdMode.h"
#include "Selection.h"
#include "UnrealClient.h"
#include "Framework/Application/SlateApplication.h"
#include "SEditorViewport.h"
#include "Slate/SceneViewport.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowEditorMode)

#define LOCTEXT_NAMESPACE "UDataflowEditorMode"

const FEditorModeID UDataflowEditorMode::EM_DataflowEditorModeId = TEXT("EM_DataflowAssetEditorMode");

namespace UE::Dataflow::Private
{
	bool bDataflowEditorEnableToolsInPIE = true;
	FAutoConsoleVariableRef CVARDataflowEditorEnableToolsInPIE(TEXT("p.Dataflow.EnableToolsInPIE"), bDataflowEditorEnableToolsInPIE,
		TEXT("Enable Dataflow Editor tools while Play In Editor is running [def:true]"));
}

namespace UE::Dataflow
{
	void RegisterModeForToolManager(FToolBuilderModeMap& InOutMap, UDataflowEditorMode* InMode)
	{
		// Drop entries whose tool manager or mode have been GC'd to keep the map bounded
		// across editor open/close cycles.
		for (auto It = InOutMap.CreateIterator(); It; ++It)
		{
			if (!It.Key().IsValid() || !It.Value().IsValid())
			{
				It.RemoveCurrent();
			}
		}

		if (InMode)
		{
			if (const UEditorInteractiveToolsContext* ToolsContext = InMode->GetInteractiveToolsContext())
			{
				if (UInteractiveToolManager* ToolManager = ToolsContext->ToolManager)
				{
					InOutMap.Add(ToolManager, InMode);
				}
			}
		}
	}

	UDataflowEditorMode* FindModeForToolManager(const FToolBuilderModeMap& InMap, const UInteractiveToolManager* ToolManager)
	{
		if (const TWeakObjectPtr<UDataflowEditorMode>* Found = InMap.Find(ToolManager))
		{
			return Found->Get();
		}
		return nullptr;
	}
}

UDataflowEditorMode::UDataflowEditorMode()
{
	Info = FEditorModeInfo(
		EM_DataflowEditorModeId,
		LOCTEXT("DataflowEditorModeName", "Dataflow Toolbox"),
		FSlateIcon(),
		false);
}

const FToolTargetTypeRequirements& UDataflowEditorMode::GetToolTargetRequirements()
{
	static const FToolTargetTypeRequirements ToolTargetRequirements =
		FToolTargetTypeRequirements({
			UMaterialProvider::StaticClass(),
			UDynamicMeshCommitter::StaticClass(),
			UDynamicMeshProvider::StaticClass()
			});

	return ToolTargetRequirements;
}

void UDataflowEditorMode::Enter()
{
	UBaseCharacterFXEditorMode::Enter();

	FViewport::ViewportResizedEvent.AddUObject(this, &UDataflowEditorMode::ViewportResized);

	// Register gizmo ContextObject for use inside interactive tools
	UE::TransformGizmoUtil::RegisterTransformGizmoContextObject(GetInteractiveToolsContext());
	if (UInteractiveToolsContext* ToolsContext = GetInteractiveToolsContext())
	{
		// Register the editor TRS gizmo builder on the EdMode-level gizmo manager so DataflowPropertyGizmoManager's
		// CreateTransformGizmo (which uses this gizmo manager) can find the builder. We deliberately don't create a
		// local UEditorTransformGizmoContextObject — the parent (mode-manager-level) one is already fully Initialize()d
		// by FEditorModeTools' constructor, and the gizmo builder finds it via FindContext walking up.
		if (UEditorInteractiveGizmoManager* GizmoManager = Cast<UEditorInteractiveGizmoManager>(ToolsContext->ToolManager->GetPairedGizmoManager()))
		{
			if (UEditorInteractiveGizmoSubsystem* GizmoSubsystem = GEditor->GetEditorSubsystem<UEditorInteractiveGizmoSubsystem>())
			{
				GizmoManager->RegisterGizmoType(UEditorInteractiveGizmoManager::TransformBuilderIdentifier(), GizmoSubsystem->GetTransformGizmoBuilder());
			}
		}
	}
	UE::SkeletalMeshGizmoUtils::RegisterTransformGizmoContextObject(GetInteractiveToolsContext());
	UE::SkeletalMeshEditorUtils::RegisterEditorContextObject(GetInteractiveToolsContext());

	// Initialize view mode to a default
	ConstructionViewMode = UE::Dataflow::FRenderingViewModeFactory::GetInstance().GetViewMode(UE::Dataflow::FDataflowConstruction3DViewMode::Name);

	// Log mode starting
	if (FEngineAnalytics::IsAvailable())
	{
		LastModeStartTimestamp = FDateTime::UtcNow();
		TArray<FAnalyticsEventAttribute> EventAttributes;
		EventAttributes.Add(FAnalyticsEventAttribute(TEXT("Timestamp"), LastModeStartTimestamp.ToString()));
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.DataflowEditor.Enter"), EventAttributes);
	}

	// Reset tracking of whether we've seen a valid mesh in 2D or 3D viewports
	FirstTimeRefocusedViews.Reset();
}

void UDataflowEditorMode::SetDataflowEditor(UDataflowEditor* InDataflowEditor) 
{ 
	DataflowEditor = InDataflowEditor; 
}

void UDataflowEditorMode::AddToolTargetFactories()
{
	GetInteractiveToolsContext()->TargetManager->AddTargetFactory(NewObject<UDynamicMeshComponentToolTargetFactory>(GetToolManager()));
	GetInteractiveToolsContext()->TargetManager->AddTargetFactory(NewObject<UStaticMeshComponentToolTargetFactory>(GetToolManager()));
	GetInteractiveToolsContext()->TargetManager->AddTargetFactory(NewObject<UStaticMeshToolTargetFactory>(GetToolManager()));
	GetInteractiveToolsContext()->TargetManager->AddTargetFactory(NewObject<USkeletalMeshComponentToolTargetFactory>(GetToolManager()));
	GetInteractiveToolsContext()->TargetManager->AddTargetFactory(NewObject<USkeletalMeshToolTargetFactory>(GetToolManager()));
	GetInteractiveToolsContext()->TargetManager->AddTargetFactory(NewObject<UDataflowComponentToolTargetFactory>(GetToolManager()));
	GetInteractiveToolsContext()->TargetManager->AddTargetFactory(NewObject<UDataflowToolTargetFactory>(GetToolManager()));
}

void UDataflowEditorMode::RegisterDataflowTool(TSharedPtr<FUICommandInfo> UICommand,
	FString ToolIdentifier,
	UInteractiveToolBuilder* Builder,
	UEditorInteractiveToolsContext* const ToolsContext,
	EToolsContextScope ToolScope)
{
	if (!Toolkit.IsValid())
	{
		return;
	}

	if (!ToolsContext)
	{
		return;
	}

	if (ToolScope == EToolsContextScope::Default)
	{
		ToolScope = GetDefaultToolScope();
	}
	ensure(ToolScope != EToolsContextScope::Editor);

	ToolsContext->ToolManager->RegisterToolType(ToolIdentifier, Builder);

	const TSharedRef<FUICommandList>& CommandList = Toolkit->GetToolkitCommands();

	CommandList->MapAction(UICommand, 
		FExecuteAction::CreateWeakLambda(ToolsContext, [this, ToolsContext, ToolIdentifier, Builder]()
		{
			UDataflowContextObject* ContextObject = ToolsContext->ContextObjectStore->FindContext<UDataflowContextObject>();
			check(ContextObject);

			if (const IDataflowEditorToolBuilder* const DataflowToolBuilder = Cast<IDataflowEditorToolBuilder>(Builder))
			{
				// Check if we need to switch view modes before starting the tool
				TArray<const UE::Dataflow::IDataflowConstructionViewMode*> SupportedModes;

				DataflowToolBuilder->GetSupportedConstructionViewModes(*ContextObject, SupportedModes);

				if (SupportedModes.Num() > 0 && !SupportedModes.Contains(GetConstructionViewMode()))
				{
					if (!bShouldRestoreSavedConstructionViewMode)
					{
						// remember the current view mode so we can restore it later
						SavedConstructionViewMode = GetConstructionViewMode()->GetName();
						bShouldRestoreSavedConstructionViewMode = true;
					}

					const TSharedPtr<FDataflowConstructionScene> ConstructionScenePtr = ConstructionScene.Pin();

					bool bHadSingleSelectedMesh = false;
					if (const USelection* const SelectedComponents = ConstructionScenePtr ? ConstructionScenePtr->GetSelectedComponents() : nullptr)
					{
						bHadSingleSelectedMesh = (SelectedComponents->Num() == 1);
					}

					// switch to the preferred view mode for the tool that's about to start
					SetConstructionViewMode(SupportedModes[0]->GetName());

					if (bHadSingleSelectedMesh)
					{
						// If there is a single dynamic mesh component in the scene, select it so the tool can start
						const TArray<TObjectPtr<UDynamicMeshComponent>> DynamicMeshComponents = ConstructionScenePtr ? ConstructionScenePtr->GetDynamicMeshComponents() : TArray<TObjectPtr<UDynamicMeshComponent>>();
						if (DynamicMeshComponents.Num() == 1)
						{
							if (USelection* const SelectedComponents = ConstructionScenePtr->GetSelectedComponents())
							{
								SelectedComponents->Select(DynamicMeshComponents[0]);
							}
						}
					}
				}
			}

			// Make sure the ContextObject's selected Collection is the from the Input side of the selected node (so that the tool gets the Collection as it appears before node execution)

			if (TSharedPtr<UE::Dataflow::FEngineContext> DataflowContext = ContextObject->GetDataflowContext())
			{
				if (UDataflowEdNode* const SelectedNode = ContextObject->GetSelectedNode())
				{
					if (const TSharedPtr<FDataflowNode> DataflowNode = SelectedNode->GetDataflowNode())
					{
						for (const FDataflowInput* const Input : DataflowNode->GetInputs())
						{
							if (Input->GetType() == FName("FManagedArrayCollection"))
							{
								const FManagedArrayCollection DefaultValue;
								TSharedRef<FManagedArrayCollection> Collection = MakeShared<FManagedArrayCollection>(Input->GetValue<FManagedArrayCollection>(*DataflowContext, DefaultValue));

								constexpr bool bCollectionIsInput = true;
								ContextObject->SetSelectedCollection(Collection, bCollectionIsInput);

								// If we have multiple input Collections, this will just take the first one. This is what the Cloth Editor does as well (but there it also checks to see if it's a ClothCollection)
								break;
							}
						}
					}
				}
			}

			ActiveToolsContext = ToolsContext;
			ToolsContext->StartTool(ToolIdentifier);
		}),
		FCanExecuteAction::CreateWeakLambda(ToolsContext, [this, ToolIdentifier, ToolsContext]()
		{
			return ShouldToolStartBeAllowed(ToolIdentifier) &&
			ToolsContext->ToolManager->CanActivateTool(EToolSide::Mouse, ToolIdentifier);
		}),
		FIsActionChecked::CreateUObject(ToolsContext, &UEdModeInteractiveToolsContext::IsToolActive, EToolSide::Mouse, ToolIdentifier),
		EUIActionRepeatMode::RepeatDisabled
	);
}

bool UDataflowEditorMode::CanUseTools() const
{
	return IsDataflowEditable();
}

TSharedPtr<const FUICommandInfo> UDataflowEditorMode::GetSelectedNodeToolCommand() const
{
	if (CanUseTools())
	{
		const FName NodeTypeName = GetSelectedNodeTypeName();
		if (!NodeTypeName.IsNone())
		{
			const TSharedRef<FUICommandList> CommandList = Toolkit->GetToolkitCommands();
			const FDataflowEditorCommandsImpl& CommandInfos = FDataflowEditorCommandsImpl::Get();
			return NodeTypeToToolCommandMap.FindRef(NodeTypeName, {});
		}
	}
	return {};
}

bool UDataflowEditorMode::CanSelectedNodeLaunchTool() const
{
	if (CanUseTools())
	{
		const bool bIsToolActive = GetToolManager()->HasActiveTool(EToolSide::Left);
		if (!bIsToolActive && !bPendingNodeSelectionChanged)
		{
			return (UDataflowEditorMode::GetSelectedNodeToolCommand() != nullptr);
		}
	}
	return false;
}

bool UDataflowEditorMode::IsSelectedNodeUpToDate(UE::Dataflow::FContext& Context) const
{
	if (TObjectPtr<UDataflowEdNode> SelectedNode = GetSelectedNode())
	{
		if (TSharedPtr<FDataflowNode> DataflowNode = SelectedNode->GetDataflowNode())
		{
			TArray<FDataflowOutput*> Outputs = DataflowNode->GetOutputs();
			for (const FDataflowOutput* Output : Outputs)
			{
				if (Output && Context.HasData(Output->CacheKey(), DataflowNode->GetTimestamp()))
				{
					return true;
				}
			}
		}
	}
	return false;
}

bool UDataflowEditorMode::IsDataflowEditable() const
{
	if (TSharedPtr<SDataflowGraphEditor> GraphEditor = DataflowGraphEditor.Pin())
	{
		return GraphEditor->IsEditable();
	}
	return false;
}

void UDataflowEditorMode::AddNode(FName NewNodeType)
{
	const UE::Dataflow::FDataflowToolRegistry& ToolRegistry = UE::Dataflow::FDataflowToolRegistry::Get();

	const FName ConnectionType = ToolRegistry.HasToolInfoForNodeType(NewNodeType) ? ToolRegistry.GetAddNodeConnectionType(NewNodeType) : FManagedArrayCollection::StaticType();
	const FName ConnectionName = ToolRegistry.HasToolInfoForNodeType(NewNodeType) ? ToolRegistry.GetAddNodeConnectionName(NewNodeType) : FName("Collection");

	UEdGraphNode* const CurrentlySelectedNode = GetSingleSelectedNodeWithOutputType(ConnectionType);
	checkf(CurrentlySelectedNode, TEXT("No node with FManagedArrayCollection output is currently selected in the Dataflow graph"));

	const UEdGraphNode* const NewNode = CreateAndConnectNewNode(NewNodeType, *CurrentlySelectedNode, ConnectionType, ConnectionName);
	verifyf(NewNode, TEXT("Failed to create a new node: %s"), *NewNodeType.ToString());

	// Wait for FDataflowEditorToolkit::OnNodeSelectionChanged to execute before actually starting the tool
	bPendingNodeSelectionChanged = true;

	// This will queue the tool to start after FDataflowEditorToolkit::OnNodeSelectionChanged finishes
	StartToolForSelectedNode(NewNode);
}

bool UDataflowEditorMode::CanAddNode(FName NewNodeType) const
{
	if (IsDataflowEditable())
	{
		const UE::Dataflow::FDataflowToolRegistry& ToolRegistry = UE::Dataflow::FDataflowToolRegistry::Get();
		const FName ConnectionType = ToolRegistry.HasToolInfoForNodeType(NewNodeType) ? ToolRegistry.GetAddNodeConnectionType(NewNodeType) : FManagedArrayCollection::StaticType();

		const UEdGraphNode* const CurrentlySelectedNode = GetSingleSelectedNodeWithOutputType(ConnectionType);
		if (CurrentlySelectedNode && GetToolManager())
		{
			const bool bToolActive = GetToolManager()->HasActiveTool(EToolSide::Left);
			return !bToolActive;
		}
	}
	return false;
}

const TArray<FName>& UDataflowEditorMode::GetToolCategories()
{
	const UDataflowEditor* LocalEditor = DataflowEditor;
	if(!LocalEditor)
	{
		if (const TSharedPtr<FDataflowConstructionScene> ConstructionScenePtr = ConstructionScene.Pin())
		{
			LocalEditor = ConstructionScenePtr->GetDataflowEditor();
		}
		else if(FAssetEditorModeManager* ModeManager = StaticCast<FAssetEditorModeManager*>(Owner))
		{
			// The local construction scene is not yet set when calling that function.
			// We then have to rely on the mode manager one 
			if(FDataflowConstructionScene* ManagerScene = StaticCast<FDataflowConstructionScene*>(ModeManager->GetPreviewScene()))
			{
				LocalEditor = ManagerScene->GetDataflowEditor();
			}
		}
	}
	static TArray<FName> EmptyToolCategories;
	return LocalEditor ? LocalEditor->GetToolCategories() : EmptyToolCategories;
}


void UDataflowEditorMode::RegisterTools()
{
	const FDataflowEditorCommandsImpl& CommandInfos = FDataflowEditorCommands::Get();

	UEditorInteractiveToolsContext* const ConstructionViewportToolsContext = GetInteractiveToolsContext();

	UE::Dataflow::FDataflowToolRegistry& ToolRegistry = UE::Dataflow::FDataflowToolRegistry::Get();
	const TArray<FName> NodeNames = ToolRegistry.GetNodeNames();
	const TArray<FName> ToolCategories = GetToolCategories();
	
	for (const FName& RegisteredNodeName : NodeNames)
	{
		if(ToolCategories.IsEmpty() || ToolCategories.Contains(ToolRegistry.GetToolCategoryForNode(RegisteredNodeName)))
		{
			const TSharedPtr<FUICommandInfo> CommandInfo = ToolRegistry.GetToolCommandForNode(RegisteredNodeName);
			UInteractiveToolBuilder* const Builder = ToolRegistry.GetToolBuilderForNode(RegisteredNodeName);

			// TODO: This is here only so the Tool can hide the all meshes in the DataflowConstructionScene. That should probably be handed in this class instead.
			if (UDataflowEditorWeightMapPaintToolBuilder* WeightMapPaintToolBuilder = Cast<UDataflowEditorWeightMapPaintToolBuilder>(Builder))
			{
				WeightMapPaintToolBuilder->SetEditorMode(this);
			}
			else if (UDataflowEditorVertexAttributePaintToolBuilder* VertexAttributePaintToolBuilder = Cast<UDataflowEditorVertexAttributePaintToolBuilder>(Builder))
			{
				VertexAttributePaintToolBuilder->SetEditorMode(this);
			}

			RegisterDataflowTool(CommandInfo, RegisteredNodeName.ToString() + FString(TEXT("Tool")), Builder, ConstructionViewportToolsContext);

			NodeTypeToToolCommandMap.Add(RegisteredNodeName, CommandInfo);

			// Register "Add Node" commands for buttons in the UI. The EditorToolkit will construct the actual toolbar buttons.
			NodeTypeToAddNodeCommandMap.Add(RegisteredNodeName, ToolRegistry.GetAddNodeCommandForNode(RegisteredNodeName));
		}
	}

}

bool UDataflowEditorMode::ShouldToolStartBeAllowed(const FString& ToolIdentifier) const
{
	// Allow switching away from tool if no changes have been made in the tool yet (which we infer from the CanAccept status)
	if (GetInteractiveToolsContext()->CanAcceptActiveTool())
	{
		return false;
	}

	const TSharedPtr<FDataflowConstructionScene> ConstructionScenePtr = ConstructionScene.Pin();
	if (ConstructionScenePtr && ConstructionScenePtr->GetDataflowModeManager() && ConstructionScenePtr->GetDataflowModeManager()->GetInteractiveToolsContext())
	{
		if (ConstructionScenePtr->GetDataflowModeManager()->GetInteractiveToolsContext()->HasActiveTool())
		{
			return false;
		}
	}

	if (UE::Dataflow::Private::bDataflowEditorEnableToolsInPIE)
	{
		// UEdMode::ShouldToolStartBeAllowed returns (!GEditor->PlayWorld && !GIsPlayInEditorWorld) but we want to allow tools to start while in PIE
		return true;
	}
	else
	{
		return UBaseCharacterFXEditorMode::ShouldToolStartBeAllowed(ToolIdentifier);
	}
}

void UDataflowEditorMode::CreateToolkit()
{
	Toolkit = MakeShared<FDataflowEditorModeToolkit>();
}


void UDataflowEditorMode::SetWireframeRenderToggleEnabled(bool bEnable)
{
	const TSharedPtr<FDataflowConstructionScene> ConstructionScenePtr = ConstructionScene.Pin();
	if (const TObjectPtr<UDataflowBaseContent> EditorContent = ConstructionScenePtr ? ConstructionScenePtr->GetEditorContent() : nullptr)
	{
		if (const TObjectPtr<UDataflow> DataflowGraph = EditorContent->GetDataflowAsset())
		{
			for (UEdGraphNode* const EdGraphNode : DataflowGraph->Nodes)
			{
				if (UDataflowEdNode* const DataflowEdNode = Cast<UDataflowEdNode>(EdGraphNode))
				{
					DataflowEdNode->SetCanEnableWireframeRenderNode(bEnable);
				}
			}
		}
	}
}

void UDataflowEditorMode::OnToolStarted(UInteractiveToolManager* Manager, UInteractiveTool* Tool)
{
	FDataflowEditorCommandsImpl::UpdateToolCommandBinding(Tool, ToolCommandList, false);

	// Temporarily disable wireframe render toggle switch on all nodes
	SetWireframeRenderToggleEnabled(false);

	// Force redraw
	if (const TSharedPtr<FDataflowConstructionScene> ConstructionScenePtr = ConstructionScene.Pin())
	{
		ConstructionScenePtr->SetDirtyFlag();
	}
}

void UDataflowEditorMode::OnToolEnded(UInteractiveToolManager* Manager, UInteractiveTool* Tool)
{
	FDataflowEditorCommandsImpl::UpdateToolCommandBinding(Tool, ToolCommandList, true);

	if (bShouldRestoreConstructionViewWireframe)
	{
		bConstructionViewWireframe = true;
		bShouldRestoreConstructionViewWireframe = false;
	}

	if (bShouldRestoreSavedConstructionViewMode)
	{
		SetConstructionViewMode(SavedConstructionViewMode);
		bShouldRestoreSavedConstructionViewMode = false;
	}
	else if (const TSharedPtr<FDataflowConstructionScene> ConstructionScenePtr = ConstructionScene.Pin())
	{
		ConstructionScenePtr->ResetConstructionScene();
	}

	if (TSharedPtr<SDataflowGraphEditor> GraphEditor = DataflowGraphEditor.Pin())
	{
		GraphEditor->SetEnabled(true);
	}

	// Re-enable wireframe render toggle switch on all nodes
	SetWireframeRenderToggleEnabled(true);
}

void UDataflowEditorMode::BindCommands()
{
	const FDataflowEditorCommandsImpl& CommandInfos = FDataflowEditorCommands::Get();
	const TSharedRef<FUICommandList>& CommandList = Toolkit->GetToolkitCommands();

	// Hook up to Enter/Esc key presses
	CommandList->MapAction(
		CommandInfos.AcceptOrCompleteActiveTool,
		FExecuteAction::CreateLambda([this](){ AcceptActiveToolActionOrTool();}),
		FCanExecuteAction::CreateLambda([this]() {
				return GetInteractiveToolsContext()->CanAcceptActiveTool() || GetInteractiveToolsContext()->CanCompleteActiveTool();
			}),
		FGetActionCheckState(),
		FIsActionButtonVisible(),
		EUIActionRepeatMode::RepeatDisabled);

	CommandList->MapAction(
		CommandInfos.CancelOrCompleteActiveTool,
		FExecuteAction::CreateLambda([this]() { CancelActiveToolActionOrTool(); }),
		FCanExecuteAction::CreateLambda([this]() {
			return GetInteractiveToolsContext()->CanCompleteActiveTool() || GetInteractiveToolsContext()->CanCancelActiveTool();
			}),
		FGetActionCheckState(),
		FIsActionButtonVisible(),
		EUIActionRepeatMode::RepeatDisabled);
}

void UDataflowEditorMode::Exit()
{
	if (UInteractiveToolsContext* ToolsContext = GetInteractiveToolsContext())
	{
		if (UInteractiveGizmoManager* GizmoManager = ToolsContext->ToolManager->GetPairedGizmoManager())
		{
			GizmoManager->DeregisterGizmoType(UEditorInteractiveGizmoManager::TransformBuilderIdentifier());
		}
	}

	UActorComponent::MarkRenderStateDirtyEvent.RemoveAll(this);

	if (const TSharedPtr<FDataflowConstructionScene> ConstructionScenePtr = ConstructionScene.Pin())
	{
		ConstructionScenePtr->ResetConstructionScene();
	}

	if (const TSharedPtr<FDataflowSimulationScene> SimulationScenePtr = SimulationScene.Pin())
	{
		SimulationScenePtr->ResetSimulationScene();
	}

	// Log mode exit
	if (FEngineAnalytics::IsAvailable())
	{
		const FTimespan ModeUsageDuration = FDateTime::UtcNow() - LastModeStartTimestamp;

		TArray<FAnalyticsEventAttribute> Attributes;
		Attributes.Add(FAnalyticsEventAttribute(TEXT("Timestamp"), FDateTime::UtcNow().ToString()));
		Attributes.Add(FAnalyticsEventAttribute(TEXT("Duration.Seconds"), static_cast<float>(ModeUsageDuration.GetTotalSeconds())));

		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.DataflowEditor.Exit"));
	}

	Super::Exit();
}

void UDataflowEditorMode::SetDataflowConstructionScene(TWeakPtr<FDataflowConstructionScene> InConstructionScene)
{
	// Remove previous delegate
	if (const TSharedPtr<FDataflowConstructionScene> ConstructionScenePtr = ConstructionScene.Pin())
	{
		ConstructionScenePtr->OnConstructionSceneUpdated.RemoveAll(this);
	}

	ConstructionScene = InConstructionScene;

	if (const TSharedPtr<FDataflowConstructionScene> ConstructionScenePtr = ConstructionScene.Pin())
	{
		UEditorInteractiveToolsContext* const PreviewToolsContext = ConstructionScenePtr->GetDataflowModeManager()->GetInteractiveToolsContext();
		UInteractiveToolManager* const PreviewToolManager = PreviewToolsContext->ToolManager;
		//#todo(brice): Make sure AddToolTargetFactories has been called. 
		//PreviewToolsContext->TargetManager->AddTargetFactory(NewObject<UClothComponentToolTargetFactory>(PreviewToolManager));
		//PreviewToolsContext->TargetManager->AddTargetFactory(NewObject<USkeletalMeshComponentToolTargetFactory>(PreviewToolManager));

		PreviewToolManager->OnToolStarted.AddUObject(this, &UDataflowEditorMode::OnToolStarted);
		PreviewToolManager->OnToolEnded.AddUObject(this, &UDataflowEditorMode::OnToolEnded);

		check(Toolkit.IsValid());

		// FBaseToolkit's OnToolStarted and OnToolEnded are protected, so we use the subclass to get at them
		FDataflowEditorModeToolkit* const DataflowModeToolkit = static_cast<FDataflowEditorModeToolkit*>(Toolkit.Get());

		PreviewToolManager->OnToolStarted.AddSP(DataflowModeToolkit, &FDataflowEditorModeToolkit::OnToolStarted);
		PreviewToolManager->OnToolEnded.AddSP(DataflowModeToolkit, &FDataflowEditorModeToolkit::OnToolEnded);

		ConstructionScenePtr->OnConstructionSceneUpdated.AddUObject(this, &UDataflowEditorMode::OnConstructionSceneUpdated);
	}
}

void UDataflowEditorMode::OnConstructionSceneUpdated()
{
	if (FDataflowEditorModeToolkit* const DataflowModeToolkit = static_cast<FDataflowEditorModeToolkit*>(Toolkit.Get()))
	{
		if (const TSharedPtr<FDataflowConstructionScene> ConstructionScenePtr = ConstructionScene.Pin())
		{
			TArray<UDataflowRenderableTypeSettings*> RenderSettings = ConstructionScenePtr->GetRenderableTypeSettings();
			DataflowModeToolkit->OnRenderSettingsUpdated(RenderSettings);
		}
	}
}

void UDataflowEditorMode::CreateToolTargets(const TArray<TObjectPtr<UObject>>& AssetsIn)
{
	ToolTargets.Reset();
	if (const TSharedPtr<FDataflowConstructionScene> ConstructionScenePtr = ConstructionScene.Pin())
	{
		if (const TObjectPtr<UDataflowBaseContent>& EditorContent = ConstructionScenePtr->GetEditorContent())
		{
			if (UToolTarget* const Target = GetInteractiveToolsContext()->TargetManager->BuildTarget(EditorContent, GetToolTargetRequirements()))
			{
				ToolTargets.Add(Target);
			}
		}
	}
}

bool UDataflowEditorMode::IsComponentSelected(const UPrimitiveComponent* InComponent)
{
	const TSharedPtr<FDataflowConstructionScene> ConstructionScenePtr = ConstructionScene.Pin();
	if (ConstructionScenePtr&& ConstructionScenePtr->GetDataflowModeManager())
	{
		if (const UTypedElementSelectionSet* const TypedElementSelectionSet = ConstructionScenePtr->GetDataflowModeManager()->GetEditorSelectionSet())
		{
			if (const FTypedElementHandle ComponentElement = UEngineElementsLibrary::AcquireEditorComponentElementHandle(InComponent))
			{
				const bool bElementSelected = TypedElementSelectionSet->IsElementSelected(ComponentElement, FTypedElementIsSelectedOptions());
				return bElementSelected;
			}
		}
	}

	return false;
}

void UDataflowEditorMode::RefocusConstructionViewportClient()
{
	TSharedPtr<FDataflowConstructionViewportClient, ESPMode::ThreadSafe> PinnedVC = ConstructionViewportClient.Pin();
	if (PinnedVC.IsValid())
	{
		// This will happen in FocusViewportOnBox anyways; do it now to get a consistent end result
		PinnedVC->ToggleOrbitCamera(false);

		const bool bPerspectiveView = (ConstructionViewMode && ConstructionViewMode->IsPerspective());
		const FVector  ViewportPosition = bPerspectiveView ? FVector(0, 150, 200) : FVector(0, 0, -100);
		const FRotator ViewportOrientation = ConstructionViewMode ? ConstructionViewMode->GetViewportOrientation() : FRotator::ZeroRotator;
		PinnedVC->SetInitialViewTransform(ELevelViewportType::LVT_Perspective, ViewportPosition, ViewportOrientation, DEFAULT_ORTHOZOOM);

		const FBox SceneBounds = SceneBoundingBox();
		constexpr bool bInstant = true;
		PinnedVC->FocusViewportOnBox(SceneBounds, bInstant);

		// Recompute near/far clip planes
		PinnedVC->SetConstructionViewMode(ConstructionViewMode);
	}
}

void UDataflowEditorMode::RefocusSimulationViewportClient()
{
	if (const TSharedPtr<FDataflowSimulationScene> SimulationScenePtr = SimulationScene.Pin())
	{
		if (const TSharedPtr<FDataflowSimulationViewportClient> PinnedVC = SimulationViewportClient.Pin())
		{
			// This will happen in FocusViewportOnBox anyways; do it now to get a consistent end result
			PinnedVC->ToggleOrbitCamera(false);

			const FBox SceneBounds = SimulationScenePtr->GetBoundingBox();

			// Set up camera for an angled view by default 
			PinnedVC->SetInitialViewTransform(ELevelViewportType::LVT_Perspective, FVector(0.0, 0.0, 0.0), FRotator(-15.0, -40.0, 0.0), DEFAULT_ORTHOZOOM);

			constexpr bool bInstant = true;
			PinnedVC->FocusViewportOnBox(SceneBounds, bInstant);
		}
	}
}

void UDataflowEditorMode::FirstTimeFocusConstructionViewport()
{
	// If this is the first time seeing a valid 2D or 3D mesh, refocus the camera on it.
	const TSharedPtr<FDataflowConstructionScene> ConstructionScenePtr = ConstructionScene.Pin();
	if (ConstructionScenePtr && ConstructionScenePtr->HasRenderableGeometry())
	{
		const FName ViewModeName = ConstructionViewMode? ConstructionViewMode->GetName(): NAME_None;
		if (!FirstTimeRefocusedViews.Contains(ViewModeName))
		{
			FirstTimeRefocusedViews.Add(ViewModeName);
			RefocusConstructionViewportClient();
		}
	}
}

void UDataflowEditorMode::SetInputFocusOnConstructionViewport()
{
	if (TSharedPtr<FDataflowConstructionViewportClient> PinnedConstructionViewportClient = ConstructionViewportClient.Pin())
	{
		if (PinnedConstructionViewportClient->Viewport)
		{
			if (FSceneViewport* SceneViewport = PinnedConstructionViewportClient->Viewport->AsSceneViewport())
			{
				if (TSharedPtr<SWidget> ViewportWidget = SceneViewport->GetViewportWidget().Pin())
				{
					FSlateApplication::Get().SetUserFocus(FSlateApplication::Get().GetUserIndexForKeyboard(), ViewportWidget, EFocusCause::SetDirectly);
				}
			}
		}
	}
}

void UDataflowEditorMode::FirstTimeFocusSimulationViewport()
{
	// If this is the first time seeing a valid 2D or 3D mesh, refocus the camera on it.
	const TSharedPtr<FDataflowSimulationScene> SimulationScenePtr = SimulationScene.Pin();
	if (SimulationScenePtr && SimulationScenePtr->HasRenderableGeometry())
	{
		RefocusSimulationViewportClient();
	}
}

void UDataflowEditorMode::InitializeTargets(const TArray<TObjectPtr<UObject>>& ObjectsToEdit)
{
	UBaseCharacterFXEditorMode::InitializeTargets(ObjectsToEdit);

	// @todo(brice) : Consider initializing the Content here from the ObjectsToEdit

	// @todo(brice) : What are the ToolTargets storing?
	// ... for(ToolTarget& : ToolTargets){
	// ... UE::ToolTarget::GetDynamicMeshCopy(Target)
	// ... UE::ToolTarget::GetMaterialSet(Target).Materials for ConstructionScene->AddDynamicMeshComponent
	// ... }

	// @todo(michael) : do we need to update the construction scene?
	if (const TSharedPtr<FDataflowConstructionScene> ConstructionScenePtr = ConstructionScene.Pin())
	{
		ConstructionScenePtr->UpdateConstructionScene();
	}
}

TObjectPtr<UDataflowBaseContent> UDataflowEditorMode::GetDataflowEditorContent() const
{
	if (const TSharedPtr<FDataflowConstructionScene> ConstructionScenePtr = ConstructionScene.Pin())
	{
		return ConstructionScenePtr ? ConstructionScenePtr->GetEditorContent().Get() : nullptr;
	}
	return nullptr;
}

void UDataflowEditorMode::ModeTick(float DeltaTime)
{
	Super::ModeTick(DeltaTime);

	if (TSharedPtr<SDataflowGraphEditor> GraphEditor = DataflowGraphEditor.Pin())
	{
		// For now don't allow selection change once the tool has uncommitted changes
		// TODO: We might want to auto-accept unsaved changes and allow switching between nodes
		if (GetInteractiveToolsContext()->CanAcceptActiveTool())
		{
			GraphEditor->SetEnabled(false);
		}
		else
		{
			GraphEditor->SetEnabled(true);
		}
	}

	const TSharedPtr<FDataflowConstructionScene> ConstructionScenePtr = ConstructionScene.Pin();
	const UDataflowBaseContent* const EditorContent = ConstructionScenePtr ? ConstructionScenePtr->GetEditorContent().Get() : nullptr;
	if (EditorContent && EditorContent->IsConstructionDirty())
	{
		ConstructionScenePtr->UpdateConstructionScene();
		FirstTimeFocusConstructionViewport();
	}

	TSharedPtr<UE::Dataflow::FContext> DataflowContext;
	if (const UEditorInteractiveToolsContext* const ConstructionToolsContext = GetInteractiveToolsContext())
	{
		if (UDataflowContextObject* ContextObject = ConstructionToolsContext->ContextObjectStore->FindContext<UDataflowContextObject>())
		{
			DataflowContext = ContextObject->GetDataflowContext();
		}
	}

	TObjectPtr<UDataflowEdNode> SelectedNode = GetSelectedNode();


	if (CanSelectedNodeLaunchTool())
	{
		if (TSharedPtr<const FUICommandInfo> Command = UDataflowEditorMode::GetSelectedNodeToolCommand())
		{
			if (DataflowContext && !IsSelectedNodeUpToDate(*DataflowContext))
			{
				UserMessage = FText::Format(LOCTEXT("OutOfDateNode", "{0} tool cannot launch because the node is out of date and need evaluation (Evaluation mode may be set to Manual)"), Command->GetLabel());
			}
			else
			{
				const bool bLaunchedTool = Toolkit->GetToolkitCommands()->TryExecuteAction(Command.ToSharedRef());
				if (!bLaunchedTool)
				{
					const USelection* Selection = ConstructionScenePtr ? ConstructionScenePtr->GetSelectedComponents() : nullptr;
					if (Selection && Selection->Num() == 0)
					{
						// TODO : find a more accurate message ?
						if (!ConstructionScenePtr->HasRenderableGeometry())
						{
							UserMessage = FText::Format(LOCTEXT("NoComponentAvailable", "{0} tool cannot launch because there's no component in the construction view - check the node settings"), Command->GetLabel());
						}
						else
						{
							UserMessage = FText::Format(LOCTEXT("ToolNeedOneComponent", "You must select a component to launch the {0} tool"), Command->GetLabel());
						}
					}
					else
					{
						UserMessage = FText::Format(LOCTEXT("ToolFailedToLaunch", "Failed to launch {0} tool - Check Editor logs"), Command->GetLabel());
					}
				}
				else
				{
					UserMessage = FText::GetEmpty();
				}
			}
		}
	}
	else
	{
		UserMessage = FText::GetEmpty();
	}

	if (bShouldRestartToolNextTick)
	{
		// If we ended the active tool in order to change view mode, restart it now
		if (EditorContent && !EditorContent->IsConstructionDirty())		// hold off restarting the tool until the scene finishes rebuilding
		{
			// First select the lone mesh in the construction scene if there is one
			if (bHadSingleSelectionBeforeToolShutdown)
			{
				const TArray<TObjectPtr<UDynamicMeshComponent>> DynamicMeshComponents = ConstructionScenePtr ? ConstructionScenePtr->GetDynamicMeshComponents() : TArray<TObjectPtr<UDynamicMeshComponent>>();
				if (DynamicMeshComponents.Num() == 1)
				{
					if (USelection* const SelectedComponents = ConstructionScenePtr->GetSelectedComponents())
					{
						SelectedComponents->Select(DynamicMeshComponents[0]);
					}
				}
			}

			// Now start the tool
			if (SelectedNode)
			{
				StartToolForSelectedNode(SelectedNode);
			}

			bShouldRestartToolNextTick = false;
		}
	}
}

void UDataflowEditorMode::ViewportResized(FViewport* Viewport, uint32 /*Unused*/)
{
	// We'd like to call Refocus*ViewportClient() when the viewport is first created, however the viewport needs to have non-zero size for FocusViewportOnBox() to work properly. 
	// So we wait until the viewport is resized and call it here.

	// Construction

	if (TSharedPtr<FDataflowConstructionViewportClient> PinnedConstructionViewportClient = ConstructionViewportClient.Pin())
	{
		if (PinnedConstructionViewportClient->Viewport == Viewport)
		{
			if (bShouldFocusConstructionView && Viewport->GetSizeXY().X > 0 && Viewport->GetSizeXY().Y > 0)
			{
				RefocusConstructionViewportClient();
				bShouldFocusConstructionView = false;
				return;
			}
		}
	}

	// Simulation

	if (TSharedPtr<FDataflowSimulationViewportClient> PinnedSimulationViewportClient = SimulationViewportClient.Pin())
	{
		if (PinnedSimulationViewportClient->Viewport == Viewport)
		{
			if (bShouldFocusSimulationView && Viewport->GetSizeXY().X > 0 && Viewport->GetSizeXY().Y > 0)
			{
				RefocusSimulationViewportClient();
				bShouldFocusSimulationView = false;
			}
		}
	}
}


FBox UDataflowEditorMode::SceneBoundingBox() const
{
	const FBox DefaultFocusBox(FVector(-50, -50, 0), FVector(+50, +50, +100));

	const TSharedPtr<FDataflowConstructionScene> ConstructionScenePtr = ConstructionScene.Pin();
	FBox BoxToReturn = ConstructionScenePtr 
		? ConstructionScenePtr->GetBoundingBox() 
		: DefaultFocusBox;
	if (!BoxToReturn.IsValid)
	{
		BoxToReturn = DefaultFocusBox;
	}
	return BoxToReturn;
}

FBox UDataflowEditorMode::SelectionBoundingBox() const
{
	// if Tool supports custom Focus box, use that first
	if (GetToolManager()->HasAnyActiveTool())
	{
		UInteractiveTool* const Tool = GetToolManager()->GetActiveTool(EToolSide::Mouse);
		IInteractiveToolCameraFocusAPI* const FocusAPI = Cast<IInteractiveToolCameraFocusAPI>(Tool);
		if (FocusAPI && FocusAPI->SupportsWorldSpaceFocusBox())
		{
			return FocusAPI->GetWorldSpaceFocusBox();
		}
	}

	// If the selection is on, the GetBoundingBox is automatically computing the selection one
	// Otherwise if nothing selected, it will return the whole scene
	return SceneBoundingBox();
}

void UDataflowEditorMode::SetConstructionViewMode(const FName& NewViewModeName)
{
	if (NewViewModeName == ConstructionViewMode->GetName())
	{
		return;
	}

	const TObjectPtr<UInteractiveToolManager> ToolManager = GetInteractiveToolsContext()->ToolManager;
	checkf(ToolManager, TEXT("No valid ToolManager found for UDataflowEditorMode"));

	const TSharedPtr<FDataflowConstructionScene> ConstructionScenePtr = ConstructionScene.Pin();

	// Check if we have a single component selected. If we do, we will attempt to re-select it once the Construction Scene is rebuilt (if we have a tool running)
	bHadSingleSelectionBeforeToolShutdown = false;

	// Also check if we needed to shut down a running tool or not
	bool bToolWasShutDown = false;

	if (UInteractiveTool* const ActiveTool = ToolManager->GetActiveTool(EToolSide::Left))
	{
		if (const USelection* const SelectedComponents = ConstructionScenePtr ? ConstructionScenePtr->GetSelectedComponents() : nullptr)
		{
			if (SelectedComponents->Num() == 1)			// TODO: Extend this to handle multiple selected components
			{
				if (const UDynamicMeshComponent* const SelectedDynamicMeshComponent = Cast<UDynamicMeshComponent>(SelectedComponents->GetSelectedObject(0)))
				{
					if (ConstructionScenePtr->GetDynamicMeshComponents().Contains(SelectedDynamicMeshComponent))
					{
						bHadSingleSelectionBeforeToolShutdown = true;
					}
				}
			}
		}

		const UInteractiveToolBuilder* const ActiveToolBuilder = ToolManager->GetActiveToolBuilder(EToolSide::Left);
		checkf(ActiveToolBuilder, TEXT("Found active tool with no active tool builder"));

		bool bToolCanHandleStateChange = false;

		if (const IDataflowEditorToolBuilder* const DataflowToolBuilder = Cast<IDataflowEditorToolBuilder>(ActiveToolBuilder))
		{
			FToolBuilderState SceneState;
			ToolManager->GetContextQueriesAPI()->GetCurrentSelectionState(SceneState);
			bToolCanHandleStateChange = DataflowToolBuilder->CanSceneStateChange(ActiveTool, SceneState);
		}

		if (!bToolCanHandleStateChange)
		{
			ToolManager->PostActiveToolShutdownRequest(ActiveTool, EToolShutdownType::Accept);
			bToolWasShutDown = true;
		}
	}

	const UE::Dataflow::FRenderingViewModeFactory& ViewModes = UE::Dataflow::FRenderingViewModeFactory::GetInstance();
	const UE::Dataflow::IDataflowConstructionViewMode* const NewMode = ViewModes.GetViewMode(NewViewModeName);
	if (!NewMode)
	{
		UE_LOGF(LogChaos, Warning, "Warning : Unknown rendering view mode: %ls", *NewViewModeName.ToString());
		return;
	}

	// Do the actual view mode updates

	ConstructionViewMode = NewMode;
	if (ConstructionScenePtr)
	{
		ConstructionScenePtr->GetEditorContent()->SetConstructionViewMode(ConstructionViewMode);
		ConstructionScenePtr->UpdateConstructionScene();
	}

	const TSharedPtr<FDataflowConstructionViewportClient> VC = ConstructionViewportClient.Pin();
	if (VC.IsValid())
	{
		VC->SetConstructionViewMode(ConstructionViewMode);
	}

	// If we are switching to a mode with a valid mesh for the first time, focus the camera on it
	FirstTimeFocusConstructionViewport();


	if (bToolWasShutDown)
	{
		// Tool restart must be done on the next tick because shutting down the current tool will cause the ConstructionView to be rebuilt next tick as well
		bShouldRestartToolNextTick = true;
	}
	else if (UInteractiveTool* const ActiveTool = ToolManager->GetActiveTool(EToolSide::Left))
	{
		// If there is a currently active tool, notify it that the scene has changed

		// First check if we previously had a single selected component before changing view modes. If so, and if there is now a single component in the construction scene, select it.
		// TODO: Extend this to handle multiple selected components

		if (bHadSingleSelectionBeforeToolShutdown)
		{
			const TArray<TObjectPtr<UDynamicMeshComponent>> DynamicMeshComponents = ConstructionScenePtr ? ConstructionScenePtr->GetDynamicMeshComponents() : TArray<TObjectPtr<UDynamicMeshComponent>>();
			if (DynamicMeshComponents.Num() == 1)
			{
				if (USelection* const SelectedComponents = ConstructionScenePtr->GetSelectedComponents())
				{
					SelectedComponents->Select(DynamicMeshComponents[0]);
				}
			}
		}

		// Now notify the active tool that the SceneState is different
		UInteractiveToolBuilder* const ActiveToolBuilder = ToolManager->GetActiveToolBuilder(EToolSide::Left);
		checkf(ActiveToolBuilder, TEXT("Found active tool with no active tool builder"));

		if (IDataflowEditorToolBuilder* const DataflowToolBuilder = Cast<IDataflowEditorToolBuilder>(ActiveToolBuilder))
		{
			FToolBuilderState SceneState;
			ToolManager->GetContextQueriesAPI()->GetCurrentSelectionState(SceneState);
			DataflowToolBuilder->SceneStateChanged(ActiveTool, SceneState);
		}
	}

	if (const TObjectPtr<const UDataflowEdNode> SelectedDataflowEdNode = GetSelectedNode())
	{
		if (const TObjectPtr<UDataflowBaseContent>& EditorContent = GetDataflowEditorContent())
		{
			TArray<FName> ViewModesForNode;
			UE::Dataflow::GetViewModesForNode(*SelectedDataflowEdNode, *EditorContent, ViewModesForNode);

			if (ViewModesForNode.Contains(NewViewModeName))
			{
				ViewModesForNode.Sort([](const FName& A, const FName& B)
				{
					return A.FastLess(B);
				});

				NodeViewModeHistory.FindOrAdd(ViewModesForNode) = NewViewModeName;
			}
		}
	}

}

void UDataflowEditorMode::SetConstructionViewModeForNode(const UDataflowEdNode* Node)
{
	const TSharedPtr<FDataflowConstructionScene> ConstructionScenePtr = ConstructionScene.Pin();
	const TObjectPtr<UDataflowBaseContent> EditorContent = ConstructionScenePtr ? ConstructionScenePtr->GetEditorContent() : nullptr;
	if (Node && EditorContent)
	{
		// Check the most recently used view mode for this "kind" of node -- where "kind" means nodes that have the same set of valid view modes
		auto FindRecentViewMode =
			[this, &Node, &EditorContent](FName& OutViewModeName) -> bool
			{
				TArray<FName> ViewModesForNode;
				UE::Dataflow::GetViewModesForNode(*Node, *EditorContent, ViewModesForNode);

				if (const FName* const FoundViewMode = FindRecentlyUsedViewMode(ViewModesForNode))
				{
					OutViewModeName = *FoundViewMode;
					return true;
				}
				return false;
			};

		// Get the first view mode that this node can use
		auto FindAnyValidViewMode =
			[this, &Node, &EditorContent](FName& OutViewModeName) -> bool
			{
				for (const TPair<FName, TUniquePtr<UE::Dataflow::IDataflowConstructionViewMode>>& ViewMode : UE::Dataflow::FRenderingViewModeFactory::GetInstance().GetViewModes())
				{
					check(ViewMode.Value.IsValid());
					const bool bCanRender = UE::Dataflow::IsViewModeSupported(*Node, *EditorContent, *ViewMode.Value);
					if (bCanRender)
					{
						OutViewModeName = ViewMode.Key;
						return true;
					}
				}
				return false;
			};

		FName NewViewMode;
		if (FindRecentViewMode(NewViewMode))               // Is there a recently used view mode we can switch to?
		{
			SetConstructionViewMode(NewViewMode);
		}
		else if (UE::Dataflow::IsViewModeSupported(*Node, *EditorContent, *GetConstructionViewMode()))        // Can we use the current view mode?
		{
			// no need to switch
		}
		else if (FindAnyValidViewMode(NewViewMode))             // Is there *any* view mode that works?
		{
			SetConstructionViewMode(NewViewMode);
		}
		else                                                    // No valid view mode found
		{
			// TODO: We should clear and disable View Mode Button. For now set default mode to the built-in 3D view mode.
			SetConstructionViewMode(UE::Dataflow::FDataflowConstruction3DViewMode::Name);
		}
	}
}


const UE::Dataflow::IDataflowConstructionViewMode* UDataflowEditorMode::GetConstructionViewMode() const
{
	return ConstructionViewMode;
}

bool UDataflowEditorMode::CanChangeConstructionViewModeTo(const FName& NewViewModeName) const
{
	if (!GetToolManager()->HasActiveTool(EToolSide::Left))
	{
		if (const TSharedPtr<FDataflowConstructionScene> ConstructionScenePtr = ConstructionScene.Pin())
		{
			if (const TObjectPtr<UDataflowBaseContent> EditorContent = GetDataflowEditorContent())
			{
				if (const TObjectPtr<UDataflowEdNode> SelectedDataflowEdNode = EditorContent->GetSelectedNode())
				{
					if (const UE::Dataflow::IDataflowConstructionViewMode* const ViewMode = UE::Dataflow::FRenderingViewModeFactory::GetInstance().GetViewMode(NewViewModeName))
					{
						if (UE::Dataflow::IsViewModeSupported(*SelectedDataflowEdNode, *EditorContent, *ViewMode))
						{
							return true;
						}
					}
				}
			}
		}
		return false;
	}

	// Check active tool to see if we can switch modes while the tool is running

	const UInteractiveToolBuilder* const ActiveToolBuilder = GetToolManager()->GetActiveToolBuilder(EToolSide::Left);
	checkf(ActiveToolBuilder, TEXT("No Active Tool Builder found despite having an Active Tool"));

	if (const IDataflowEditorToolBuilder* const DataflowToolBuilder = Cast<const IDataflowEditorToolBuilder>(ActiveToolBuilder))
	{
		const UEditorInteractiveToolsContext* const ConstructionToolsContext = GetInteractiveToolsContext();
		checkf(ConstructionToolsContext, TEXT("No Tools Context found in Dataflow Editor"));

		const UDataflowContextObject* const DataflowContextObject = ConstructionToolsContext->ContextObjectStore->FindContext<UDataflowContextObject>();
		checkf(DataflowContextObject, TEXT("No Dataflow Context Object found in ContextObjectStore, despite having an Active Tool. This should have been created by the time a tool is activated"));

		TArray<const UE::Dataflow::IDataflowConstructionViewMode*> SupportedViewModes;
		DataflowToolBuilder->GetSupportedConstructionViewModes(*DataflowContextObject, SupportedViewModes);

		if (const UE::Dataflow::IDataflowConstructionViewMode* const NewViewMode = UE::Dataflow::FRenderingViewModeFactory::GetInstance().GetViewMode(NewViewModeName))
		{
			return SupportedViewModes.Contains(NewViewMode);
		}
	}

	return false;
}


void UDataflowEditorMode::ToggleConstructionViewWireframe()
{
	check(false);
	bConstructionViewWireframe = !bConstructionViewWireframe;
	if (const TSharedPtr<FDataflowConstructionScene> ConstructionScenePtr = ConstructionScene.Pin())
	{
		ConstructionScenePtr->UpdateConstructionScene();
	}
}

bool UDataflowEditorMode::CanSetConstructionViewWireframeActive() const
{
	if (!GetToolManager()->HasActiveTool(EToolSide::Left))
	{
		return true;
	}

	const UInteractiveToolBuilder* const ActiveToolBuilder = GetToolManager()->GetActiveToolBuilder(EToolSide::Left);
	checkf(ActiveToolBuilder, TEXT("No Active Tool Builder found despite having an Active Tool"));

	const IDataflowEditorToolBuilder* const DataflowToolBuilder = Cast<const IDataflowEditorToolBuilder>(ActiveToolBuilder);
	checkf(DataflowToolBuilder, TEXT("Cloth Editor has an active Tool Builder that does not implement IDataflowEditorToolBuilder"));
	return DataflowToolBuilder->CanSetConstructionViewWireframeActive();
}

void UDataflowEditorMode::SetConstructionViewportClient(TWeakPtr<FDataflowConstructionViewportClient, ESPMode::ThreadSafe> InViewportClient)
{
	ConstructionViewportClient = InViewportClient;

	TSharedPtr<FDataflowConstructionViewportClient> VC = ConstructionViewportClient.Pin();
	if (VC.IsValid())
	{
		VC->SetConstructionViewMode(ConstructionViewMode);
		VC->SetToolCommandList(ToolCommandList);
	}
}

void UDataflowEditorMode::SetSimulationViewportClient(TWeakPtr<FDataflowSimulationViewportClient, ESPMode::ThreadSafe> InViewportClient)
{
	SimulationViewportClient = InViewportClient;
}

void UDataflowEditorMode::InitializeContextObject()
{
	const TSharedPtr<FDataflowConstructionScene> ConstructionScenePtr = ConstructionScene.Pin();
	if (const TObjectPtr<UDataflowBaseContent>& EditorContent = ConstructionScenePtr ? ConstructionScenePtr->GetEditorContent() : nullptr)
	{
		const UEditorInteractiveToolsContext* const ConstructionToolsContext = GetInteractiveToolsContext();

		UDataflowContextObject* ContextObject = ConstructionToolsContext->ContextObjectStore->FindContext<UDataflowContextObject>();
		if (!ContextObject)
		{
			ContextObject = EditorContent;
			ConstructionToolsContext->ContextObjectStore->AddContextObject(ContextObject);
		}

		check(ContextObject);
		ContextObject->SetConstructionViewMode(ConstructionViewMode);
	}
}

void UDataflowEditorMode::DeleteContextObject()
{
	UEditorInteractiveToolsContext* const ConstructionToolsContext = GetInteractiveToolsContext();
	if (UDataflowContextObject* ContextObject = ConstructionToolsContext->ContextObjectStore->FindContext<UDataflowContextObject>())
	{
		ConstructionToolsContext->ContextObjectStore->RemoveContextObject(ContextObject);
	}
}

void UDataflowEditorMode::SetDataflowGraphEditor(TSharedPtr<SDataflowGraphEditor> InGraphEditor)
{
	if (InGraphEditor)
	{
		DataflowGraphEditor = InGraphEditor;
		InitializeContextObject();
	}
	else
	{
		DeleteContextObject();
	}
}

TObjectPtr<UDataflowEdNode> UDataflowEditorMode::GetSelectedNode() const
{
	if (TObjectPtr<UDataflowBaseContent> EditorContent = GetDataflowEditorContent())
	{
		return EditorContent->GetSelectedNode();
	}
	return nullptr;
}

FName UDataflowEditorMode::GetSelectedNodeTypeName() const
{
	if (TObjectPtr<UDataflowEdNode> SelectedNode = GetSelectedNode())
	{
		if (const TSharedPtr<const FDataflowNode> DataflowNode = SelectedNode->GetDataflowNode())
		{
			return DataflowNode->GetType();
		}
	}
	return NAME_None;
}

void UDataflowEditorMode::StartToolForSelectedNode(const UObject* SelectedNode)
{
	if (const UDataflowEdNode* const EdNode = Cast<UDataflowEdNode>(SelectedNode))
	{
		if (const TSharedPtr<const FDataflowNode> DataflowNode = EdNode->GetDataflowNode())
		{
			const FName DataflowNodeType = DataflowNode->GetType();
			NodeTypeForPendingToolStart = DataflowNodeType;
		}
	}
}

void UDataflowEditorMode::ShutdownActiveToolIfNeeded(EToolShutdownType ShutdownType)
{
	UEditorInteractiveToolsContext* const ToolsContext = GetInteractiveToolsContext();
	checkf(ToolsContext, TEXT("No valid ToolsContext found for UDataflowEditorMode"));

	UInteractiveToolManager* const ToolManager = ToolsContext->ToolManager;
	checkf(ToolManager, TEXT("ToolsContext doesn't have a valid ToolManager"));

	if (UInteractiveTool* const ActiveTool = ToolManager->GetActiveTool(EToolSide::Left))
	{
		// Accept will potentailly update the graph, if nothing has changed so far we want to simply cancel 
		if (ShutdownType == EToolShutdownType::Accept && !ToolsContext->CanAcceptActiveTool())
		{
			ShutdownType = EToolShutdownType::Completed;
		}
		ToolManager->PostActiveToolShutdownRequest(ActiveTool, ShutdownType);
	}
}

void UDataflowEditorMode::OnDataflowNodeDeleted(const TSet<UObject*>& DeletedNodes)
{
	UEditorInteractiveToolsContext* const ToolsContext = GetInteractiveToolsContext();
	checkf(ToolsContext, TEXT("No valid ToolsContext found for UDataflowEditorMode"));
	const bool bCanCancel = ToolsContext->CanCancelActiveTool();
	ToolsContext->EndTool(bCanCancel ? EToolShutdownType::Cancel : EToolShutdownType::Completed);
}

bool UDataflowEditorMode::IsAnyNodeSelected() const
{
	return (GetSelectedNode() != nullptr);
}

UEdGraphNode* UDataflowEditorMode::GetSingleSelectedNodeWithOutputType(const FName& SelectedNodeOutputTypeName) const
{
	if (TObjectPtr<UDataflowEdNode> SelectedDataflowEdNode = GetSelectedNode())
	{
		if (TSharedPtr<const FDataflowNode> SelectedDataflowNode = SelectedDataflowEdNode->GetDataflowNode())
		{
			for (const FDataflowOutput* const Output : SelectedDataflowNode->GetOutputs())
			{
				if (Output->GetType() == SelectedNodeOutputTypeName)
				{
					return SelectedDataflowEdNode;
				}
			}
		}
	}
	return nullptr;
}

UEdGraphNode* UDataflowEditorMode::CreateNewNode(const FName& NewNodeTypeName)
{
	if (const TObjectPtr<UDataflowBaseContent> EditorContent = GetDataflowEditorContent())
	{
		if (TObjectPtr<const UDataflowEdNode> SelectedNode = EditorContent->GetSelectedNode())
		{
			if (const TObjectPtr<UDataflow>& DataflowGraph = EditorContent->GetDataflowAsset())
			{
				const TSharedPtr<FAssetSchemaAction_Dataflow_CreateNode_DataflowEdNode> NodeAction =
					FAssetSchemaAction_Dataflow_CreateNode_DataflowEdNode::CreateAction(DataflowGraph, NewNodeTypeName);
				constexpr UEdGraphPin* FromPin = nullptr;
				constexpr bool bSelectNewNode = true;

				const FVector2f Location = SelectedNode->GetPosition() + FVector2f(250, 0);

				return NodeAction->PerformAction(DataflowGraph, FromPin, Location, bSelectNewNode);
			}
		}
	}
	return nullptr;
}


UEdGraphNode* UDataflowEditorMode::CreateAndConnectNewNode(const FName& NewNodeTypeName, UEdGraphNode& UpstreamNode, const FName& ConnectionTypeName, const FName& NewNodeConnectionName)
{
	const TSharedPtr<FDataflowConstructionScene> ConstructionScenePtr = ConstructionScene.Pin();
	if (const TObjectPtr<UDataflowBaseContent>& EditorContent = ConstructionScenePtr ? ConstructionScenePtr->GetEditorContent() : nullptr)
	{
		if (const TObjectPtr<UDataflow>& DataflowGraph = EditorContent->GetDataflowAsset())
		{
			// First find the specified output of the upstream node, plus any pins it's connected to

			UEdGraphPin* UpstreamNodeOutputPin = nullptr;
			TArray<UEdGraphPin*> ExistingNodeInputPins;

			const UDataflowEdNode* const UpstreamDataflowEdNode = CastChecked<UDataflowEdNode>(&UpstreamNode);
			const TSharedPtr<const FDataflowNode> UpstreamDataflowNode = UpstreamDataflowEdNode->GetDataflowNode();

			for (const FDataflowOutput* const Output : UpstreamDataflowNode->GetOutputs())
			{
				if (Output->GetType() == ConnectionTypeName)
				{
					UpstreamNodeOutputPin = UpstreamDataflowEdNode->FindPin(*Output->GetName().ToString(), EGPD_Output);
					ExistingNodeInputPins = UpstreamNodeOutputPin->LinkedTo;
					break;
				}
			}

			// Add the new node 

			UEdGraphNode* const NewEdNode = CreateNewNode(NewNodeTypeName);
			checkf(NewEdNode, TEXT("Failed to create a new node in the DataflowGraph"));

			UDataflowEdNode* const NewDataflowEdNode = CastChecked<UDataflowEdNode>(NewEdNode);
			const TSharedPtr<FDataflowNode> NewDataflowNode = NewDataflowEdNode->GetDataflowNode();

			// Re-wire the graph

			if (UpstreamNodeOutputPin)
			{
				UEdGraphPin* NewNodeInputPin = nullptr;
				for (const FDataflowInput* const NewNodeInput : NewDataflowNode->GetInputs())
				{
					if (NewNodeInput->GetType() == ConnectionTypeName && NewNodeInput->GetName() == NewNodeConnectionName)
					{
						NewNodeInputPin = NewDataflowEdNode->FindPin(*NewNodeInput->GetName().ToString(), EGPD_Input);
					}
				}

				UEdGraphPin* NewNodeOutputPin = nullptr;
				for (const FDataflowOutput* const NewNodeOutput : NewDataflowNode->GetOutputs())
				{
					if (NewNodeOutput->GetType() == ConnectionTypeName && NewNodeOutput->GetName() == NewNodeConnectionName)
					{
						NewNodeOutputPin = NewDataflowEdNode->FindPin(*NewNodeOutput->GetName().ToString(), EGPD_Output);
						break;
					}
				}

				check(NewNodeInputPin);
				check(NewNodeOutputPin);

				DataflowGraph->GetSchema()->TryCreateConnection(UpstreamNodeOutputPin, NewNodeInputPin);

				for (UEdGraphPin* DownstreamInputPin : ExistingNodeInputPins)
				{
					DataflowGraph->GetSchema()->TryCreateConnection(NewNodeOutputPin, DownstreamInputPin);
				}
			}

			DataflowGraph->NotifyGraphChanged();

			return NewEdNode;

		}
	}
	return nullptr;
}

const FName* UDataflowEditorMode::FindRecentlyUsedViewMode(const TArray<FName>& AvailableViewModes) const
{
	TArray<FName> Key = AvailableViewModes;
	Key.Sort([](const FName& A, const FName& B)
	{
		return A.FastLess(B);
	});
	return NodeViewModeHistory.Find(Key);
}

#undef LOCTEXT_NAMESPACE

