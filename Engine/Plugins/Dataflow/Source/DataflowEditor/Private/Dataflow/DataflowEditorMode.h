// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseCharacterFXEditorMode.h"
#include "DataflowEditorMode.generated.h"

#define UE_API DATAFLOWEDITOR_API

class FDataflowConstructionScene;
class SDataflowGraphEditor;
class FDataflowConstructionViewportClient;
class FDataflowSimulationScene;
class FDataflowSimulationViewportClient;
class UDataflowBaseContent;
class UDataflowEdNode;
class UDataflowEditorMode;
class UEdGraphNode;
class FViewport;
class UDataflowEditor;
class UInteractiveToolManager;
namespace UE::Dataflow
{
	class IDataflowConstructionViewMode;
	class FContext;

	// Process-wide singleton tool builders (held by FDataflowToolRegistry) can be shared across multiple simultaneously-open Dataflow editors. Storing a single
	// Mode pointer on such a builder would alias to whichever editor opened last, causing silent wrong-editor focus and dangling-pointer crashes 
	// when one editor closes while another's tool is still active. These helpers maintain a per-tool-manager binding so each editor's tool sees its own mode.
	using FToolBuilderModeMap = TMap<TWeakObjectPtr<const UInteractiveToolManager>, TWeakObjectPtr<UDataflowEditorMode>>;

	/** Insert InMode keyed by its tool manager, pruning any entries whose mode or tool manager have been GC'd. */
	DATAFLOWEDITOR_API void RegisterModeForToolManager(FToolBuilderModeMap& InOutMap, UDataflowEditorMode* InMode);

	/** Look up the mode bound to ToolManager, or nullptr if absent / stale. */
	DATAFLOWEDITOR_API UDataflowEditorMode* FindModeForToolManager(const FToolBuilderModeMap& InMap, const UInteractiveToolManager* ToolManager);
}
enum class EToolShutdownType : uint8;

/**
 * The dataflow editor mode is the mode used in the cloth asset editor. It holds most of the inter-tool state.
 * We put things in a mode instead of directly into the asset editor in case we want to someday use the mode
 * in multiple asset editors.
 */
UCLASS(MinimalAPI, Transient)
class UDataflowEditorMode final : public UBaseCharacterFXEditorMode
{
	GENERATED_BODY()

public:

	UE_API const static FEditorModeID EM_DataflowEditorModeId;

	UE_API UDataflowEditorMode();

	UE_API void SetDataflowEditor(UDataflowEditor* InDataflowEditor);
	TWeakPtr<FDataflowConstructionViewportClient, ESPMode::ThreadSafe> GetConstructionViewportClient() { return ConstructionViewportClient; }

	/**
	* Gets the scene bounding box
	*/
	UE_API virtual FBox SceneBoundingBox() const override;

	// Bounding box for selected rest space mesh components
	UE_API FBox SelectionBoundingBox() const;

	/**
	* Gets the tool target requirements for the mode.
	*/
	static UE_API const FToolTargetTypeRequirements& GetToolTargetRequirements();

	/**
	* Construction View Mode
	*/
	UE_API void SetConstructionViewMode(const FName& NewViewModeName);
	UE_API void SetConstructionViewModeForNode(const UDataflowEdNode* Node);

	UE_API const UE::Dataflow::IDataflowConstructionViewMode* GetConstructionViewMode() const;
	UE_API bool CanChangeConstructionViewModeTo(const FName& NewViewModeName) const;

	UE_API void ToggleConstructionViewWireframe();
	UE_API bool CanSetConstructionViewWireframeActive() const;
	bool IsConstructionViewWireframeActive() const
	{
		return bConstructionViewWireframe;
	}

	/** Set the construction scene to store the dynamic meshes generated by the tools */
	UE_API void SetDataflowConstructionScene(TWeakPtr<FDataflowConstructionScene> InConstructionScene);
	const TWeakPtr<FDataflowConstructionScene>& GetDataflowConstructionScene() { return ConstructionScene; }
	TWeakPtr<const FDataflowConstructionScene> GetDataflowConstructionScene() const { return ConstructionScene; }

	/** Set the simulation scene to store the simulation components */
	void SetDataflowSimlationScene(TWeakPtr<FDataflowSimulationScene> InSimulationScene) { SimulationScene = InSimulationScene; }
	const TWeakPtr<FDataflowSimulationScene>& GetDataflowSimulationScene() { return SimulationScene; }
	TWeakPtr<const FDataflowSimulationScene> GetDataflowSimulationScene() const { return SimulationScene; }

	/** Set the data flow graph editor to create nodes once the tools have ended*/
	UE_API void SetDataflowGraphEditor(TSharedPtr<SDataflowGraphEditor> DataflowGraphEditor);

	TObjectPtr<UEditorInteractiveToolsContext> GetActiveToolsContext()
	{
		return ActiveToolsContext;
	}

	UE_API void StartToolForSelectedNode(const UObject* SelectedNode);
	UE_API void ShutdownActiveToolIfNeeded(EToolShutdownType ShutdownType);

	UE_API void SetInputFocusOnConstructionViewport();

	/** Each node has a set of view modes it can use. For each distinct set of available view modes, we store the last mode that the user has chosen.
	 *  Use this function to find the most recently used view mode for nodes having the same set of available modes. 
	 *	Returns nullptr if there is nothing stored for the input set of modes 
	 */
	UE_API const FName* FindRecentlyUsedViewMode(const TArray<FName>& AvailableViewModes) const;

	UE_API bool IsAnyNodeSelected() const;
	UE_API TObjectPtr<UDataflowEdNode> GetSelectedNode() const;
	UE_API FName GetSelectedNodeTypeName() const;

	UE_API FText GetUserMessage() const { return UserMessage; }
	UE_API bool HasUserMessage() const { return !UserMessage.IsEmpty(); }

private:
	friend class FDataflowEditorToolkit;
	friend class FDataflowEditorModeToolkit;

	// UEdMode
	UE_API virtual void Enter() final;
	UE_API virtual void Exit() override;
	UE_API virtual void ModeTick(float DeltaTime) override;
	UE_API virtual bool ShouldToolStartBeAllowed(const FString& ToolIdentifier) const override;
	UE_API virtual void OnToolStarted(UInteractiveToolManager* Manager, UInteractiveTool* Tool) override;
	UE_API virtual void OnToolEnded(UInteractiveToolManager* Manager, UInteractiveTool* Tool) override;
	UE_API virtual void CreateToolkit() override;
	UE_API virtual void BindCommands() override;


	// UBaseCharacterFXEditorMode
	UE_API virtual void AddToolTargetFactories() override;
	UE_API virtual void RegisterTools() override;
	UE_API virtual void CreateToolTargets(const TArray<TObjectPtr<UObject>>& AssetsIn) override;
	UE_API virtual void InitializeTargets(const TArray<TObjectPtr<UObject>>& AssetsIn) override;

	// Use this function to register tools rather than UEdMode::RegisterTool() because we need to specify the ToolsContext
	UE_API void RegisterDataflowTool(TSharedPtr<FUICommandInfo> UICommand,
							  FString ToolIdentifier,
							  UInteractiveToolBuilder* Builder,
							  UEditorInteractiveToolsContext* const ToolsContext,
							  EToolsContextScope ToolScope = EToolsContextScope::Default);

	// Functions for quickly adding a node to the graph, e.g. when triggered by a toolbar button
	UE_API void AddNode(FName NewNodeType);
	UE_API bool CanAddNode(FName NewNodeType) const;

	UE_API void SetSimulationViewportClient(TWeakPtr<FDataflowSimulationViewportClient, ESPMode::ThreadSafe>);
	UE_API void RefocusSimulationViewportClient();
	UE_API void FirstTimeFocusSimulationViewport();

	UE_API void SetConstructionViewportClient(TWeakPtr<FDataflowConstructionViewportClient, ESPMode::ThreadSafe>);
	UE_API void RefocusConstructionViewportClient();
	UE_API void FirstTimeFocusConstructionViewport();
	UE_API bool IsComponentSelected(const UPrimitiveComponent* InComponent);
	UE_API void OnDataflowNodeDeleted(const TSet<UObject*>& DeletedNodes);

	/**
	* Return the single selected node in the Dataflow Graph Editor only if it has an output of the specified type
	* If there is not a single node selected, or if it does not have the specified output, return null
	*/
	UE_API UEdGraphNode* GetSingleSelectedNodeWithOutputType(const FName& SelectedNodeOutputTypeName) const;

	/**
	 * Create a node with the specified type in the graph
	*/
	UE_API UEdGraphNode* CreateNewNode(const FName& NewNodeTypeName);

	/** Create a node with the specified type, then connect it to the output of the specified UpstreamNode.
	* If the specified output of the upstream node is already connected to another node downstream, we first break
	* that connecttion, then insert the new node along the previous connection.
	* We want to turn this:
	*
	* [UpstreamNode] ----> [DownstreamNode(s)]
	*
	* to this:
	*
	* [UpstreamNode] ----> [NewNode] ----> [DownstreamNode(s)]
	*
	*
	* @param NewNodeTypeName The type of node to create, by name
	* @param UpstreamNode Node to connect the new node to
	* @param ConnectionTypeName The type of output of the upstream node to connect our new node to
	* @param NewNodeConnectionName The name of the input/output connection on our new node that will be connected
	* @return The newly-created node
	*/
	UE_API UEdGraphNode* CreateAndConnectNewNode(const FName& NewNodeTypeName, UEdGraphNode& UpstreamNode, const FName& ConnectionTypeName, const FName& NewNodeConnectionName);

	/** On all nodes in the graph, set whether the wireframe rendering toggle button is enabled */
	UE_API void SetWireframeRenderToggleEnabled(bool bEnable);

	UE_API void InitializeContextObject();
	UE_API void DeleteContextObject();

	void SetPendingNodeSelectionChanged(bool bIsPending)
	{
		bPendingNodeSelectionChanged = bIsPending;
	}

	/** Get the tool categories from the construction scene */
	UE_API const TArray<FName>& GetToolCategories();

	bool CanUseTools() const;
	bool CanSelectedNodeLaunchTool() const;
	bool IsSelectedNodeUpToDate(UE::Dataflow::FContext& Context) const;
	TSharedPtr<const FUICommandInfo> GetSelectedNodeToolCommand() const;
	bool IsDataflowEditable() const;

	TObjectPtr<UDataflowBaseContent> GetDataflowEditorContent() const;

	/** Dataflow construction scene from the toolkit */
	TWeakPtr<FDataflowConstructionScene> ConstructionScene;

	/** Dataflow simulation scene from the toolkit */
	TWeakPtr<FDataflowSimulationScene> SimulationScene;

	/** Correspondence between node types and commands to add the node to the graph */
	TMap<FName, TSharedPtr<const FUICommandInfo>> NodeTypeToAddNodeCommandMap;

	/** Correspondence between node types and commands to launch tools */
	TMap<FName, TSharedPtr<const FUICommandInfo>> NodeTypeToToolCommandMap;

	/** Graph editor is required to add nodes once some tools are complete */
	TWeakPtr<SDataflowGraphEditor> DataflowGraphEditor;

	// Points to a view mode owned by FRenderingViewModeFactory
	const UE::Dataflow::IDataflowConstructionViewMode* ConstructionViewMode = nullptr;

	// Whether we should restore the previous view mode when a tool ends
	bool bShouldRestoreSavedConstructionViewMode = false;
	
	// The Construction view mode that was active before starting the current tool. When the tool ends, restore this view mode if bShouldRestoreSavedConstructionViewMode is true.
	FName SavedConstructionViewMode;

	bool bConstructionViewWireframe = false;
	bool bShouldRestoreConstructionViewWireframe = false;

	UPROPERTY()
	TObjectPtr<UEditorInteractiveToolsContext> ActiveToolsContext = nullptr;

	TWeakPtr<FDataflowConstructionViewportClient, ESPMode::ThreadSafe> ConstructionViewportClient;
	TWeakPtr<FDataflowSimulationViewportClient, ESPMode::ThreadSafe> SimulationViewportClient;

	void OnConstructionSceneUpdated();

	// The first time we get a valid mesh, refocus the camera on it
	TSet<FName> FirstTimeRefocusedViews;

	// Whether the rest space viewport should focus on the rest space mesh on the next tick
	bool bShouldFocusConstructionView = true;
	bool bShouldFocusSimulationView = true;

	UE_API void ViewportResized(FViewport* Viewport, uint32 Unused);

	// Dataflow node type whose corresponding tool should be started on the next Tick
	FName NodeTypeForPendingToolStart;

	// If we should start the tool for the selected node on the next ModeTick call. Used when switching ViewModes to allow the scene to be rebuilt before restarting the tool.
	bool bShouldRestartToolNextTick = false;

	// Whether we have a single mesh selected in the Construction Scene prior to shutting down a running tool. The Scene will likely be rebuilt when the tool ends, resulting in
	// losing selection information, so we save this before shutting the tool down.
	bool bHadSingleSelectionBeforeToolShutdown = false;

	// Whether we are waiting for a node selection change to finish before starting a tool
	bool bPendingNodeSelectionChanged = false;

	UDataflowEditor* DataflowEditor = nullptr;

	// Timestamp for telemetry
	FDateTime LastModeStartTimestamp;

	// Message to display to the user in the viewport
	FText UserMessage;

	// Each node has a set of view modes it can use. For each distinct set of available view modes, we store the last mode that the user has chosen.
	// This will enable switching between different kinds of nodes and not have the view mode reset to a default every time.
	TMap<TArray<FName>, FName> NodeViewModeHistory;
};

#undef UE_API
