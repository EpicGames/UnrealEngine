// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowEditorModeToolkit.h"

#include "AssetEditorModeManager.h"
#include "Dataflow/DataflowEditorCommands.h"
#include "Dataflow/DataflowEditorMode.h"
#include "Dataflow/DataflowEditorStyle.h"
#include "Dataflow/DataflowConstructionViewport.h"
#include "Dataflow/DataflowEditorPreviewSceneBase.h"
#include "Dataflow/DataflowConstructionScene.h"
#include "DataflowRendering/DataflowRenderableTypeSettings.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "Framework/Application/SlateApplication.h"
#include "InteractiveToolManager.h"
#include "MeshAttributePaintTool.h"
#include "SBaseCharacterFXEditorViewport.h"
#include "SPrimaryButton.h"
#include "Tools/UEdMode.h"
#include "Widgets/Images/SImage.h"
#include "ToolMenus.h"
#include "Dataflow/DataflowToolRegistry.h"
#include "Toolkits/AssetEditorModeUILayer.h"

#define LOCTEXT_NAMESPACE "FDataflowEditorModeToolkit"

FName FDataflowEditorModeToolkit::GetToolkitFName() const
{
	return FName("DataflowEditorMode");
}

FText FDataflowEditorModeToolkit::GetBaseToolkitName() const
{
	return NSLOCTEXT("DataflowEditorModeToolkit", "DisplayName", "DataflowEditorMode");
}

void FDataflowEditorModeToolkit::BindCommands()
{
	if (GetScriptableEditorMode().IsValid())
	{
		UDataflowEditorMode* EditorMode = Cast<UDataflowEditorMode>(GetScriptableEditorMode());
		const TArray<FName> ToolCategories = EditorMode->GetToolCategories();

		UE::Dataflow::FDataflowToolRegistry& ToolRegistry = UE::Dataflow::FDataflowToolRegistry::Get();
		TArray<FName> NodeNames = ToolRegistry.GetNodeNames();

		for (const FName& NodeName : NodeNames)
		{
			ToolkitCommands->MapAction(ToolRegistry.GetAddNodeCommandForNode(NodeName),
				FExecuteAction::CreateUObject(EditorMode, &UDataflowEditorMode::AddNode, NodeName),
				FCanExecuteAction::CreateUObject(EditorMode, &UDataflowEditorMode::CanAddNode, NodeName));
		}
	}
}

void FDataflowEditorModeToolkit::Init(const TSharedPtr<IToolkitHost>& InitToolkitHost, TWeakObjectPtr<UEdMode> InOwningMode)
{
	bUsesToolkitBuilder = true;
	
	FModeToolkit::Init(InitToolkitHost, InOwningMode);

	BindCommands();

	ModeToolPallete.ModeWarningArea = SNew(STextBlock)
			.AutoWrapText(true)
			.Font(FCoreStyle::GetDefaultFontStyle("Bold", 9))
			.ColorAndOpacity(FSlateColor(FLinearColor(0.9f, 0.15f, 0.15f)));
	ModeToolPallete.ModeWarningArea->SetText(FText::GetEmpty());
	ModeToolPallete.ModeWarningArea->SetVisibility(EVisibility::Collapsed);

	ModeToolPallete.ModeHeaderArea = SNew(STextBlock)
		.AutoWrapText(true)
		.Font(FCoreStyle::GetDefaultFontStyle("Bold", 12));
	ModeToolPallete.ModeHeaderArea->SetText(LOCTEXT("SelectToolLabel", "Select a Tool from the Toolbar"));
	ModeToolPallete.ModeHeaderArea->SetJustification(ETextJustify::Center);
	
	ToolWarningArea = SNew(STextBlock)
		.AutoWrapText(true)
		.Font(FCoreStyle::GetDefaultFontStyle("Bold", 9))
		.ColorAndOpacity(FSlateColor(FLinearColor(0.9f, 0.15f, 0.15f)));
	ToolWarningArea->SetText(FText::GetEmpty());

	// Set up tool message areas
	RegisterPalettes();
	ClearNotification();
	ClearWarning();

	// create the toolkit widget
	{
		ToolkitSections->ModeWarningArea = ModeToolPallete.ModeWarningArea;
		ToolkitSections->DetailsView = ModeDetailsView;
		ToolkitSections->ToolWarningArea = ToolWarningArea;

		SAssignNew(ToolkitWidget, SBorder)
		.HAlign(HAlign_Fill)
		.Padding(0)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		[
			ToolkitBuilder->GenerateWidget()->AsShared()
		];	
	}

	ActiveToolName = FText::GetEmpty();

	if(UEditorInteractiveToolsContext* Context = GetScriptableEditorMode()->GetInteractiveToolsContext())
	{
		Context->OnToolNotificationMessage.AddSP(this, &FBaseCharacterFXEditorModeToolkit::PostNotification);
		Context->OnToolWarningMessage.AddSP(this, &FBaseCharacterFXEditorModeToolkit::PostWarning);
	}

	ModeDetailsView->OnFinishedChangingProperties().AddSP(this, &FDataflowEditorModeToolkit::OnSettingsPropertiesChanged);

	// add viewport overlay widget to accept / cancel tool
	MakeToolAcceptCancelWidget();
}

void FDataflowEditorModeToolkit::OnSettingsPropertiesChanged(const FPropertyChangedEvent& PropertyChangedEvent)
{
	if (ActiveToolName.IsEmpty())
	{
		RefreshConstructionScene();
	}
}

void FDataflowEditorModeToolkit::MakeToolAcceptCancelWidget()
{
	SAssignNew(ViewportOverlayWidget, SHorizontalBox)
	+SHorizontalBox::Slot()
	.HAlign(HAlign_Center)
	.VAlign(VAlign_Bottom)
	.Padding(FMargin(0.0f, 0.0f, 0.f, 15.f))
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("EditorViewport.OverlayBrush"))
		.Padding(8.f)
		[
			SNew(SHorizontalBox)

			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(FMargin(0.f, 0.f, 8.f, 0.f))
			[
				SNew(SImage)
				.Image_Lambda([this] () { return ActiveToolIcon; })
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(FMargin(0.f, 0.f, 8.f, 0.f))
			[
				SNew(STextBlock)
				.Text(this, &FBaseCharacterFXEditorModeToolkit::GetActiveToolDisplayName)
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(FMargin(0.0, 0.f, 2.f, 0.f))
			[
				SNew(SPrimaryButton)
				.Text(LOCTEXT("OverlayAccept", "Accept"))
				.ToolTipText(LOCTEXT("OverlayAcceptTooltip", "Accept/Commit the results of the active Tool [Enter]"))
				.OnClicked_Lambda([this]() { 
				    GetCurrentToolsContext()->EndTool(EToolShutdownType::Accept);
					return FReply::Handled(); 
					})
				.IsEnabled_Lambda([this]() { return GetCurrentToolsContext()->CanAcceptActiveTool(); })
				.Visibility_Lambda([this]() { return GetCurrentToolsContext()->ActiveToolHasAccept() ? EVisibility::Visible : EVisibility::Collapsed; })
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(FMargin(2.0, 0.f, 0.f, 0.f))
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "Button")
				.TextStyle( FAppStyle::Get(), "DialogButtonText" )
				.Text(LOCTEXT("OverlayCancel", "Cancel"))
				.ToolTipText(LOCTEXT("OverlayCancelTooltip", "Cancel the active Tool [Esc]"))
				.HAlign(HAlign_Center)
				.OnClicked_Lambda([this]() { 
					GetCurrentToolsContext()->EndTool(EToolShutdownType::Cancel);
					return FReply::Handled(); 
					})
				.IsEnabled_Lambda([this]() { return GetCurrentToolsContext()->CanCancelActiveTool(); })
				.Visibility_Lambda([this]() { return GetCurrentToolsContext()->ActiveToolHasAccept() ? EVisibility::Visible : EVisibility::Collapsed; })
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(FMargin(2.0, 0.f, 0.f, 0.f))
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "PrimaryButton")
				.TextStyle(FAppStyle::Get(), "DialogButtonText")
				.Text(LOCTEXT("OverlayComplete", "Complete"))
				.ToolTipText(LOCTEXT("OverlayCompleteTooltip", "Exit the active Tool [Enter]"))
				.HAlign(HAlign_Center)
				.OnClicked_Lambda([this]() { 
					GetCurrentToolsContext()->EndTool(EToolShutdownType::Completed);
					return FReply::Handled(); 
					})
				.IsEnabled_Lambda([this]() {
					return GetCurrentToolsContext()->CanCompleteActiveTool();
				})
				.Visibility_Lambda([this]() { return GetCurrentToolsContext()->CanCompleteActiveTool() ? EVisibility::Visible : EVisibility::Collapsed; })
			]
		]	
	];
}

void FDataflowEditorModeToolkit::RegisterPalettes()
{
	FDataflowEditorCommandsImpl& Commands = FDataflowEditorCommandsImpl::Get();
	const TSharedPtr<FUICommandList> CommandList = GetToolkitCommands();
	
	ToolkitSections = MakeShared<FToolkitSections>();

	FToolkitBuilderArgs ToolkitBuilderArgs(GetScriptableEditorMode()->GetModeInfo().ToolbarCustomizationName);
	ToolkitBuilderArgs.ToolkitCommandList = GetToolkitCommands();
	ToolkitBuilderArgs.ToolkitSections = ToolkitSections;
	ToolkitBuilderArgs.SelectedCategoryTitleVisibility = EVisibility::Collapsed;
	ToolkitBuilder = MakeShared<FToolkitBuilder>(ToolkitBuilderArgs);

	if (GetScriptableEditorMode().IsValid())
	{
		UDataflowEditorMode* EditorMode = Cast<UDataflowEditorMode>(GetScriptableEditorMode());
		const TArray<FName> ToolCategories = EditorMode->GetToolCategories();

		UE::Dataflow::FDataflowToolRegistry& ToolRegistry = UE::Dataflow::FDataflowToolRegistry::Get();
		TArray<FName> NodeNames = ToolRegistry.GetNodeNames();
		
		for(const FName& CategoryName : ToolCategories)
		{
			// Get all the tools that belongs to that category
			TArray<TSharedPtr<FUICommandInfo>> CategoryCommands;
			for (const FName& NodeName : NodeNames)
			{
				if(ToolRegistry.GetToolCategoryForNode(NodeName) == CategoryName)
				{
					const TSharedPtr<FUICommandInfo> CommandInfo = ToolRegistry.GetAddNodeCommandForNode(NodeName);
					CategoryCommands.Add(CommandInfo);
				}
			}
			// Add palette and the matching tools
			if(const TSharedPtr<FUICommandInfo>* CommandInfo = Commands.ToolPaletteCommands.Find(CategoryName))
			{
				ToolkitBuilder->AddPalette(MakeShareable(new FToolPalette((*CommandInfo).ToSharedRef(), CategoryCommands)));
			}
			else
			{
				TSharedPtr<FUICommandInfo> NewCommandInfo;
				const FText Description = FText::Format(LOCTEXT("DataflowToolCategoryDescription", "Tool category {0} used to register dataflow tools."), FText::FromName(CategoryName));
				
                FUICommandInfo::MakeCommandInfo(
                	Commands.AsShared(),
                	NewCommandInfo,
                	CategoryName, FText::FromName(CategoryName), Description, FSlateIcon(),
                	EUserInterfaceActionType::RadioButton,
                	FInputChord() );

				Commands.ToolPaletteCommands.Add(CategoryName, NewCommandInfo);

				ToolkitBuilder->AddPalette(MakeShareable(new FToolPalette(NewCommandInfo.ToSharedRef(), CategoryCommands)));
			}
		}

		// Select the first tool palette if available
		if (ToolCategories.Num() > 0)
		{
			if (const TSharedPtr<FUICommandInfo>* CommandInfo = Commands.ToolPaletteCommands.Find(ToolCategories[0]))
			{
				if (CommandInfo->IsValid())
				{
					ToolkitBuilder->SetActivePaletteOnLoad((*CommandInfo).Get());
				}
			}
		}
	}
	
	//ToolkitBuilder->SetActivePaletteOnLoad(Commands.LoadGeneralTools.Get());
	ToolkitBuilder->UpdateWidget();

	// if selected palette changes, make sure we are showing the palette command buttons, which may be hidden by active Tool
	ModeToolPallete.ActivePaletteChangedHandle = ToolkitBuilder->OnActivePaletteChanged.AddLambda([this]()
	{
		ToolkitBuilder->SetActivePaletteCommandsVisibility(EVisibility::Visible);
	});
}

const FSlateBrush* FDataflowEditorModeToolkit::GetActiveToolIcon(const FString& ActiveToolIdentifier) const
{
	FName ActiveToolIconName = ISlateStyle::Join(FDataflowEditorCommandsImpl::Get().GetContextName(), TCHAR_TO_ANSI(*ActiveToolIdentifier));
	return FDataflowEditorStyle::Get().GetOptionalBrush(ActiveToolIconName);
}

SBaseCharacterFXEditorViewport* FDataflowEditorModeToolkit::GetViewportWidgetForManager(UInteractiveToolManager* Manager)
{
	if (OwningEditorMode.IsValid(false))
	{
		const UEdMode* const Mode = OwningEditorMode.Get();

		if (const UDataflowEditorMode* const DataflowEdMode = Cast<UDataflowEditorMode>(Mode))
		{
			if (const TSharedPtr<const FDataflowPreviewSceneBase> PreviewScene = DataflowEdMode->GetDataflowConstructionScene().Pin())
			{
				if (const UEditorInteractiveToolsContext* const PreviewToolsContext = PreviewScene->GetDataflowModeManager()->GetInteractiveToolsContext())
				{
					const UInteractiveToolManager* const PreviewToolManager = PreviewToolsContext->ToolManager;

					//@todo(SimulationViewport) : Update to include simulation view when its added
					//if (Manager == PreviewToolManager)
					//{
					//	if (const TSharedPtr<SChaosClothAssetEditor3DViewport> Widget = PreviewViewportWidget.Pin())
					//	{
					//		return Widget.Get();
					//	}
					//}
					//else
					//{
					if (const TSharedPtr<SDataflowConstructionViewport> Widget = ConstructionViewportWidget.Pin())
					{
						return Widget.Get();
					}
					//}
				}
			}
		}
	}

	return nullptr;
}

UEditorInteractiveToolsContext* FDataflowEditorModeToolkit::GetCurrentToolsContext()
{
	if (OwningEditorMode.IsValid(/*bEvenIfPendingKill*/ false))
	{
		UEdMode* const Mode = OwningEditorMode.Get();
		if (UDataflowEditorMode* const DataflowEdMode = Cast<UDataflowEditorMode>(Mode))
		{
			return DataflowEdMode->GetActiveToolsContext();
		}
	}

	// this should not happen, but don't crash if it does
	return GetEditorModeManager().GetInteractiveToolsContext();
}

void FDataflowEditorModeToolkit::OnToolStarted(UInteractiveToolManager* Manager, UInteractiveTool* Tool)
{
	ensure(Tool == Manager->GetActiveTool(EToolSide::Left));

	if (Tool)
	{
		UpdateActiveToolProperties(Tool);

		Tool->OnPropertySetsModified.AddSP(this, &FDataflowEditorModeToolkit::UpdateActiveToolProperties, Tool);
		Tool->OnPropertyModifiedDirectlyByTool.AddSP(this, &FDataflowEditorModeToolkit::InvalidateCachedDetailPanelState);

		ModeToolPallete.ModeHeaderArea->SetVisibility(EVisibility::Collapsed);
		ActiveToolName = Tool->GetToolInfo().ToolDisplayName;

		FString ActiveToolIdentifier = Manager->GetActiveToolName(EToolSide::Mouse); 
		ActiveToolIdentifier.InsertAt(0, ".");
		ActiveToolIcon = GetActiveToolIcon(ActiveToolIdentifier);
		
		if (SBaseCharacterFXEditorViewport* Widget = GetViewportWidgetForManager(Manager))
		{
			Widget->AddOverlayWidget(ViewportOverlayWidget.ToSharedRef());

			// Manually set keyboard focus to this viewport. Otherwise the user has to click on the viewport before Tool keyboard shortcuts will work.
			FSlateApplication::Get().SetUserFocus(FSlateApplication::Get().GetUserIndexForKeyboard(), Widget->AsShared(), EFocusCause::SetDirectly);
		}
	}
}

void FDataflowEditorModeToolkit::OnToolEnded(UInteractiveToolManager* Manager, UInteractiveTool* Tool)
{
	ModeToolPallete.ModeHeaderArea->SetVisibility(EVisibility::Visible);
	UpdateModeDetailsView();
	ActiveToolName = FText::GetEmpty();
	ClearNotification();
	ClearWarning();

	if (SBaseCharacterFXEditorViewport* const Widget = GetViewportWidgetForManager(Manager))
	{
		Widget->RemoveOverlayWidget(ViewportOverlayWidget.ToSharedRef());
	}

	if (UInteractiveTool* const CurTool = Manager->GetActiveTool(EToolSide::Left))
	{
		CurTool->OnPropertySetsModified.RemoveAll(this);
		CurTool->OnPropertyModifiedDirectlyByTool.RemoveAll(this);
	}

	RefreshConstructionScene();
}

void FDataflowEditorModeToolkit::CustomizeModeDetailsViewArgs(FDetailsViewArgs& ArgsInOut)
{
	FBaseCharacterFXEditorModeToolkit::CustomizeModeDetailsViewArgs(ArgsInOut);
	ArgsInOut.bAllowSearch = true;
	ArgsInOut.bAllowFavoriteSystem = true;
}

void FDataflowEditorModeToolkit::RefreshConstructionScene()
{
	auto GetConstructionScene = [this]() -> TWeakPtr<FDataflowConstructionScene>
		{
			constexpr bool bEvenIfPendingKill = false;
			if (OwningEditorMode.IsValid(bEvenIfPendingKill))
			{
				UEdMode* const Mode = OwningEditorMode.Get();
				if (UDataflowEditorMode* const DataflowEdMode = Cast<UDataflowEditorMode>(Mode))
				{
					return DataflowEdMode->GetDataflowConstructionScene();
				}
			}
			return nullptr;
		};

	if (const TSharedPtr<FDataflowConstructionScene> ConstructionScene = GetConstructionScene().Pin())
	{
		if (const TObjectPtr<UDataflowBaseContent>& EditorContent = ConstructionScene->GetEditorContent())
		{
			EditorContent->SetConstructionDirty(true);
		}
	}
}

void FDataflowEditorModeToolkit::SetConstructionViewportWidget(TWeakPtr<SDataflowConstructionViewport> InConstructionViewportWidget)
{
	ConstructionViewportWidget = InConstructionViewportWidget;
}

void FDataflowEditorModeToolkit::SetSimulationViewportWidget(TWeakPtr<SDataflowSimulationViewport> InSimulationViewportWidget)
{
	SimulationViewportWidget = InSimulationViewportWidget;
}

void FDataflowEditorModeToolkit::OnRenderSettingsUpdated(const TArray<UDataflowRenderableTypeSettings*>& InRenderSettings)
{
	RenderSettings = InRenderSettings; 
	UpdateModeDetailsView();
}

void FDataflowEditorModeToolkit::UpdateModeDetailsView()
{
	if (ModeDetailsView)
	{
		TArray<UObject*> ObjectsToSet;
		if (UInteractiveTool* CurTool = GetScriptableEditorMode()->GetToolManager(EToolsContextScope::EdMode)->GetActiveTool(EToolSide::Left))
		{
			ObjectsToSet.Append(CurTool->GetToolProperties(true));
		}
		else
		{
			// if a tool is up , it has its own settings, so we only display the selection render settings when the tool is not up 
			ObjectsToSet.Append(RenderSettings);
		}

		ModeDetailsView->SetObjects(ObjectsToSet);
	}
}

void FDataflowEditorModeToolkit::UpdateActiveToolProperties(UInteractiveTool* Tool)
{
	UpdateModeDetailsView();
}

void FDataflowEditorModeToolkit::InvalidateCachedDetailPanelState(UObject* ChangedObject)
{
	ModeDetailsView->InvalidateCachedState();
}

#undef LOCTEXT_NAMESPACE
