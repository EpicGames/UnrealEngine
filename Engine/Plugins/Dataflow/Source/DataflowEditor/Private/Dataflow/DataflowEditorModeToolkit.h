// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseCharacterFXEditorModeToolkit.h"

#define UE_API DATAFLOWEDITOR_API

class UEditorInteractiveToolsContext;
class SDataflowConstructionViewport;
class SDataflowSimulationViewport;
class SBaseCharacterFXEditorViewport;
class UDataflowRenderableTypeSettings;

/** Tool pallete data for the dataflow mode toolkit */
struct FDataflowEditorModeToolPalette
{
	/**  Mode area to display the warning messages */ 
	TSharedPtr<STextBlock> ModeWarningArea;
	
	/**  Mode header for the tools names*/ 
	TSharedPtr<STextBlock> ModeHeaderArea;

	/** Too warning area*/
	TSharedPtr<STextBlock> ToolWarningArea;

	/** Handle when the active palette is changed */
	FDelegateHandle ActivePaletteChangedHandle;
};

/**
 * The dataflow editor mode toolkit is responsible for the panel on the side in the dataflow editor
 * that shows mode and tool properties. Tool buttons would go in Init().
 * Note: When there are separate viewports/worlds/modemanagers/toolscontexts, this ModeToolkit will track which
 * one is currently active.
 */

class FDataflowEditorModeToolkit : public FBaseCharacterFXEditorModeToolkit
{
public:
	UE_API void Init(const TSharedPtr<IToolkitHost>& InitToolkitHost, TWeakObjectPtr<UEdMode> InOwningMode);

	//~ Begin FBaseCharacterFXEditorModeToolkit interface
	UE_API virtual const FSlateBrush* GetActiveToolIcon(const FString& Identifier) const override;
	//~ End FBaseCharacterFXEditorModeToolkit interface

	//~ Begin FModeToolkit interface
	UE_API virtual void OnToolStarted(UInteractiveToolManager* Manager, UInteractiveTool* Tool) override;
	UE_API virtual void OnToolEnded(UInteractiveToolManager* Manager, UInteractiveTool* Tool) override;
	UE_API virtual void CustomizeModeDetailsViewArgs(FDetailsViewArgs& ArgsInOut) override;
	//~ End FModeToolkit interface

	//~ Begin IToolkit interface
	UE_API virtual FName GetToolkitFName() const override;
	UE_API virtual FText GetBaseToolkitName() const override;
	//~ End IToolkit interface

	UE_API void SetConstructionViewportWidget(TWeakPtr<SDataflowConstructionViewport>);
	UE_API void SetSimulationViewportWidget(TWeakPtr<SDataflowSimulationViewport>);

	void OnRenderSettingsUpdated(const TArray<UDataflowRenderableTypeSettings*>& RenderSettings);

private:

	/** Register the toolkit builder palettes */
	void RegisterPalettes();

	/** Make the accept/cancel overlay widget */
	void MakeToolAcceptCancelWidget();

	/** Update the active tool properties */
	void UpdateActiveToolProperties(UInteractiveTool* Tool);

	void UpdateModeDetailsView();

	/** Invalidate the cache state */
	void InvalidateCachedDetailPanelState(UObject* ChangedObject);

	/** Bind all the tools commands */
	void BindCommands();

	// Get the viewport widget associated with the given manager
	// TODO: This should not be necessary any more as we do not run tools in the Simulation Viewport (JIRA UE-201248)
	UE_API SBaseCharacterFXEditorViewport* GetViewportWidgetForManager(UInteractiveToolManager* Manager);

	/** Current tool context used to create nodes */
	UE_API UEditorInteractiveToolsContext* GetCurrentToolsContext();

	void OnSettingsPropertiesChanged(const FPropertyChangedEvent& PropertyChangedEvent);

	void RefreshConstructionScene();

	/** Construction viewport */
	TWeakPtr<SDataflowConstructionViewport> ConstructionViewportWidget;

	/** Simulation viewport*/
	TWeakPtr<SDataflowSimulationViewport> SimulationViewportWidget;

	/** Dataflow toolkit mode tool palette data */
	FDataflowEditorModeToolPalette ModeToolPallete;

	TArray<UDataflowRenderableTypeSettings*> RenderSettings;
};

#undef UE_API
