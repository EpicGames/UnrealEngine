// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowEditorModeUILayer.h"
#include "Dataflow/DataflowEditorToolkit.h"
#include "Dataflow/DataflowEditorModule.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"
#include "Toolkits/IToolkit.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowEditorModeUILayer)

const FName UDataflowEditorUISubsystem::EditorSidePanelAreaName = "DataflowEditorSidePanelArea";


void UDataflowEditorUISubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	FDataflowEditorModule& DataflowEditorModule = FModuleManager::GetModuleChecked<FDataflowEditorModule>("DataflowEditor");
	DataflowEditorModule.OnRegisterLayoutExtensions().AddUObject(this, &UDataflowEditorUISubsystem::RegisterLayoutExtensions);
}

void UDataflowEditorUISubsystem::Deinitialize()
{
	FDataflowEditorModule& DataflowEditorModule = FModuleManager::GetModuleChecked<FDataflowEditorModule>("DataflowEditor");
	DataflowEditorModule.OnRegisterLayoutExtensions().RemoveAll(this);
}

void UDataflowEditorUISubsystem::RegisterLayoutExtensions(FLayoutExtender& Extender)
{
	FTabManager::FTab NewTab(FTabId(UAssetEditorUISubsystem::TopLeftTabID), ETabState::ClosedTab);
	Extender.ExtendStack(UDataflowEditorUISubsystem::EditorSidePanelAreaName, ELayoutExtensionPosition::After, NewTab);
}


