// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowEditorModule.h"

#include "Dataflow/DataflowConstructionVisualization.h"
#include "Dataflow/DataflowEditorStyle.h"
#include "Dataflow/DataflowEditorMode.h"
#include "Dataflow/DataflowEditorToolkit.h"
#include "Dataflow/DataflowEditorCommands.h"
#include "Dataflow/DataflowEngineRendering.h"
#include "Dataflow/DataflowFreezeActionsCustomization.h"
#include "Dataflow/DataflowFunctionProperty.h"
#include "Dataflow/DataflowFunctionPropertyCustomization.h"
#include "Dataflow/DataflowInstance.h"
#include "Dataflow/DataflowSNodeFactories.h"
#include "Dataflow/DataflowSkeletalMeshAttachment.h"
#include "Dataflow/DebugDrawText3dVisualization.h"
#include "Dataflow/MeshStatsConstructionVisualization.h"
#include "Dataflow/ScalarVertexPropertyGroupCustomization.h"
#include "Dataflow/DataflowAnyTypeCustomization.h"
#include "Dataflow/DataflowToolRegistry.h"
#include "DataflowEditorTools/DataflowEditorWeightMapPaintTool.h"
#include "DataflowEditorTools/DataflowEditorVertexAttributePaintTool.h"
#include "Dataflow/DataflowCollectionAddScalarVertexPropertyNode.h"
#include "Dataflow/DataflowSelectionToolNode.h"
#include "DataflowEditorTools/DataflowMeshSelectionTool.h"

#include "Dataflow/DataflowInstanceDetails.h"

#include "PropertyEditorModule.h"
#include "EditorModeRegistry.h"
#include "Dataflow/DataflowCollectionEditSkinWeightsNode.h"
#include "Dataflow/DataflowCollectionEditSkeletonBonesNode.h"
#include "DataflowEditorTools/DataflowEditorSkinWeightsPaintTool.h"
#include "DataflowEditorTools/DataflowEditorEditSkeletonBonesTool.h"
#include "DataflowEditorTools/DataflowEditorCorrectSkinWeightsNode.h"
#include "Dataflow/DataflowColorRamp.h"
#include "Dataflow/DataflowColorRampCustomization.h"
#include "Dataflow/DataflowTemplateRegistry.h"

#include "DataflowRendering/DataflowStaticMeshRenderableType.h"
#include "DataflowRendering/DataflowSkeletalMeshRenderableType.h"
#include "DataflowRendering/DataflowDynamicMeshRenderableType.h"
#include "DataflowRendering/DataflowGeometryCollectionRenderableType.h"
#include "DataflowRendering/DataflowGeometryCollectionProximityRenderableType.h"
#include "DataflowRendering/DataflowPointRenderableType.h"
#include "DataflowRendering/DataflowPointArrayRenderableType.h"
#include "DataflowRendering/DataflowBoxRenderableType.h"
#include "DataflowRendering/DataflowSphereRenderableType.h"
#include "DataflowRendering/DataflowMaterialRenderableType.h"
#include "DataflowRendering/DataflowTextureRenderableType.h"
#include "DataflowRendering/DataflowSamplerRenderableType.h"
#include "DataflowRendering/DataflowPlaneRenderableType.h"
#include "DataflowRendering/DataflowPhysicsAssetRenderableType.h"

#include "Engine/SkeletalMesh.h"
#include "ContentBrowserModule.h"
#include "Modules/ModuleManager.h"
#include "Styling/SlateIconFinder.h"
#include "Widgets/Images/SImage.h"
#include "AssetViewWidgets.h"
#include "ISettingsModule.h"

#define LOCTEXT_NAMESPACE "DataflowEditor"

namespace UE::Dataflow::Private
{
	static const FName ScalarVertexPropertyGroupName = TEXT("ScalarVertexPropertyGroup");
	static const FName DataflowFunctionPropertyName = TEXT("DataflowFunctionProperty");
	static const FName DataflowVariableOverridesName = TEXT("DataflowVariableOverrides");
	static const FName DataflowFreezeActionsName = TEXT("DataflowFreezeActions");
	static const FName DataflowColorRampName = TEXT("DataflowColorRamp");
	static const FName DataflowLinearColorRampName = TEXT("DataflowLinearColorRamp");

	static const FName HasDataflowRegistryTag = TEXT("HasDataflow");
	static const FName IsDataflowEmbeddedRegistryTag = TEXT("IsDataflowEmbedded");

	class FDataflowEditorWeightMapPaintToolActionCommands : public TInteractiveToolCommands<FDataflowEditorWeightMapPaintToolActionCommands>
	{
	public:
		FDataflowEditorWeightMapPaintToolActionCommands() : 
			TInteractiveToolCommands<FDataflowEditorWeightMapPaintToolActionCommands>(
				TEXT("DataflowEditorWeightMapPaintToolContext"),
				LOCTEXT("DataflowEditorWeightMapPaintToolContext", "Dataflow Weight Map Paint Tool Context"),
				NAME_None,
				FAppStyle::GetAppStyleSetName())
		{}

		virtual void GetToolDefaultObjectList(TArray<UInteractiveTool*>& ToolCDOs) override
		{
			ToolCDOs.Add(GetMutableDefault<UDataflowEditorWeightMapPaintTool>());
		}
	};

	class FDataflowEditorSkinWeightPaintToolActionCommands : public TInteractiveToolCommands<FDataflowEditorSkinWeightPaintToolActionCommands>
	{
	public:
		FDataflowEditorSkinWeightPaintToolActionCommands() : 
			TInteractiveToolCommands<FDataflowEditorSkinWeightPaintToolActionCommands>(
				TEXT("DataflowEditorSkinWeightPaintToolContext"),
				LOCTEXT("DataflowEditorSkinWeightPaintToolContext", "Dataflow Skin weight Paint Tool Context"),
				NAME_None,
				FAppStyle::GetAppStyleSetName())
		{}

		virtual void GetToolDefaultObjectList(TArray<UInteractiveTool*>& ToolCDOs) override
		{
			ToolCDOs.Add(GetMutableDefault<UDataflowEditorSkinWeightsPaintTool>());
		}
	};

	class FDataflowEditorEditSkeletonBonesToolActionCommands : public TInteractiveToolCommands<FDataflowEditorEditSkeletonBonesToolActionCommands>
	{
	public:
		FDataflowEditorEditSkeletonBonesToolActionCommands() : 
			TInteractiveToolCommands<FDataflowEditorEditSkeletonBonesToolActionCommands>(
				TEXT("DataflowEditorEditSkeletonBonesToolContext"),
				LOCTEXT("DataflowEditorEditSkeletonBonesToolContext", "Dataflow skeleton edit Tool Context"),
				NAME_None,
				FAppStyle::GetAppStyleSetName())
		{}

		virtual void GetToolDefaultObjectList(TArray<UInteractiveTool*>& ToolCDOs) override
		{
			ToolCDOs.Add(GetMutableDefault<UDataflowEditorEditSkeletonBonesTool>());
		}
	};

	class FDataflowEditorVertexAttributePaintToolActionCommands : public TInteractiveToolCommands<FDataflowEditorVertexAttributePaintToolActionCommands>
	{
	public:
		FDataflowEditorVertexAttributePaintToolActionCommands() :
			TInteractiveToolCommands<FDataflowEditorVertexAttributePaintToolActionCommands>(
				TEXT("DataflowEditorVertexAttributePaintTooContext"),
				LOCTEXT("DataflowEditorVertexAttributePaintTooContext", "Dataflow Vertex Attribute Paint Tool Context"),
				NAME_None,
				FAppStyle::GetAppStyleSetName())
		{}

		virtual void GetToolDefaultObjectList(TArray<UInteractiveTool*>& ToolCDOs) override
		{
			ToolCDOs.Add(GetMutableDefault<UDataflowEditorVertexAttributePaintTool>());
		}
	};

	class FDataflowToolActionCommandBindings : public UE::Dataflow::FDataflowToolRegistry::IDataflowToolActionCommands
	{
	public:
		FDataflowToolActionCommandBindings()
		{
			FDataflowEditorWeightMapPaintToolActionCommands::Register();
			FDataflowEditorSkinWeightPaintToolActionCommands::Register();
			FDataflowEditorEditSkeletonBonesToolActionCommands::Register();
			FDataflowEditorVertexAttributePaintToolActionCommands::Register();
		}

		virtual void UnbindActiveCommands(const TSharedPtr<FUICommandList>& UICommandList) const override
		{
			checkf(FDataflowEditorWeightMapPaintToolActionCommands::IsRegistered(), TEXT("Expected WeightMapPaintTool actions to have been registered"));
			FDataflowEditorWeightMapPaintToolActionCommands::Get().UnbindActiveCommands(UICommandList);

			checkf(FDataflowEditorSkinWeightPaintToolActionCommands::IsRegistered(), TEXT("Expected SkinWeightPaintTool actions to have been registered"));
			FDataflowEditorSkinWeightPaintToolActionCommands::Get().UnbindActiveCommands(UICommandList);

			checkf(FDataflowEditorEditSkeletonBonesToolActionCommands::IsRegistered(), TEXT("Expected SkeletonEditTool actions to have been registered"));
			FDataflowEditorEditSkeletonBonesToolActionCommands::Get().UnbindActiveCommands(UICommandList);

			checkf(FDataflowEditorVertexAttributePaintToolActionCommands::IsRegistered(), TEXT("Expected VertexAttributePaintTool actions to have been registered"));
			FDataflowEditorVertexAttributePaintToolActionCommands::Get().UnbindActiveCommands(UICommandList);
		}

		virtual void BindCommandsForCurrentTool(const TSharedPtr<FUICommandList>& UICommandList, UInteractiveTool* Tool) const override
		{
			if (ExactCast<UDataflowEditorWeightMapPaintTool>(Tool))
			{
				checkf(FDataflowEditorWeightMapPaintToolActionCommands::IsRegistered(), TEXT("Expected WeightMapPaintTool actions to have been registered"));
				FDataflowEditorWeightMapPaintToolActionCommands::Get().BindCommandsForCurrentTool(UICommandList, Tool);
			}
			if (ExactCast<UDataflowEditorSkinWeightsPaintTool>(Tool))
			{
				checkf(FDataflowEditorSkinWeightPaintToolActionCommands::IsRegistered(), TEXT("Expected SkinWeightPaintTool actions to have been registered"));
				FDataflowEditorSkinWeightPaintToolActionCommands::Get().BindCommandsForCurrentTool(UICommandList, Tool);
			}
			if (ExactCast<UDataflowEditorEditSkeletonBonesTool>(Tool))
			{
				checkf(FDataflowEditorEditSkeletonBonesToolActionCommands::IsRegistered(), TEXT("Expected SkeletonEditTool actions to have been registered"));
				FDataflowEditorEditSkeletonBonesToolActionCommands::Get().BindCommandsForCurrentTool(UICommandList, Tool);
			}
			if (ExactCast<UDataflowEditorVertexAttributePaintTool>(Tool))
			{
				checkf(FDataflowEditorVertexAttributePaintToolActionCommands::IsRegistered(), TEXT("Expected VertexAttributePaintTool actions to have been registered"));
				FDataflowEditorVertexAttributePaintToolActionCommands::Get().BindCommandsForCurrentTool(UICommandList, Tool);
			}
		}
	};

	TSharedRef<SWidget> OnGenerateDataflowAssetBindingIcons(const FAssetData& AssetData)
	{
		bool bHasDataflowAsset = false;
		AssetData.GetTagValue(HasDataflowRegistryTag, bHasDataflowAsset);
		bool bIsDataflowEmbedded = false;
		AssetData.GetTagValue(IsDataflowEmbeddedRegistryTag, bIsDataflowEmbedded);

		if (bHasDataflowAsset)
		{
			const FSlateBrush* DataflowAssetIconBrush = FSlateIcon(FAppStyle::GetAppStyleSetName(), "GraphEditor.EventGraph_16x").GetIcon();

			// Wrap the icon in an outer background layer to improve visibility on light thumbnails
			constexpr float Padding = 4.f;
			constexpr float OuterBoxSize = 16.f;
			TSharedRef<SWidget> BorderWidget = SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("AssetThumbnail", ".AssetThumbnailStatusBar"))
				[
					SNew(SBox)
					.WidthOverride(OuterBoxSize)
					.HeightOverride(OuterBoxSize)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					[
						SNew(SImage)
						.Image(DataflowAssetIconBrush)
						.ColorAndOpacity(bIsDataflowEmbedded
							? FSlateColor(FLinearColor::White)
							: FSlateColor(FLinearColor::Yellow)
						)
					]
				];

			// make sure we do not show the dataflow icon if the thumbnail is a list item icon
			// to avoid convering the actual thumbnail 
			auto  VisibilityBasedOnSizeLambda = [BorderWidget]() -> EVisibility
				{
					TSharedPtr<SWidget> Parent = BorderWidget->GetParentWidget();
					while (Parent)
					{
						if (Parent->GetType() == SAssetListViewRow::StaticWidgetClass().GetWidgetType())
						{
							if (Parent->GetDesiredSize().Y < OuterBoxSize * 3.f)
							{
								return EVisibility::Collapsed;
							}
						}
						Parent = Parent->GetParentWidget();
					}
					return EVisibility::Visible;
				};

			return SNew(SBox)
				.Padding(Padding, Padding, Padding, Padding)
				.Visibility_Lambda(MoveTemp(VisibilityBasedOnSizeLambda))
				[
					BorderWidget
				];
		}
		return SNullWidget::NullWidget;
	}

	TSharedRef<SWidget> OnGenerateDataflowAssetBindingTooltip(const FAssetData& AssetData)
	{
		bool bHasDataflowAsset = false;
		AssetData.GetTagValue(HasDataflowRegistryTag, bHasDataflowAsset);

		bool bIsDataflowEmbedded = false;
		AssetData.GetTagValue(IsDataflowEmbeddedRegistryTag, bIsDataflowEmbedded);

		if (bHasDataflowAsset)
		{
			FText TooltipText;
			if (bIsDataflowEmbedded)
			{
				TooltipText = LOCTEXT("DataflowIndicatorTooltip_EmbeddedText", "Embedded Dataflow Graph");
			}
			else
			{
				
				const FText DataflowAssetPathName = FText::FromString(AssetData.GetObjectPathString());
				TooltipText = FText::Format(LOCTEXT("DataflowIndicatorTooltip_Format", "Shared Dataflow Asset : {0}"), DataflowAssetPathName);
			}

			const float Padding = 8.f;
			const float OuterBoxSize = 16.f;
			return SNew(STextBlock)
				.Text(TooltipText)
				.Font(FAppStyle::GetFontStyle("ContentBrowser.Tooltip.EntryFont"))
				;
		}
		return SNullWidget::NullWidget;
	}
}

const FColor FDataflowEditorModule::SurfaceColor = FLinearColor(0.6f, 0.6f, 0.6f).ToRGBE();

void FDataflowEditorModule::StartupModule()
{
	using namespace UE::Dataflow::Private;

#if WITH_EDITOR
	// register settings
	if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
	{
		SettingsModule->RegisterSettings("Editor", "ContentEditors", "DataflowEditor",
			LOCTEXT("DataflowEditorLabel", "Dataflow Editor"),
			LOCTEXT("DataflowEditorDesc", "Dataflow Editor Settings"),
			GetMutableDefault<UDataflowEditorUserSettings>()
		);
	}
#endif //WITH_EDITOR


	FDataflowEditorStyle::Get();

	// Register type customizations
	if (FPropertyEditorModule* const PropertyModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor"))
	{
		PropertyModule->RegisterCustomPropertyTypeLayout(ScalarVertexPropertyGroupName, FOnGetPropertyTypeCustomizationInstance::CreateStatic(&UE::Dataflow::FScalarVertexPropertyGroupCustomization::MakeInstance));
		PropertyModule->RegisterCustomPropertyTypeLayout(DataflowFunctionPropertyName, FOnGetPropertyTypeCustomizationInstance::CreateStatic(&UE::Dataflow::FFunctionPropertyCustomization::MakeInstance));
		PropertyModule->RegisterCustomPropertyTypeLayout(DataflowVariableOverridesName, FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FDataflowVariableOverridesDetails::MakeInstance));
		PropertyModule->RegisterCustomPropertyTypeLayout(DataflowFreezeActionsName, FOnGetPropertyTypeCustomizationInstance::CreateStatic(&UE::Dataflow::FFreezeActionsCustomization::MakeInstance));

		PropertyModule->RegisterCustomPropertyTypeLayout(DataflowColorRampName, FOnGetPropertyTypeCustomizationInstance::CreateStatic(&UE::Dataflow::FDataflowColorRampCustomization::MakeInstance));
		PropertyModule->RegisterCustomPropertyTypeLayout(DataflowLinearColorRampName, FOnGetPropertyTypeCustomizationInstance::CreateStatic(&UE::Dataflow::FDataflowLinearColorRampCustomization::MakeInstance));

		UE::Dataflow::RegisterAnyTypeCustomizations(*PropertyModule);
	}

	UE::Dataflow::RenderingCallbacks();

	UE::Dataflow::FDataflowToolRegistry& ToolRegistry = UE::Dataflow::FDataflowToolRegistry::Get();
	TSharedRef<const FDataflowToolActionCommandBindings> Actions = MakeShared<FDataflowToolActionCommandBindings>();

	ToolRegistry.AddNodeToToolMapping(FDataflowCollectionAddScalarVertexPropertyNode::StaticType(),
		NewObject<UDataflowEditorVertexAttributePaintToolBuilder>(), Actions,
		FSlateIcon(FName("DataflowEditorStyle"), FName("Dataflow.PaintWeightMap")), LOCTEXT("AddWeightMapNodeButtonText", "Paint Weight Map"));

	ToolRegistry.AddNodeToToolMapping(FDataflowCollectionEditSkinWeightsNode::StaticType(),
		NewObject<UDataflowEditorSkinWeightsPaintToolBuilder>(), Actions,
		FSlateIcon(FName("DataflowEditorStyle"), FName("Dataflow.EditSkinWeights")), LOCTEXT("AddSkinWeightNodeButtonText", "Edit Skin Weights"));

	ToolRegistry.AddNodeToToolMapping(FDataflowCollectionEditSkeletonBonesNode::StaticType(),
		NewObject<UDataflowEditorEditSkeletonBonesToolBuilder>(), Actions,
		FSlateIcon(FName("DataflowEditorStyle"), FName("Dataflow.EditSkeletonBones")), LOCTEXT("AddSkeletonEditNodeButtonText", "Edit Skeleton Bones"));

	ToolRegistry.AddNodeToToolMapping(FDataflowSelectionToolNode::StaticType(),
		NewObject<UDataflowMeshSelectionToolBuilder>(), Actions,
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.StrictBoxSelect"), LOCTEXT("AddSelectionNodeButtonText", "Selection"));

	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowCorrectSkinWeightsNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowGetSkinningSelectionNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowSetSkinningSelectionNode);

	UE::Dataflow::FDataflowConstructionVisualizationRegistry& ConstructionVisualizationRegistry = UE::Dataflow::FDataflowConstructionVisualizationRegistry::GetInstance();
	ConstructionVisualizationRegistry.RegisterVisualization(MakeUnique<UE::Dataflow::FMeshStatsConstructionVisualization>());
	ConstructionVisualizationRegistry.RegisterVisualization(MakeUnique<UE::Dataflow::FDebugDrawText3dVisualization>());

	FDataflowEditorCommands::Register();

	// register rendering types
	UE::Dataflow::Private::RegisterStaticMeshRenderableTypes();
	UE::Dataflow::Private::RegisterSkeletalMeshRenderableTypes();
	UE::Dataflow::Private::RegisterDynamicMeshRenderableTypes();
	UE::Dataflow::Private::RegisterGeometryCollectionRenderableTypes();
	UE::Dataflow::Private::RegisterGeometryCollectionProximityRenderableTypes();
	UE::Dataflow::Private::RegisterPointRenderableTypes();
	UE::Dataflow::Private::RegisterPointArrayRenderableTypes();
	UE::Dataflow::Private::RegisterBoxRenderableTypes();
	UE::Dataflow::Private::RegisterSphereRenderableTypes();
	UE::Dataflow::Private::RegisterMaterialRenderableTypes();
	UE::Dataflow::Private::RegisterTextureRenderableTypes();
	UE::Dataflow::Private::RegisterSamplerRenderableTypes();
	UE::Dataflow::Private::RegisterPlaneRenderableTypes();
	UE::Dataflow::Private::RegisterPhysicsAssetRenderableTypes();

	// register an browser extension to show when an asset is bound to a dataflow asset
	if (FContentBrowserModule* ContentBrowserModule = FModuleManager::Get().GetModulePtr<FContentBrowserModule>(TEXT("ContentBrowser")))
	{
		AssetViewExtraStateGeneratorHandle = ContentBrowserModule->AddAssetViewExtraStateGenerator(FAssetViewExtraStateGenerator(
			FOnGenerateAssetViewExtraStateIndicators::CreateStatic(&UE::Dataflow::Private::OnGenerateDataflowAssetBindingIcons),
			FOnGenerateAssetViewExtraStateIndicators::CreateStatic(&UE::Dataflow::Private::OnGenerateDataflowAssetBindingTooltip)
		));
	}

	FDataflowAttachmentFactory::Get().Register(USkeletalMesh::StaticClass()->GetFName(), [](UObject* Owner) { return NewObject<UDataflowSkeletalMeshAttachment>(Owner); });
	SkeletalMeshDataflowMenusHandle = UE::DataflowAssetDefinitionHelpers::RegisterDataflowAssetMenus(USkeletalMesh::StaticClass());

	const FSlateIcon SkeletalMeshIcon(FAppStyle::GetAppStyleSetName(), "Icons.SkeletalMesh");

	// Mesh to skeletal mesh ( via Medial Skeleton )
	FDataflowTemplateRegistry::Get().RegisterTemplateAsset(
		USkeletalMesh::StaticClass(),
		{
			.AssetPath = FSoftObjectPath(TEXT("/Dataflow/Templates/SKM/DF_SKM_FromMesh_Template.DF_SKM_FromMesh_Template")),
			.DisplayName = LOCTEXT("Dataflow_SkeletalMesh_FromMesh_Template_Label", "Skeletonizer Template"),
			.Tooltip = LOCTEXT("Dataflow_SkeletalMesh_FromMesh_Template_Tooltip", "Dataflow graph that procedurally generates a skeleton mesh and associated skeleton from a static mesh using medial skeleton nodes"),
			.Icon = SkeletalMeshIcon
		});

	// Skeleton mesh subdivision
	FDataflowTemplateRegistry::Get().RegisterTemplateAsset(
		USkeletalMesh::StaticClass(),
		{
			.AssetPath = FSoftObjectPath(TEXT("/Dataflow/Templates/SKM/DF_SKM_SubD_Template.DF_SKM_SubD_Template")),
			.DisplayName = LOCTEXT("Dataflow_SkeletalMesh_Subd_Template_Label", "Subdivision Template"),
			.Tooltip = LOCTEXT("Dataflow_SkeletalMesh_Subd_Template_Tooltip", "Dataflow graph that subdivides a base skeletal mesh into a high resolution one. It is recommended to set this Dataflow on a copy of the base skeletal mesh you are trying to subdivide"),
			 .Icon = SkeletalMeshIcon
		});

	// Skeleton mesh from mesh splitting
	FDataflowTemplateRegistry::Get().RegisterTemplateAsset(
		USkeletalMesh::StaticClass(),
		{
			.AssetPath = FSoftObjectPath(TEXT("/Dataflow/Templates/SKM/DF_SKM_MeshSplit_Template.DF_SKM_MeshSplit_Template")),
			.DisplayName = LOCTEXT("Dataflow_SkeletalMesh_MeshSplit_Template_Label", "Mesh Split Template"),
			.Tooltip = LOCTEXT("Dataflow_SkeletalMesh_MeshSplit_Template_Tooltip", "Dataflow graph that takes a static mesh, split it by mesh island and create a skeletal mesh and skeleton out of it, where each part of the mesh is a assigned a unique bone"),
			 .Icon = SkeletalMeshIcon
		});

	// reimport from mesh 
	FDataflowTemplateRegistry::Get().RegisterTemplateAsset(
		USkeletalMesh::StaticClass(),
		{
			.AssetPath = FSoftObjectPath(TEXT("/Dataflow/Templates/SKM/DF_SKM_ReimportFromMesh_Template.DF_SKM_ReimportFromMesh_Template")),
			.DisplayName = LOCTEXT("Dataflow_SkeletalMesh_ReimportMesh_Template_Label", "Re-Import From Static Mesh Template"),
			.Tooltip = LOCTEXT("Dataflow_SkeletalMesh_ReimportMesh_Template_Tooltip", "Dataflow graph that update the topology of a skeletal mesh from a static mesh ( attributes and materials are transfered over ) \n IMPORTANT : once the dataflow is assigned and configured, use the 'Asset context menu > Scripted Asset Actions > Reimport from mesh' to update the skeletal mesh"),
			 .Icon = SkeletalMeshIcon
		});
}

void FDataflowEditorModule::ShutdownModule()
{
	using namespace UE::Dataflow::Private;

	UE::DataflowAssetDefinitionHelpers::UnregisterDataflowAssetMenus(SkeletalMeshDataflowMenusHandle);

	if (FContentBrowserModule* ContentBrowserModule = FModuleManager::Get().GetModulePtr<FContentBrowserModule>(TEXT("ContentBrowser")))
	{
		if (AssetViewExtraStateGeneratorHandle.IsValid())
		{
			ContentBrowserModule->RemoveAssetViewExtraStateGenerator(AssetViewExtraStateGeneratorHandle);
		}
	}

	FEditorModeRegistry::Get().UnregisterMode(UDataflowEditorMode::EM_DataflowEditorModeId);

	// Deregister type customizations
	if (FPropertyEditorModule* const PropertyModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor"))
	{
		PropertyModule->UnregisterCustomPropertyTypeLayout(ScalarVertexPropertyGroupName);
		PropertyModule->UnregisterCustomPropertyTypeLayout(DataflowFunctionPropertyName);
		PropertyModule->UnregisterCustomPropertyTypeLayout(DataflowVariableOverridesName);
		PropertyModule->UnregisterCustomPropertyTypeLayout(DataflowFreezeActionsName);
		PropertyModule->UnregisterCustomPropertyTypeLayout(DataflowColorRampName);
		PropertyModule->UnregisterCustomPropertyTypeLayout(DataflowLinearColorRampName);

		UE::Dataflow::UnregisterAnyTypeCustomizations(*PropertyModule);
	}

	FDataflowEditorCommands::Unregister();

	UE::Dataflow::FDataflowToolRegistry& ToolRegistry = UE::Dataflow::FDataflowToolRegistry::Get();
	ToolRegistry.RemoveNodeToToolMapping(FDataflowCollectionAddScalarVertexPropertyNode::StaticType());
	ToolRegistry.RemoveNodeToToolMapping(FDataflowCollectionEditSkinWeightsNode::StaticType());
	ToolRegistry.RemoveNodeToToolMapping(FDataflowCollectionEditSkeletonBonesNode::StaticType());
	ToolRegistry.RemoveNodeToToolMapping(FDataflowSelectionToolNode::StaticType());

	UE::Dataflow::FDataflowConstructionVisualizationRegistry& ConstructionVisualizationRegistry = UE::Dataflow::FDataflowConstructionVisualizationRegistry::GetInstance();
	ConstructionVisualizationRegistry.DeregisterVisualization(UE::Dataflow::FMeshStatsConstructionVisualization::Name);
	ConstructionVisualizationRegistry.DeregisterVisualization(UE::Dataflow::FDebugDrawText3dVisualization::Name);

	if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
	{
		SettingsModule->UnregisterSettings("Editor", "ContentEditors", "DataflowEditor");
	}
}

IMPLEMENT_MODULE(FDataflowEditorModule, DataflowEditor)

#undef LOCTEXT_NAMESPACE
