// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowEditorOptions.h"
#include "AssetViewerSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowEditorOptions)

UDataflowEditorOptions::UDataflowEditorOptions(class FObjectInitializer const& ObjectInitializer)
{
	// Construction
	ConstructionViewFOV = 75.0f;
	// The construction viewport often becomes overexposed in 2D with auto exposure turned on especially with the "Grey Wireframe" profile active, so we default to fixed
	bConstructionViewFixedExposure = true;
	ConstructionProfileName = UDefaultEditorProfiles::EditingProfileName.ToString();

	// Simulation
	SimulationViewFOV = 75.0f;
	bSimulationViewFixedExposure = false;
	SimulationProfileName = UDefaultEditorProfiles::GreyAmbientProfileName.ToString();
}

FName UDataflowEditorOptions::GetCategoryName() const
{
	return TEXT("Plugins");
}

#if WITH_EDITOR

FText UDataflowEditorOptions::GetSectionText() const
{
	return NSLOCTEXT("DataflowEditorPlugin", "DataflowEditorSettingsSection", "Dataflow Editor");
}

#endif
