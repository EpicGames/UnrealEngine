// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "Engine/DeveloperSettings.h"
#include "Settings/EditorViewportSettings.h"
#include "DataflowEditorOptions.generated.h"

UENUM(BlueprintType)
enum class EDataflowConstructionViewportMousePanButton : uint8
{
	/** Right Mouse Button */
	Right = 0,

	/** Middle Mouse Button */
	Middle = 1,

	/** Either Right or Middle Mouse Button */
	RightOrMiddle = 2,
};

UENUM(BlueprintType)
enum class EDataflowEditorEvaluationMode : uint8
{
	/** Dataflow graph will evaluate automatically when values are changed */
	Automatic = 0,

	/** Dataflow graph will not eveluate until the user presses the evaluate button in the editor */
	Manual = 1,
};

/** A settings class used to store and retreive user settings related to the Dataflow Editor */
UCLASS(HideCategories = Object, Config = EditorPerProjectUserSettings, Meta = (DisplayName = "Dataflow Editor"), MinimalAPI)
class UDataflowEditorOptions : public UDeveloperSettings
{
	GENERATED_UCLASS_BODY()

	/** FOV for the Construction viewport camera */
	UPROPERTY(config)
	float ConstructionViewFOV;

	/** FOV for the Simulation viewport camera */
	UPROPERTY(config)
	float SimulationViewFOV;

	/** Whether the Construction viewport has Auto or Fixed exposure */
	UPROPERTY(config)
	bool bConstructionViewFixedExposure;

	/** Whether the Simulation viewport has Auto or Fixed exposure */
	UPROPERTY(config)
	bool bSimulationViewFixedExposure;

	/** Preview Scene Profile for the Construction viewport */
	UPROPERTY(config)
	FString ConstructionProfileName;

	/** Preview Scene Profile for the Simulation viewport */
	UPROPERTY(config)
	FString SimulationProfileName;

	/** The default camera speed settings for the viewport */
	UPROPERTY(config)
	FEditorViewportCameraSpeedSettings ConstructionCameraSpeedSettings;

	UPROPERTY(config)
	FEditorViewportCameraSpeedSettings SimulationCameraSpeedSettings;

	/** Which mouse button controls camera panning in the Construction Viewport in 2D view mode */
	UPROPERTY(Config, EditAnywhere, Category = UI)
	EDataflowConstructionViewportMousePanButton ConstructionViewportMousePanButton;

	UPROPERTY(Config, EditAnywhere, Category = UI)
	EDataflowEditorEvaluationMode EditorEvaluationMode = EDataflowEditorEvaluationMode::Automatic;

	// UDeveloperSettings overrides
	virtual FName GetCategoryName() const override;

#if WITH_EDITOR
	virtual FText GetSectionText() const override;
#endif

};
