// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowEditorPreviewSceneBase.h"
#include "AssetEditorModeManager.h"
#include "Dataflow/DataflowEditor.h"
#include "Components/PrimitiveComponent.h"
#include "Elements/Framework/EngineElementsLibrary.h"
#include "Selection.h"
#include "AssetViewerSettings.h"
#include "Dataflow/DataflowElement.h"
#include "DataStorage/Features.h"
#include "Elements/Columns/TypedElementHiearchyColumns.h"
#include "Elements/Columns/TypedElementLabelColumns.h"
#include "Elements/Interfaces/TypedElementDataStorageCompatibilityInterface.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Columns/TypedElementSelectionColumns.h"
#include "Elements/Columns/TypedElementViewportColumns.h"
#include "Elements/Columns/TypedElementVisibilityColumns.h"
#include "Dataflow/DataflowDebugDrawComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowEditorPreviewSceneBase)

#define LOCTEXT_NAMESPACE "FDataflowPreviewSceneBase"


namespace UE::Dataflow::Private
{
	static void AddObjectColumns(UE::Editor::DataStorage::ICoreProvider* DataStorage, const UE::Editor::DataStorage::RowHandle& Row, const bool bIsConstruction)
	{
		if(bIsConstruction)
		{
			DataStorage->AddColumn<FDataflowConstructionObjectTag>(Row);
		}
		else
		{
			DataStorage->AddColumn<FDataflowSimulationObjectTag>(Row);
		}
		DataStorage->AddColumn<FTypedElementSyncFromWorldTag>(Row);
		DataStorage->AddColumn<FTypedElementLabelColumn>(Row);
		DataStorage->AddColumn<FTypedElementLabelHashColumn>(Row);
		DataStorage->AddColumn<FVisibleInEditorColumn>(Row);
		DataStorage->AddColumn<FTableRowParentColumn>(Row);
		DataStorage->AddColumn<FDataflowSceneTypeColumn>(Row);
	}
}

bool bDataflowShowFloorDefault = true;
FAutoConsoleVariableRef CVARDataflowShowFloorDefault(TEXT("p.Dataflow.Editor.ShowFloor"), bDataflowShowFloorDefault, TEXT("Show the floor in the dataflow editor[def:false]"));

bool bDataflowShowEnvironmentDefault = true;
FAutoConsoleVariableRef CVARDataflowShowEnvironmentDefault(TEXT("p.Dataflow.Editor.ShowEnvironment"), bDataflowShowEnvironmentDefault, TEXT("Show the environment in the dataflow editor[def:false]"));

FDataflowPreviewSceneBase::FDataflowPreviewSceneBase(FPreviewScene::ConstructionValues ConstructionValues, UDataflowEditor* InEditor, const FName& InActorName)
	: FAdvancedPreviewScene(ConstructionValues)
	, DataflowEditor(InEditor)
{
	// Remove the base class's callback and add our own
	DefaultSettings->OnAssetViewerSettingsChanged().Remove(RefreshDelegate);
	RefreshDelegate = DefaultSettings->OnAssetViewerSettingsChanged().AddRaw(this, &FDataflowPreviewSceneBase::OnAssetViewerSettingsRefresh);

	FActorSpawnParameters SpawnParameters;
	SpawnParameters.Name = InActorName;
	RootSceneActor = GetWorld()->SpawnActor<AActor>(AActor::StaticClass(), SpawnParameters);
	
	check(DataflowEditor);
	
	SetFloorVisibility(bDataflowShowFloorDefault, false);
	SetEnvironmentVisibility(bDataflowShowEnvironmentDefault, false);
}

FDataflowPreviewSceneBase::~FDataflowPreviewSceneBase()
{
	UnregisterSceneElements();
	ModifySceneElements().Reset();
}

TObjectPtr<UDataflowBaseContent>& FDataflowPreviewSceneBase::GetEditorContent() 
{ 
	return DataflowEditor->GetEditorContent();
}

const TObjectPtr<UDataflowBaseContent>& FDataflowPreviewSceneBase::GetEditorContent() const 
{ 
	return DataflowEditor->GetEditorContent();
}

TArray<TObjectPtr<UDataflowBaseContent>>& FDataflowPreviewSceneBase::GetTerminalContents() 
{ 
	return DataflowEditor->GetTerminalContents();
}

const TArray<TObjectPtr<UDataflowBaseContent>>& FDataflowPreviewSceneBase::GetTerminalContents() const 
{ 
	return DataflowEditor->GetTerminalContents();
}

void FDataflowPreviewSceneBase::AddReferencedObjects(FReferenceCollector& Collector)
{
	FAdvancedPreviewScene::AddReferencedObjects(Collector);
	if (const TObjectPtr<UDataflowBaseContent> EditorContent = GetEditorContent())
	{
		EditorContent->AddContentObjects(Collector);
	}
}

bool FDataflowPreviewSceneBase::IsComponentSelected(const UPrimitiveComponent* InComponent) const
{
	if(DataflowModeManager.IsValid())
	{
		if (const UTypedElementSelectionSet* const TypedElementSelectionSet = DataflowModeManager->GetEditorSelectionSet())
		{
			if (const FTypedElementHandle ComponentElement = UEngineElementsLibrary::AcquireEditorComponentElementHandle(InComponent))
			{
				const bool bElementSelected = TypedElementSelectionSet->IsElementSelected(ComponentElement, FTypedElementIsSelectedOptions());
				return bElementSelected;
			}
		}
	}
	return false;
}

FBox FDataflowPreviewSceneBase::GetBoundingBox() const
{
	FBox SceneBounds(ForceInitToZero);
	if(DataflowModeManager.IsValid())
	{
		USelection* const SelectedComponents = DataflowModeManager->GetSelectedComponents();

		TArray<TWeakObjectPtr<UObject>> SelectedObjects;
		const int32 NumSelected = SelectedComponents ? SelectedComponents->GetSelectedObjects(SelectedObjects): 0;
		
		if(NumSelected > 0)
		{
			for(const TWeakObjectPtr<UObject> SelectedObject : SelectedObjects)
			{
				if(const UPrimitiveComponent* SelectedComponent = Cast<UPrimitiveComponent>(SelectedObject))
				{
					SceneBounds += SelectedComponent->Bounds.GetBox();
				}
			}
		}
		else
		{
			for(const IDataflowDebugDrawInterface::FDataflowElementsType::ElementType& SceneElement : GetSceneElements())
			{
				if(SceneElement.IsValid() && SceneElement->bIsSelected)
				{
					SceneBounds += SceneElement->BoundingBox;
				}
			}
			if(!SceneBounds.IsValid)
			{
				RootSceneActor->ForEachComponent<UPrimitiveComponent>(false, [&](const UPrimitiveComponent* InPrimComp)
				{
					if (InPrimComp->IsRegistered() && InPrimComp != Cast<UPrimitiveComponent>(DebugDrawComponent.Get()))
					{
						SceneBounds += InPrimComp->Bounds.GetBox();
					}
				});
			}
		}
	}
	return SceneBounds;
}


void FDataflowPreviewSceneBase::OnAssetViewerSettingsRefresh(const FName& InPropertyName)
{
	// If the profile was changed, update the current index and the scene.
	if (InPropertyName == GET_MEMBER_NAME_CHECKED(FPreviewSceneProfile, ProfileName))
	{
		CurrentProfileIndex = DefaultSettings->Profiles.IsValidIndex(CurrentProfileIndex) ? CurrentProfileIndex : 0;
		PreviousRotation = DefaultSettings->Profiles[CurrentProfileIndex].LightingRigRotation;
		UILightingRigRotationDelta = 0.0f;

		UpdateScene(DefaultSettings->Profiles[CurrentProfileIndex]);
	}
	else if (DefaultSettings->Profiles.IsValidIndex(CurrentProfileIndex))
	{
		const bool bNameNone = InPropertyName == NAME_None;

		const bool bUpdateEnvironment = (InPropertyName == GET_MEMBER_NAME_CHECKED(FPreviewSceneProfile, EnvironmentCubeMap)) || (InPropertyName == GET_MEMBER_NAME_CHECKED(FPreviewSceneProfile, LightingRigRotation) || (InPropertyName == GET_MEMBER_NAME_CHECKED(UAssetViewerSettings, Profiles)));
		const bool bUpdateSkyLight = bUpdateEnvironment || (InPropertyName == GET_MEMBER_NAME_CHECKED(FPreviewSceneProfile, SkyLightIntensity) || InPropertyName == GET_MEMBER_NAME_CHECKED(FPreviewSceneProfile, bUseSkyLighting) || (InPropertyName == GET_MEMBER_NAME_CHECKED(UAssetViewerSettings, Profiles)));
		const bool bUpdateDirectionalLight = (InPropertyName == GET_MEMBER_NAME_CHECKED(FPreviewSceneProfile, DirectionalLightIntensity)) || (InPropertyName == GET_MEMBER_NAME_CHECKED(FPreviewSceneProfile, DirectionalLightColor));
		const bool bUpdatePostProcessing = (InPropertyName == GET_MEMBER_NAME_CHECKED(FPreviewSceneProfile, PostProcessingSettings)) || (InPropertyName == GET_MEMBER_NAME_CHECKED(FPreviewSceneProfile, bPostProcessingEnabled));

		UILightingRigRotationDelta += PreviousRotation - DefaultSettings->Profiles[CurrentProfileIndex].LightingRigRotation;
		PreviousRotation = DefaultSettings->Profiles[CurrentProfileIndex].LightingRigRotation;

		UpdateScene(DefaultSettings->Profiles[CurrentProfileIndex], bUpdateSkyLight || bNameNone, bUpdateEnvironment || bNameNone, bUpdatePostProcessing || bNameNone, bUpdateDirectionalLight || bNameNone);
	}
}

void FDataflowPreviewSceneBase::SetCurrentProfileIndex(int32 NewProfileIndex)
{
	CurrentProfileIndex = NewProfileIndex;
}

void FDataflowPreviewSceneBase::AddSceneObject(UObject* SceneObject, const bool bIsConstruction) const
{
	using namespace UE::Editor::DataStorage;
	if (ICompatibilityProvider* Compatibility = GetMutableDataStorageFeature<ICompatibilityProvider>(CompatibilityFeatureName))
	{
		if (ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
		{
			RowHandle Row = Compatibility->FindRowWithCompatibleObject(SceneObject);
			
			if(Row == InvalidRowHandle)
			{
				Row = Compatibility->AddCompatibleObjectExplicit(SceneObject);
			}
			
			UE::Dataflow::Private::AddObjectColumns(DataStorage, Row, bIsConstruction);
			DataStorage->AddColumn<FDataflowSceneObjectTag>(Row, GetEditorContent()->GetDataflowOwner()->GetFName());
		}
	}
}

void FDataflowPreviewSceneBase::AddSceneStruct(void* SceneStruct, const TWeakObjectPtr<const UScriptStruct> TypeInfo,  bool bIsConstruction) const
{
	using namespace UE::Editor::DataStorage;
	if (ICompatibilityProvider* Compatibility = GetMutableDataStorageFeature<ICompatibilityProvider>(CompatibilityFeatureName))
	{
		if (ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
		{
			RowHandle Row = Compatibility->FindRowWithCompatibleObjectExplicit(SceneStruct);
			
			if(Row == InvalidRowHandle)
			{
				Row = Compatibility->AddCompatibleObjectExplicit(SceneStruct, TypeInfo);
			}

			UE::Dataflow::Private::AddObjectColumns(DataStorage, Row, bIsConstruction);
			DataStorage->AddColumn<FDataflowSceneStructTag>(Row, GetEditorContent()->GetDataflowOwner()->GetFName());
		}
	}
}

void FDataflowPreviewSceneBase::RemoveSceneObject(UObject* SceneObject) const
{
	using namespace UE::Editor::DataStorage;

	if (ICompatibilityProvider* Compatibility = GetMutableDataStorageFeature<ICompatibilityProvider>(CompatibilityFeatureName))
	{
		Compatibility->RemoveCompatibleObject(SceneObject);
	}
}

void FDataflowPreviewSceneBase::RemoveSceneStruct(void* SceneStruct) const
{
	using namespace UE::Editor::DataStorage;

	if (ICompatibilityProvider* Compatibility = GetMutableDataStorageFeature<ICompatibilityProvider>(CompatibilityFeatureName))
	{
		Compatibility->RemoveCompatibleObject(SceneStruct);
	}
}

void FDataflowPreviewSceneBase::UnregisterSceneElements()
{
	for(IDataflowDebugDrawInterface::FDataflowElementsType::ElementType& SceneElement : ModifySceneElements())
	{
		if(SceneElement.IsValid())
		{
			RemoveSceneStruct(SceneElement.Get());
		}
	}
}

void FDataflowPreviewSceneBase::RegisterSceneElements(const bool bIsConstruction)
{
	for(IDataflowDebugDrawInterface::FDataflowElementsType::ElementType& SceneElement : ModifySceneElements())
	{
		if(SceneElement.IsValid())
		{
			AddSceneStruct(SceneElement.Get(), FDataflowBaseElement::StaticStruct(), bIsConstruction);
		}
	}
}

USelection* FDataflowPreviewSceneBase::GetSelectedComponents() const
{
	return DataflowModeManager ? DataflowModeManager->GetSelectedComponents() : nullptr;
}

#undef LOCTEXT_NAMESPACE

