// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"
#include "Interfaces/IPluginManager.h"
#include "Materials/Material.h"
#include "Misc/Paths.h"
#include "Dataflow/DataflowEditorCommands.h"
#include "Dataflow/DataflowEditorUtil.h"

class FDataflowEditorStyle final : public FSlateStyleSet
{
public:
	FDataflowEditorStyle() : FSlateStyleSet("DataflowEditorStyle")
	{
		const FVector2D Icon16x16(16.f, 16.f);
		const FVector2D Icon28x14(28.f, 14.f);
		const FVector2D Icon20x20(20.f, 20.f);
		const FVector2D Icon24x24(24.f, 24.f);
		const FVector2D Icon40x40(40.f, 40.f);
		const FVector2D Icon64x64(64.f, 64.f);

		const FVector2D ToolbarIconSize = Icon20x20;

		SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));
		SetContentRoot(IPluginManager::Get().FindPlugin("Dataflow")->GetBaseDir() / TEXT("Resources"));

		Set("ClassIcon.Dataflow", new IMAGE_BRUSH_SVG("DataflowAsset_16", Icon16x16));
		Set("ClassThumbnail.Dataflow", new IMAGE_BRUSH_SVG("DataflowAsset_64", Icon64x64));

		Set("Dataflow.Render.Unknown", new IMAGE_BRUSH("Slate/Switch_Undetermined_56x_28x", Icon28x14));
		Set("Dataflow.Render.Disabled", new IMAGE_BRUSH("Slate/Switch_OFF_56x_28x", Icon28x14));
		Set("Dataflow.Render.Enabled", new IMAGE_BRUSH("Slate/Switch_ON_56x_28x", Icon28x14));

		Set("Dataflow.Cached.False", new IMAGE_BRUSH("Slate/status_grey", Icon16x16));
		Set("Dataflow.Cached.True", new IMAGE_BRUSH("Slate/status_green", Icon16x16));

		Set("Dataflow.SelectObject", new IMAGE_BRUSH("Slate/Dataflow_SelectObject_40x", Icon40x40));
		Set("Dataflow.SelectFace", new IMAGE_BRUSH("Slate/Dataflow_SelectFace_40x", Icon40x40));
		Set("Dataflow.SelectVertex", new IMAGE_BRUSH("Slate/Dataflow_SelectVertex_40x", Icon40x40));

		Set("Dataflow.FreezeNode", new IMAGE_BRUSH("Slate/Dataflow_FreezeNode_24x", Icon24x24));  // There's also a 32x version if it is decided that it is too small
		
		// Dataflow weight map and skin weights icons 
		Set("Dataflow.PaintWeightMap", new IMAGE_BRUSH_SVG("Slate/Dataflow_WeightMap", Icon20x20));
		Set("Dataflow.EditSkinWeights", new IMAGE_BRUSH_SVG("Slate/Dataflow_SkinWeight", Icon20x20));
		Set("Dataflow.EditSkeletonBones", new IMAGE_BRUSH_SVG("Slate/Dataflow_EditSkeleton", Icon20x20));

		// Dataflow simulation icons
		Set("Dataflow.ResetSimulation", new CORE_IMAGE_BRUSH_SVG("Starship/Common/Undo", Icon20x20));
		Set("Dataflow.LockSimulation", new CORE_IMAGE_BRUSH_SVG("Starship/Common/lock", Icon20x20));
		Set("Dataflow.UnlockSimulation", new CORE_IMAGE_BRUSH_SVG("Starship/Common/lock-unlocked", Icon20x20));

		DefaultMaterial = Cast<UMaterial>(StaticLoadObject(UMaterial::StaticClass(), NULL, TEXT("/Engine/BasicShapes/BasicShapeMaterial")));
		VertexMaterial = Cast<UMaterial>(StaticLoadObject(UMaterial::StaticClass(), NULL, TEXT("/Engine/EditorMaterials/Dataflow/DataflowVertexMaterial")));
		DefaultTwoSidedMaterial = Cast<UMaterial>(StaticLoadObject(UMaterial::StaticClass(), NULL, TEXT("/Engine/EditorMaterials/Dataflow/DataflowTwoSidedVertexMaterial")));

		FSlateStyleRegistry::RegisterSlateStyle(*this);
	}

	~FDataflowEditorStyle()
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*this);
	}


	/** Default Rendering Material for Mesh surfaces */
	UMaterial* DefaultMaterial = nullptr;
	UMaterial* VertexMaterial = nullptr;
	UMaterial* DefaultTwoSidedMaterial = nullptr;


public:

	static FDataflowEditorStyle& Get()
	{
		static FDataflowEditorStyle Inst;
		return Inst;
	}

};

