// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowEditorSubGraphTabSummoner.h"

#include "Dataflow/DataflowGraphEditor.h"
#include "Dataflow/DataflowEditorToolkit.h"
#include "Widgets/Docking/SDockTab.h"

#include "WorkflowOrientedApp/WorkflowTabManager.h"

FDataflowEditorSubGraphTabSummoner::FDataflowEditorSubGraphTabSummoner(TSharedPtr<FDataflowEditorToolkit> InEditorToolkitPtr, FOnCreateGraphEditorWidget CreateGraphEditorWidgetCallback)
	: FDocumentTabFactoryForObjects<UDataflowSubGraph>("DataflowEditor_SubGraphTab", InEditorToolkitPtr)
	, EditorToolkitWeakPtr(InEditorToolkitPtr)
	, OnCreateGraphEditorWidget(CreateGraphEditorWidgetCallback)
{
}

TAttribute<FText> FDataflowEditorSubGraphTabSummoner::ConstructTabNameForObject(UDataflowSubGraph* DocumentID) const
{
	return TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateStatic(&FDataflowEditorSubGraphTabSummoner::GetGraphDisplayName, (const UEdGraph*)DocumentID));
}

FText FDataflowEditorSubGraphTabSummoner::GetGraphDisplayName(const UEdGraph* Graph)
{
	return FText::FromString(Graph->GetFName().ToString());
}

TSharedPtr<SDataflowGraphEditor> FDataflowEditorSubGraphTabSummoner::GetDataflowEditorFromTab(TSharedPtr<SDockTab> Tab) const
{
	TSharedRef<SGraphEditor> GraphEditor = StaticCastSharedRef<SGraphEditor>(Tab->GetContent());
	return StaticCastSharedPtr<SDataflowGraphEditor>(GraphEditor.ToSharedPtr());
}

void FDataflowEditorSubGraphTabSummoner::OnTabActivated(TSharedPtr<SDockTab> Tab) const
{
	if (TSharedPtr<SDataflowGraphEditor> DataflowEditor = GetDataflowEditorFromTab(Tab))
	{
		if (TSharedPtr<FDataflowEditorToolkit> Toolkit = EditorToolkitWeakPtr.Pin())
		{
			Toolkit->SetSubGraphTabActiveState(DataflowEditor, true);
		}
	}
}

// Called when a tab created from this factory is brought to the foreground
void FDataflowEditorSubGraphTabSummoner::OnTabForegrounded(TSharedPtr<SDockTab> Tab) const
{
	if (TSharedPtr<SDataflowGraphEditor> DataflowEditor = GetDataflowEditorFromTab(Tab))
	{
		if (TSharedPtr<FDataflowEditorToolkit> Toolkit = EditorToolkitWeakPtr.Pin())
		{
			Toolkit->SetSubGraphTabActiveState(DataflowEditor, true);
		}
	}
}

void FDataflowEditorSubGraphTabSummoner::OnTabBackgrounded(TSharedPtr<SDockTab> Tab) const
{
	if (TSharedPtr<SDataflowGraphEditor> DataflowEditor = GetDataflowEditorFromTab(Tab))
	{
		if (TSharedPtr<FDataflowEditorToolkit> Toolkit = EditorToolkitWeakPtr.Pin())
		{
			Toolkit->SetSubGraphTabActiveState(DataflowEditor, false);
		}
	}

}

void FDataflowEditorSubGraphTabSummoner::OnTabRefreshed(TSharedPtr<SDockTab> Tab) const
{
	if (TSharedPtr<SDataflowGraphEditor> DataflowEditor = GetDataflowEditorFromTab(Tab))
	{
		DataflowEditor->NotifyGraphChanged();
	}
}

void FDataflowEditorSubGraphTabSummoner::SaveState(TSharedPtr<SDockTab> Tab, TSharedPtr<FTabPayload> Payload) const
{
	// NOT YET IMPLEMENTED
}

TSharedRef<SWidget> FDataflowEditorSubGraphTabSummoner::CreateTabBodyForObject(const FWorkflowTabSpawnInfo& Info, UDataflowSubGraph* DocumentID) const
{
	check(Info.TabInfo.IsValid());
	return OnCreateGraphEditorWidget.Execute(Info.TabInfo.ToSharedRef(), DocumentID);
}

const FSlateBrush* FDataflowEditorSubGraphTabSummoner::GetTabIconForObject(const FWorkflowTabSpawnInfo& Info, UDataflowSubGraph* DocumentID) const
{
	const FSlateBrush* ReturnValue = FAppStyle::GetBrush(TEXT("GraphEditor.Function_16x"));
	return ReturnValue;
}

TSharedRef<FGenericTabHistory> FDataflowEditorSubGraphTabSummoner::CreateTabHistoryNode(TSharedPtr<FTabPayload> Payload)
{
	return MakeShareable(new FGenericTabHistory(SharedThis(this), Payload));
}