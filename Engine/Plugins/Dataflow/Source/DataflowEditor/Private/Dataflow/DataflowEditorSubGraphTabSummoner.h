// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorkflowOrientedApp/WorkflowUObjectDocuments.h"
#include "Dataflow/DataflowSubGraph.h"

class FDataflowEditorToolkit;
class SDataflowGraphEditor;
class SGraphEditor;
/**
* Summoner class for Dataflow SubGraph tabs
*/
struct FDataflowEditorSubGraphTabSummoner : public FDocumentTabFactoryForObjects<UDataflowSubGraph>
{
public:
	DECLARE_DELEGATE_RetVal_TwoParams(TSharedRef<SGraphEditor>, FOnCreateGraphEditorWidget, TSharedRef<FTabInfo>, UDataflowSubGraph*);

public:
	FDataflowEditorSubGraphTabSummoner(TSharedPtr<FDataflowEditorToolkit> InEditorToolkitPtr, FOnCreateGraphEditorWidget CreateGraphEditorWidgetCallback);

	virtual void OnTabActivated(TSharedPtr<SDockTab> Tab) const override;

	virtual void OnTabForegrounded(TSharedPtr<SDockTab> Tab) const override;

	virtual void OnTabBackgrounded(TSharedPtr<SDockTab> Tab) const override;

	virtual void OnTabRefreshed(TSharedPtr<SDockTab> Tab) const override;

	virtual void SaveState(TSharedPtr<SDockTab> Tab, TSharedPtr<FTabPayload> Payload) const override;

protected:
	static FText GetGraphDisplayName(const UEdGraph* Graph);

	virtual TAttribute<FText> ConstructTabNameForObject(UDataflowSubGraph* DocumentID) const override;

	virtual TSharedRef<SWidget> CreateTabBodyForObject(const FWorkflowTabSpawnInfo& Info, UDataflowSubGraph* DocumentID) const override;

	virtual const FSlateBrush* GetTabIconForObject(const FWorkflowTabSpawnInfo& Info, UDataflowSubGraph* DocumentID) const override;

	virtual TSharedRef<FGenericTabHistory> CreateTabHistoryNode(TSharedPtr<FTabPayload> Payload) override;

	TSharedPtr<SDataflowGraphEditor> GetDataflowEditorFromTab(TSharedPtr<SDockTab> Tab) const;

protected:
	TWeakPtr<FDataflowEditorToolkit> EditorToolkitWeakPtr;
	FOnCreateGraphEditorWidget OnCreateGraphEditorWidget;
};

