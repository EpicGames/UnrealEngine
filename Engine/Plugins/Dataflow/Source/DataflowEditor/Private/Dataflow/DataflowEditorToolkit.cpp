// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowEditorToolkit.h"

#include "AdvancedPreviewScene.h"
#include "AdvancedPreviewSceneModule.h"
#include "Algo/Transform.h"
#include "AssetEditorModeManager.h"
#include "AssetViewerSettings.h"
#include "DataflowEditorStyle.h"
#include "DetailCategoryBuilder.h"
#include "Dataflow/AssetDefinition_DataflowAsset.h"
#include "Dataflow/DataflowAdvancedPreviewDetailsTab.h"
#include "Dataflow/DataflowAssetViewerSettingsCustomization.h"
#include "Dataflow/DataflowDebugDrawComponent.h"
#include "Dataflow/DataflowDebugDraw.h"
#include "Dataflow/DataflowEditor.h"
#include "Dataflow/DataflowContent.h"
#include "Dataflow/DataflowCollectionSpreadSheet.h"
#include "Dataflow/DataflowCollectionSpreadSheetWidget.h"
#include "Dataflow/DataflowConstructionScene.h"
#include "Dataflow/DataflowConstructionViewportClient.h"
#include "Dataflow/DataflowEditorCommands.h"
#include "Dataflow/DataflowEditorMode.h"
#include "Dataflow/DataflowEditorModeToolkit.h"
#include "Dataflow/DataflowEditorModule.h"
#include "Dataflow/DataflowEditorModeUILayer.h"
#include "Dataflow/DataflowEditorUtil.h"
#include "Dataflow/DataflowEditorSubGraphTabSummoner.h"
#include "Dataflow/DataflowConstructionViewport.h"
#include "Dataflow/DataflowEditorOptions.h"
#include "Dataflow/DataflowEdNode.h"
#include "Dataflow/DataflowGraphEditor.h"
#include "Dataflow/DataflowNodeDetailExtension.h"
#include "Dataflow/DataflowInstance.h"
#include "Dataflow/DataflowRenderingViewMode.h"
#include "Dataflow/DataflowSceneProfileIndexStorage.h"
#include "Dataflow/DataflowSimulationScene.h"
#include "Dataflow/DataflowSimulationVisualization.h"
#include "Dataflow/DataflowSchema.h"
#include "Dataflow/DataflowSkeletonView.h"
#include "Dataflow/DataflowOutlinerView.h"
#include "Dataflow/DataflowSimulationViewport.h"
#include "Dataflow/DataflowSimulationViewportClient.h"
#include "Dataflow/DataflowSubGraphNodes.h"
#include "Dataflow/DataflowVariableNodes.h"
#include "Dataflow/DataflowMembersWidget.h"
#include "Dataflow/DataflowSelectionView.h"
#include "Dataflow/DataflowOutputLog.h"
#include "Dataflow/DataflowPath.h"
#include "DataflowEditorTools/DataflowPropertyGizmoManager.h"
#include "Dataflow/DataflowSimulationManager.h"
#include "Dataflow/DataflowSimulationNodes.h"
#include "Chaos/CacheCollection.h"
#include "EditorModeManager.h"
#include "EditorViewportTabContent.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "FileHelpers.h"
#include "GameFramework/Actor.h"
#include "Interfaces/Interface_AssetUserData.h"
#include "ISkeletonTree.h"
#include "ISceneOutliner.h"
#include "IStructureDetailsView.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "Selection.h"
#include "GeometryCache.h"
#include "ToolMenus.h"
#include "WorkflowOrientedApp/WorkflowTabManager.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Toolkits/AssetEditorToolkitMenuContext.h"
#include "UObject/PackageReload.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/Docking/SDockTab.h"
#include "Framework/Docking/SDockingTabStack.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/SDataflowSimulationTimeline.h"
#include "Widgets/DataflowSimulationBinding.h"

#define LOCTEXT_NAMESPACE "DataflowEditorToolkit"

//DEFINE_LOG_CATEGORY_STATIC(EditorToolkitLog, Log, All);

bool bDataflowEnableSkeletonView = false;
FAutoConsoleVariableRef CVARDataflowEnableSkeletonView(TEXT("p.Dataflow.Editor.EnableSkeletonView"), bDataflowEnableSkeletonView,
	TEXT("Deprecated Tool! Allows the Dataflow editor to create a skeleton view that reflects the hierarchy and selection state of the construction viewport.[def:false]"));

bool bDataflowLogNodeEvaluation = false;
FAutoConsoleVariableRef CVARDataflowLogNodeEvaluation(TEXT("p.Dataflow.Editor.LogNodeEvaluation"), bDataflowLogNodeEvaluation,
	TEXT("When enabled, logs the begin and end evaluation of Dataflow nodes - if the graph is large this may cause evry large amounty of log messages to be displayed in the output window.[def:false]"));

bool bDataflowForceGraphEditable = false;
FAutoConsoleVariableRef CVARbDataflowForceGraphEditable(TEXT("p.Dataflow.Editor.ForceGraphEditable"), bDataflowForceGraphEditable,
	TEXT("when enabled all graph will be editable and the Unlock/lock editing buttons will not be visible.[def:false]"));

const FName FDataflowEditorToolkit::GraphCanvasTabId(TEXT("DataflowEditor_GraphCanvas"));
const FName FDataflowEditorToolkit::SubGraphCanvasTabId(TEXT("DataflowEditor_SubGraphTab"));
const FName FDataflowEditorToolkit::NodeDetailsTabId(TEXT("DataflowEditor_NodeDetails"));
const FName FDataflowEditorToolkit::PreviewSceneTabId(TEXT("DataflowEditor_PreviewScene"));
const FName FDataflowEditorToolkit::OutlinerViewTabId(TEXT("DataflowEditor_SceneOutliner"));
const FName FDataflowEditorToolkit::SkeletonViewTabId(TEXT("DataflowEditor_SkeletonView"));
const FName FDataflowEditorToolkit::SelectionViewTabId_1(TEXT("DataflowEditor_SelectionView_1"));
const FName FDataflowEditorToolkit::SelectionViewTabId_2(TEXT("DataflowEditor_SelectionView_2"));
const FName FDataflowEditorToolkit::SelectionViewTabId_3(TEXT("DataflowEditor_SelectionView_3"));
const FName FDataflowEditorToolkit::SelectionViewTabId_4(TEXT("DataflowEditor_SelectionView_4"));
const FName FDataflowEditorToolkit::CollectionSpreadSheetTabId_1(TEXT("DataflowEditor_CollectionSpreadSheet_1"));
const FName FDataflowEditorToolkit::CollectionSpreadSheetTabId_2(TEXT("DataflowEditor_CollectionSpreadSheet_2"))
 ;
const FName FDataflowEditorToolkit::CollectionSpreadSheetTabId_3(TEXT("DataflowEditor_CollectionSpreadSheet_3"));
const FName FDataflowEditorToolkit::CollectionSpreadSheetTabId_4(TEXT("DataflowEditor_CollectionSpreadSheet_4"));
const FName FDataflowEditorToolkit::SimulationViewportTabId(TEXT("DataflowEditor_SimulationViewport"));
const FName FDataflowEditorToolkit::SimulationVisualizationTabId(TEXT("DataflowEditor_SimulationVisualizationTab"));
const FName FDataflowEditorToolkit::SimulationTimelineTabId(TEXT("DataflowEditor_SimulationTimelineTab"));
const FName FDataflowEditorToolkit::MembersWidgetTabId(TEXT("DataflowEditor_MembersWidgetTab"));
const FName FDataflowEditorToolkit::OutputLogTabId(TEXT("DataflowEditor_OutputLog"));

namespace UE::Dataflow::Private
{
	static const FName DataflowEditorToolkitName("DataflowEditor");
	static const float DataflowLayoutTimelineMaxHeight = 75.f;

	/** Create a debug draw component to visualize per node informations */
	TUniquePtr<IDataflowDebugDrawInterface> CreateDebugDrawComponent(
		TSharedPtr<FDataflowPreviewSceneBase>& DataflowScene,
		TObjectPtr<class UDataflowDebugDrawComponent>& DebugDrawComponent)
	{
		if (DataflowScene && !DebugDrawComponent)
		{
			if (TObjectPtr<AActor> RootActor = DataflowScene->GetRootActor())
			{
				if (UDataflowDebugDrawComponent* Component = Cast<UDataflowDebugDrawComponent>(RootActor->FindComponentByClass<UDataflowDebugDrawComponent>()))
				{
					DebugDrawComponent = Cast<UDataflowDebugDrawComponent>(Component);
				}
				else
				{
					DebugDrawComponent = NewObject<UDataflowDebugDrawComponent>(RootActor, FName(LOCTEXT("DataflowDebugDrawComponent", "Dataflow Debug Draw Component").ToString()), RF_Transient);
					DebugDrawComponent->RegisterComponentWithWorld(DataflowScene->GetWorld());
				}
			}
		}

		checkf(DebugDrawComponent, TEXT("Could not create or find a DebugDrawComponentComponent"));

		FDataflowDebugRenderSceneProxy* SceneProxy = static_cast<FDataflowDebugRenderSceneProxy*>(DebugDrawComponent->GetSceneProxy());
		checkf(SceneProxy, TEXT("Could not find a FDataflowDebugRenderSceneProxy on the DebugDrawComponent"));

		return MakeUnique<FDataflowDebugDraw>(SceneProxy, DataflowScene->ModifySceneElements());
	}

	/** Update the debug draw component from the selected/pinned nodes matching the NodeType */
	template<typename NodeType>
	void UpdateDebugDrawComponent(const TObjectPtr<UDataflowBaseContent>& EditorContent,
		const FString& RootName, TSharedPtr<FDataflowPreviewSceneBase> DataflowScene, const bool bIsConstruction, FString& DebugDrawOverlay,
		const TSharedPtr<FEditorModeTools> EditorModeManager)
	{
		if(!DataflowScene || !EditorContent)
		{
			return;
		}
		DataflowScene->UnregisterSceneElements();
		DataflowScene->ModifySceneElements().Reset();
		
		TObjectPtr<UDataflowDebugDrawComponent>& DebugDrawComponent = DataflowScene->ModifyDebugDrawComponent();
		const TUniquePtr<IDataflowDebugDrawInterface> DebugDrawObject =
				CreateDebugDrawComponent(DataflowScene, DebugDrawComponent);

		TSharedPtr<FDataflowBaseElement> RootSceneElement = MakeShared<FDataflowBaseElement>(RootName, nullptr, FBox(ForceInitToZero), bIsConstruction);
		DataflowScene->ModifySceneElements().Add(RootSceneElement);

		FName CurrentViewMode;
		if (UDataflowEditorMode* const DataflowMode = Cast<UDataflowEditorMode>(EditorModeManager->GetActiveScriptableMode(UDataflowEditorMode::EM_DataflowEditorModeId)))
		{
			if (const UE::Dataflow::IDataflowConstructionViewMode* const ConstructionViewMode = DataflowMode->GetConstructionViewMode())
			{
				CurrentViewMode = ConstructionViewMode->GetName();
			}
		}
	
		if (const TSharedPtr<UE::Dataflow::FEngineContext> DataflowContext = EditorContent->GetDataflowContext())
		{
			if (const UDataflowEdNode* const SelectedNode = EditorContent->GetSelectedNode())
			{
				if (const TSharedPtr<const FDataflowNode> SelectedDataflowNode = SelectedNode->GetDataflowNode())
				{
					if(SelectedDataflowNode->IsA(NodeType::StaticType()))
					{
						if (SelectedDataflowNode->CanDebugDrawViewMode(CurrentViewMode))
						{
							DebugDrawObject->ResetAllState();

							FDataflowNode::FDebugDrawParameters DebugDrawParameters;
							DebugDrawParameters.bNodeIsSelected = true;
							DebugDrawParameters.bNodeIsPinned = false;
							DebugDrawParameters.CurrentViewMode = CurrentViewMode;
							SelectedDataflowNode->DebugDraw(*DataflowContext, *DebugDrawObject, DebugDrawParameters);
						}
					}
				}
			}

			if (EditorContent->GetDataflowAsset())
			{
				for (const UDataflowEdNode* const PinnedNode : EditorContent->GetDataflowAsset()->GetWireframeRenderTargets())
				{
					if (const TSharedPtr<const FDataflowNode> PinnedDataflowNode = PinnedNode->GetDataflowNode())
					{
						if (PinnedDataflowNode->IsA(NodeType::StaticType()))
						{
							if (PinnedDataflowNode->CanDebugDrawViewMode(CurrentViewMode))
							{
								DebugDrawObject->ResetAllState();

								FDataflowNode::FDebugDrawParameters DebugDrawParameters;
								DebugDrawParameters.bNodeIsSelected = false;
								DebugDrawParameters.bNodeIsPinned = true;
								DebugDrawParameters.CurrentViewMode = CurrentViewMode;
								PinnedDataflowNode->DebugDraw(*DataflowContext, *DebugDrawObject, DebugDrawParameters);
							}
						}
					}
				}
			}
		}

		DataflowScene->RegisterSceneElements(bIsConstruction);

		DebugDrawComponent->UpdateBounds();
		DebugDrawComponent->MarkRenderTransformDirty();

		DebugDrawOverlay = DebugDrawObject->GetOverlayText();
	}

	static TSharedPtr<FDataflowEditorToolkit> GetAssetEditorToolkitFromMenuContext(UAssetEditorToolkitMenuContext* Context)
	{
		if (Context)
		{
			if (const TSharedPtr<FAssetEditorToolkit> Toolkit = Context->Toolkit.Pin())
			{
				if (Toolkit->GetToolkitFName() == DataflowEditorToolkitName)  // Note: This will not detect subclasses of FDataflowEditorToolkit
				{
					return StaticCastSharedPtr<FDataflowEditorToolkit>(Toolkit);
				}
			}
		}
		return nullptr;
	}

	//////////////////////////////////////////////////////////////

	void FLogMessageStorage::GetMessage(int32 Index, UE::Dataflow::FContext::FLogMessage& OutMessage) const
	{
		if (Entries.IsValidIndex(Index))
		{
			OutMessage = Entries[Index].Message;
		}
		else
		{
			OutMessage = UE::Dataflow::FContext::FLogMessage();
		}

	}

	void FLogMessageStorage::Add(const UE::Dataflow::FContext::FLogMessage& Message)
	{
		FTimestamp Timestamp = FTimestamp::Invalid;
		if (TSharedPtr<const FDataflowNode> Node = Message.Node.Pin())
		{
			Timestamp = Node->GetTimestamp();
		}
		Entries.Emplace(FEntry
			{
				.Timestamp = Timestamp,
				.Message = Message,
			});
	}

	void FLogMessageStorage::GetMessageForNode(const FDataflowNode& Node, TArray<UE::Dataflow::FContext::FLogMessage>& OutMessages) const
	{
		for (const FEntry& Entry : Entries)
		{
			if (Entry.Message.Node == Node.AsWeak())
			{
				OutMessages.Add(Entry.Message);
			}
		}
	}

	EMessageSeverity::Type FLogMessageStorage::GetMaxSeverityForNode(const FDataflowNode& Node) const
	{
		EMessageSeverity::Type OutSeverity = EMessageSeverity::Info;
		for (const FEntry& Entry : Entries)
		{
			if (Entry.Message.Node == Node.AsWeak())
			{
				if (Entry.Message.Severity < OutSeverity)
				{
					OutSeverity = Entry.Message.Severity;
				}
			}
		}
		return OutSeverity;
	}

	void FLogMessageStorage::RemoveForNode(const FDataflowNode& Node, const FTimestamp& Timestamp)
	{
		TArray<FEntry> NewEntries;
		NewEntries.Reserve(Entries.Num());

		for (const FEntry& Entry : Entries)
		{
			bool bKeep = true;
			if (Entry.Message.Node == Node.AsWeak())
			{
				if (Timestamp == FTimestamp::Invalid || Entry.Timestamp < Timestamp)
				{
					bKeep = false;
				}
			}
			else
			{
				// invalid node , remove the message even if the node does not match
				bKeep = false;
			}
			if (bKeep)
			{
				NewEntries.Emplace(Entry);
			}
		}
		Entries = MoveTemp(NewEntries);
	}

	EMessageSeverity::Type FLogMessageStorage::GetMaxSeverity()
	{
		EMessageSeverity::Type OutSeverity = EMessageSeverity::Info;
		for (const FEntry& Entry : Entries)
		{
			if (Entry.Message.Severity < OutSeverity)
			{
				OutSeverity = Entry.Message.Severity;
			}
		}
		return OutSeverity;
	}

}

FDataflowEditorToolkit::FDataflowEditorToolkit(UAssetEditor* InOwningAssetEditor)
	: FBaseCharacterFXEditorToolkit(InOwningAssetEditor, UE::Dataflow::Private::DataflowEditorToolkitName)
	, DataflowEditor(Cast< UDataflowEditor>(InOwningAssetEditor))
	, bDataflowEnableGraphEval(true)
	, LogMessages(MakeShared<UE::Dataflow::Private::FLogMessageStorage>())
{
	EvaluationMode = EDataflowEditorEvaluationMode::Automatic;

	// When saving, only prompt to checkout and save assets that are actually modified
	bCheckDirtyOnAssetSave = true;

	bAllowEvaluationInPIE = false;
	if (const UDataflowEvaluationSettings* EvaluationSettings = DataflowEditor->FindEditorSettings<UDataflowEvaluationSettings>())
	{
		bAllowEvaluationInPIE = EvaluationSettings->bAllowEvaluationInPIE;
	}

	check(DataflowEditor);

	ConstructionDefaultLayout = FTabManager::NewLayout(FName("DataflowConstructionLayout12"))
		->AddArea
		(
			FTabManager::NewPrimaryArea()->SetOrientation(Orient_Vertical)
			->Split
			(
				FTabManager::NewSplitter()->SetOrientation(Orient_Horizontal)
				->SetSizeCoefficient(0.50f)	// Relative height of (Tools Panel, Construction Viewport, Preview Viewport) vs (Dataflow Graph Editor, Outliner)
				->Split
				(
				FTabManager::NewStack()
						->SetSizeCoefficient(0.15f)		// Relative width of (Tools Panel) vs (Construction Viewport, Preview Viewport)
						->SetExtensionId(UDataflowEditorUISubsystem::EditorSidePanelAreaName)
						->SetHideTabWell(false)
				)
				->Split
				(
				FTabManager::NewSplitter()->SetOrientation(Orient_Vertical)
						->SetSizeCoefficient(0.7f)
						->Split
					(
					FTabManager::NewStack()
							->SetSizeCoefficient(0.9f)		// Relative width of (Construction Viewport) vs (Tools Panel, Preview Viewport)
							->AddTab(ViewportTabID, ETabState::OpenedTab)
							->AddTab(SimulationViewportTabId, ETabState::OpenedTab)
							->SetExtensionId("ViewportArea")
							->SetHideTabWell(true)
							->SetForegroundTab(ViewportTabID)
					)
					->Split
					(
					FTabManager::NewStack()
							->SetSizeCoefficient(0.1f)		// Relative width of (simulation timeline) vs (Preview Viewport, Construction Viewport)
							->SetMinSize(UE::Dataflow::Private::DataflowLayoutTimelineMaxHeight)
							->AddTab(SimulationTimelineTabId, ETabState::OpenedTab)
							->SetExtensionId("TimelineArea")
							->SetHideTabWell(true)
					)
				)
				->Split
				(
				FTabManager::NewSplitter()->SetOrientation(Orient_Vertical)
						->SetSizeCoefficient(0.15f)
						->Split
					(
						FTabManager::NewStack()
							->SetSizeCoefficient(0.7f)		// Relative width of (Tools Panel) vs (Construction Viewport, Preview Viewport)
							->AddTab(OutlinerViewTabId, ETabState::OpenedTab)
							->SetExtensionId("OutlinerArea")
							->SetHideTabWell(false)
					)
					->Split
					(
						FTabManager::NewStack()
							->SetSizeCoefficient(0.3f)
							->AddTab(DetailsTabID, ETabState::OpenedTab)
							->AddTab(PreviewSceneTabId, ETabState::OpenedTab)
							->SetExtensionId("DetailsArea")
							->SetHideTabWell(false)
							->SetForegroundTab(DetailsTabID)
					)
				)
			)
			->Split
			(
				FTabManager::NewSplitter()->SetOrientation(Orient_Horizontal)
				->SetSizeCoefficient(0.50f)	// Relative height of (Dataflow Node Details) vs (Asset Details, Preview Scene Details)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.15f)
					->AddTab(MembersWidgetTabId, ETabState::OpenedTab)
					->SetExtensionId("MembersArea")
					->SetHideTabWell(false)
					->SetForegroundTab(MembersWidgetTabId)
				)
				->Split
				(
					FTabManager::NewStack()
						->SetSizeCoefficient(0.7f)	// Relative height of (Tools Panel, Construction Viewport, Preview Viewport, Dataflow Graph Editor, Outliner) vs (Asset Details, Preview Scene Details, Dataflow Node Details)
						->AddTab(GraphCanvasTabId, ETabState::OpenedTab)
						->AddTab(SubGraphCanvasTabId, ETabState::ClosedTab)
						->AddTab(CollectionSpreadSheetTabId_1, ETabState::OpenedTab)
						->AddTab(OutputLogTabId, ETabState::OpenedTab)
						->SetExtensionId("GraphArea")
						->SetHideTabWell(false)
						->SetForegroundTab(GraphCanvasTabId)
				)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.15f)	// Relative height of (Dataflow Node Details) vs (Asset Details, Preview Scene Details)
					->AddTab(NodeDetailsTabId, ETabState::OpenedTab)
					->SetExtensionId("NodeArea")
					->SetHideTabWell(false)
				)
			)
		);

	SimulationDefaultLayout = FTabManager::NewLayout(FName("DataflowSimulationLayout4"))
		->AddArea
		(
			FTabManager::NewPrimaryArea()->SetOrientation(Orient_Vertical)
			->Split
			(
				FTabManager::NewSplitter()->SetOrientation(Orient_Horizontal)
				->SetSizeCoefficient(0.50f)	// Relative height of (Tools Panel, Construction Viewport, Preview Viewport) vs (Dataflow Graph Editor, Outliner)
				->Split
				(
				FTabManager::NewSplitter()->SetOrientation(Orient_Vertical)
						->SetSizeCoefficient(0.85f)
						->Split
					(
					FTabManager::NewStack()
							->SetSizeCoefficient(0.9f)		// Relative width of (Construction Viewport) vs (Tools Panel, Preview Viewport)
							->AddTab(SimulationViewportTabId, ETabState::OpenedTab)
							->SetExtensionId("ViewportArea")
							->SetHideTabWell(true)
							->SetForegroundTab(ViewportTabID)
					)
					->Split
					(
					FTabManager::NewStack()
							->SetSizeCoefficient(0.1f)		// Relative width of (simulation timeline) vs (Preview Viewport, Construction Viewport)
							->SetMinSize(UE::Dataflow::Private::DataflowLayoutTimelineMaxHeight)
							->AddTab(SimulationTimelineTabId, ETabState::OpenedTab)
							->SetExtensionId("TimelineArea")
							->SetHideTabWell(true)
					)
				)
				->Split
				(
				FTabManager::NewSplitter()->SetOrientation(Orient_Vertical)
						->SetSizeCoefficient(0.15f)
						->Split
					(
						FTabManager::NewStack()
							->SetSizeCoefficient(0.7f)		// Relative width of (Tools Panel) vs (Construction Viewport, Preview Viewport)
							->AddTab(OutlinerViewTabId, ETabState::OpenedTab)
							->SetExtensionId("OutlinerArea")
							->SetHideTabWell(false)
					)
					->Split
					(
						FTabManager::NewStack()
							->SetSizeCoefficient(0.3f)
							->AddTab(DetailsTabID, ETabState::OpenedTab)
							->AddTab(PreviewSceneTabId, ETabState::OpenedTab)
							->SetExtensionId("DetailsArea")
							->SetHideTabWell(false)
							->SetForegroundTab(DetailsTabID)
					)
				)
			)
			->Split
			(
				FTabManager::NewSplitter()->SetOrientation(Orient_Horizontal)
				->SetSizeCoefficient(0.50f)	// Relative height of (Dataflow Node Details) vs (Asset Details, Preview Scene Details)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.15f)
					->AddTab(MembersWidgetTabId, ETabState::OpenedTab)
					->SetExtensionId("MembersArea")
					->SetHideTabWell(false)
					->SetForegroundTab(MembersWidgetTabId)
				)
				->Split
				(
					FTabManager::NewStack()
						->SetSizeCoefficient(0.7f)	// Relative height of (Tools Panel, Construction Viewport, Preview Viewport, Dataflow Graph Editor, Outliner) vs (Asset Details, Preview Scene Details, Dataflow Node Details)
						->AddTab(GraphCanvasTabId, ETabState::OpenedTab)
						->AddTab(SubGraphCanvasTabId, ETabState::ClosedTab)
						->AddTab(CollectionSpreadSheetTabId_1, ETabState::OpenedTab)
						->AddTab(OutputLogTabId, ETabState::OpenedTab)
						->SetExtensionId("GraphArea")
						->SetHideTabWell(false)
						->SetForegroundTab(GraphCanvasTabId)
				)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.15f)	// Relative height of (Dataflow Node Details) vs (Asset Details, Preview Scene Details)
					->AddTab(NodeDetailsTabId, ETabState::OpenedTab)
					->SetExtensionId("NodeArea")
					->SetHideTabWell(false)
				)
			)
		);

	if(TObjectPtr<UDataflowBaseContent>& EditorContent = DataflowEditor->GetEditorContent())
	{
		if (EditorContent->GetDataflowAsset() && EditorContent->GetDataflowAsset()->Type == EDataflowType::Simulation)
		{
			StandaloneDefaultLayout = SimulationDefaultLayout;
			bForceViewportTab = false;
		}
		else
		{
			StandaloneDefaultLayout = ConstructionDefaultLayout;
			bForceViewportTab = true;
		}
	}

	// Add any extenders specified by the UISubsystem
	// The extenders provide defined locations for FModeToolkit to attach
	// tool palette tabs and detail panel tabs
	LayoutExtender = MakeShared<FLayoutExtender>();
	FDataflowEditorModule* Module = &FModuleManager::LoadModuleChecked<FDataflowEditorModule>("DataflowEditor");
	Module->OnRegisterLayoutExtensions().Broadcast(*LayoutExtender);
	StandaloneDefaultLayout->ProcessExtensions(*LayoutExtender);

	FAdvancedPreviewScene::ConstructionValues PreviewSceneArgs;
	PreviewSceneArgs.bShouldSimulatePhysics = 1;
	PreviewSceneArgs.bCreatePhysicsScene = 1;
	
	ConstructionScene = MakeShared<FDataflowConstructionScene>(PreviewSceneArgs, DataflowEditor);
	ObjectScene = ConstructionScene;

	SimulationScene = MakeShared<FDataflowSimulationScene>(PreviewSceneArgs, DataflowEditor);

	SimulationSceneProfileIndexStorage = MakeShared<FDataflowSimulationSceneProfileIndexStorage>(SimulationScene);

	IConsoleVariable* const ConsoleVar = IConsoleManager::Get().FindConsoleVariable(TEXT("p.Dataflow.EnableGraphEval"));
	bDataflowEnableGraphEval = ConsoleVar ? ConsoleVar->GetBool() : true;

	if (ConsoleVar)
	{
		GraphEvalCVarChangedDelegateHandle = ConsoleVar->OnChangedDelegate().AddLambda([this](IConsoleVariable* Var)
		{
			bDataflowEnableGraphEval = Var->GetBool();
		});
	}
}

FDataflowEditorToolkit::~FDataflowEditorToolkit()
{
	if (ConstructionScene && ConstructionScene->ModifyDebugDrawComponent())
	{
		ConstructionScene->ModifyDebugDrawComponent()->UnregisterComponent();
	}
	if (SimulationScene && SimulationScene->ModifyDebugDrawComponent())
	{
		SimulationScene->ModifyDebugDrawComponent()->UnregisterComponent();
	}

	if (IConsoleVariable* const ConsoleVar = IConsoleManager::Get().FindConsoleVariable(TEXT("p.Dataflow.EnableGraphEval")))
	{
		ConsoleVar->OnChangedDelegate().Remove(GraphEvalCVarChangedDelegateHandle);
	}

	if (SimulationScene && SimulationScene->GetPreviewSceneDescription())
	{
		SimulationScene->GetPreviewSceneDescription()->DataflowSimulationSceneDescriptionChanged.Remove(OnSimulationSceneChangedDelegateHandle);
	}

	if (GraphEditor)
	{
		GraphEditor->OnSelectionChangedMulticast.Remove(OnSelectionChangedMulticastDelegateHandle);
		GraphEditor->OnNodeDeletedMulticast.Remove(OnNodeDeletedMulticastDelegateHandle);
	}

	UnregisterContextHandlers();

	if (NodeDetailsEditor)
	{
		NodeDetailsEditor->GetOnFinishedChangingPropertiesDelegate().Remove(OnFinishedChangingPropertiesDelegateHandle);
	}

	if (AssetDetailsEditor)
	{
		AssetDetailsEditor->OnFinishedChangingProperties().Remove(OnFinishedChangingAssetPropertiesDelegateHandle);
	}

	if (DataflowOutputLog)
	{
		DataflowOutputLog->GetOnOutputLogMessageTokenClickedDelegate().Remove(OnOutputLogMessageTokenClickedDelegateHandle);
	}

	// We need to force the dataflow editor mode deletion now because otherwise the preview and rest-space worlds
	// will end up getting destroyed before the mode's Exit() function gets to run, and we'll get some
	// warnings when we destroy any mode actors.
	EditorModeManager->DestroyMode(UDataflowEditorMode::EM_DataflowEditorModeId);
	SimulationModeManager->DestroyMode(UDataflowEditorMode::EM_DataflowEditorModeId);
}

void FDataflowEditorToolkit::RegisterContextHandlers()
{
	const TObjectPtr<UDataflowBaseContent>& EditorContent = GetEditorContent();
	check(EditorContent);

	if (TSharedPtr<UE::Dataflow::FEngineContext> DataflowContext = EditorContent->GetDataflowContext())
	{
		OnNodeBeginEvaluateMulticastDelegateHandle = DataflowContext->OnNodeBeginEvaluateMulticast.AddSP(this, &FDataflowEditorToolkit::OnNodeBeginEvaluate);
		OnNodeFinishEvaluateMulticastDelegateHandle = DataflowContext->OnNodeFinishEvaluateMulticast.AddSP(this, &FDataflowEditorToolkit::OnNodeFinishEvaluate);

		OnContextLogDelegateHandle = DataflowContext->GetOnContextLogMulticast().AddSP(this, &FDataflowEditorToolkit::OnContextLog);
	}

	FDataflowAssetDelegates::OnNodeInvalidated.AddSP(this, &FDataflowEditorToolkit::OnNodeInvalidated);
	FDataflowAssetDelegates::OnDataflowPostEditUndo.AddSPLambda(this, [this, EditorContent](UDataflow& DataflowAsset)
		{
			// An undo/redo may have changed which Dataflow the owning asset references
			// (e.g. undoing the creation of a new embedded Dataflow on a Groom). If the
			// panel is no longer in sync with the owner, rebind the graph editor.
			if (EditorContent->GetDataflowAsset() != GetDataflowAsset(EditorContent->GetDataflowOwner()))
			{
				ResetEditor();
				
			}

			if (NodeDetailsEditor.IsValid() && IsValid(&DataflowAsset))
			{
				// Bug in SStructureDetailsView (UE-349909) causes the UI to get out of sync and requires the view to be recreated
				TSet<TObjectPtr<UObject>> SelectionSet;
				Algo::Transform(SelectedDataflowNodes, SelectionSet, [](const TObjectPtr<UDataflowEdNode>& EdNode) { return Cast<UObject>(EdNode); });
				FDataflowEditorCommands::OnSelectedNodesChanged(NodeDetailsEditor, &DataflowAsset, &DataflowAsset, SelectionSet);
			}
		});
}

void FDataflowEditorToolkit::UnregisterContextHandlers()
{
	const TObjectPtr<UDataflowBaseContent>& EditorContent = GetEditorContent();
	check(EditorContent);

	if (TSharedPtr<UE::Dataflow::FEngineContext> DataflowContext = EditorContent->GetDataflowContext())
	{
		DataflowContext->OnNodeBeginEvaluateMulticast.Remove(OnNodeBeginEvaluateMulticastDelegateHandle);
		DataflowContext->OnNodeFinishEvaluateMulticast.Remove(OnNodeFinishEvaluateMulticastDelegateHandle);

		DataflowContext->GetOnContextLogMulticast().Remove(OnContextLogDelegateHandle);
	}

	FDataflowAssetDelegates::OnNodeInvalidated.RemoveAll(this);
	FDataflowAssetDelegates::OnDataflowPostEditUndo.RemoveAll(this);
}

void FDataflowEditorToolkit::CreateEditorModeManager()
{
	auto SetSelectionName = [](const USelection* SelectionObject)
	{
		if(SelectionObject)
		{
			if(UTypedElementSelectionSet* SelectionSet = SelectionObject->GetElementSelectionSet())
			{
				const FName SelectionSetName(FString::Printf(TEXT("DataflowSelectionSet%p"), SelectionSet));
				SelectionSet->SetNameForTedsIntegration(SelectionSetName);
			}
		}
	};
	
	// Setup the construction manager / scene
	FBaseCharacterFXEditorToolkit::CreateEditorModeManager();
	static_cast<FDataflowPreviewSceneBase*>(ObjectScene.Get())->GetDataflowModeManager()
		= StaticCastSharedPtr<FAssetEditorModeManager>(EditorModeManager);

	SetSelectionName(EditorModeManager->GetSelectedActors());
	SetSelectionName(EditorModeManager->GetSelectedComponents());
	SetSelectionName(EditorModeManager->GetSelectedObjects());

	// Setup the simulation manager / scene
	SimulationModeManager = MakeShared<FAssetEditorModeManager>();
	StaticCastSharedPtr<FAssetEditorModeManager>(SimulationModeManager)->SetPreviewScene(
		SimulationScene.Get());
	static_cast<FDataflowPreviewSceneBase*>(SimulationScene.Get())->GetDataflowModeManager()
		= StaticCastSharedPtr<FAssetEditorModeManager>(SimulationModeManager);
	
	SetSelectionName(SimulationModeManager->GetSelectedActors());
	SetSelectionName(SimulationModeManager->GetSelectedComponents());
	SetSelectionName(SimulationModeManager->GetSelectedObjects());
}

void FDataflowEditorToolkit::NotifyPreChange(FEditPropertyChain* PropertyAboutToChange)
{
	if (const TObjectPtr<UDataflowBaseContent>& EditorContent = GetEditorContent())
	{
		ensure(EditorContent);
		if (UDataflow* DataflowAsset = EditorContent->GetDataflowAsset())
		{
			FDataflowEditorCommands::OnNotifyPropertyPreChange(NodeDetailsEditor, DataflowAsset, PropertyAboutToChange);
		}
	}
}

bool FDataflowEditorToolkit::CanOpenDataflowEditor(UObject* ObjectToEdit)
{
	return UE::Dataflow::InstanceUtils::HasValidDataflowAsset(ObjectToEdit);
}

bool FDataflowEditorToolkit::HasDataflowAsset(UObject* ObjectToEdit)
{
	return (GetDataflowAsset(ObjectToEdit) != nullptr);
}

UDataflow* FDataflowEditorToolkit::GetDataflowAsset(UObject* ObjectToEdit)
{
	return UE::Dataflow::InstanceUtils::GetDataflowAssetFromObject(ObjectToEdit);
}

const UDataflow* FDataflowEditorToolkit::GetDataflowAsset(const UObject* ObjectToEdit)
{
	return UE::Dataflow::InstanceUtils::GetDataflowAssetFromObject(ObjectToEdit);
}

bool FDataflowEditorToolkit::IsSimulationDataflowAsset() const
{
	if (const TObjectPtr<UDataflowBaseContent>& EditorContent = GetEditorContent())
	{
		return (EditorContent->GetDataflowAsset() && EditorContent->GetDataflowAsset()->Type == EDataflowType::Simulation);
	}
	return false;
}

FName FDataflowEditorToolkit::GetGraphLogName() const
{
	static FName CSimulationGraphName = TEXT("Simulation");
	static FName CConstructionGraphName = TEXT("Construction");

	if (IsSimulationDataflowAsset())
	{
		return CSimulationGraphName;
	}

	return CConstructionGraphName;
}

//~ Begin FBaseCharacterFXEditorToolkit overrides

FEditorModeID FDataflowEditorToolkit::GetEditorModeId() const
{
	return UDataflowEditorMode::EM_DataflowEditorModeId;
}

TObjectPtr<UDataflowBaseContent>& FDataflowEditorToolkit::GetEditorContent()
{
	return DataflowEditor->GetEditorContent();
}

const TObjectPtr<UDataflowBaseContent>& FDataflowEditorToolkit::GetEditorContent() const
{
	return DataflowEditor->GetEditorContent();
}

TArray<TObjectPtr<UDataflowBaseContent>>& FDataflowEditorToolkit::GetTerminalContents()
{
	return DataflowEditor->GetTerminalContents();
}

const TArray<TObjectPtr<UDataflowBaseContent>>& FDataflowEditorToolkit::GetTerminalContents() const
{
	return DataflowEditor->GetTerminalContents();
}

bool FDataflowEditorToolkit::OnRequestClose(EAssetEditorCloseReason InCloseReason)
{
	// Note: This needs a bit of adjusting, because currently OnRequestClose seems to be 
	// called multiple times when the editor itself is being closed. We can take the route 
	// of NiagaraScriptToolkit and remember when changes are discarded, but this can cause
	// issues if the editor close sequence is interrupted due to some other asset editor.

	UDataflowEditorMode* DataflowEdMode = Cast<UDataflowEditorMode>(EditorModeManager->GetActiveScriptableMode(UDataflowEditorMode::EM_DataflowEditorModeId));
	if (!DataflowEdMode) {
		// If we don't have a valid mode, because the OnRequestClose is currently being called multiple times,
		// simply return true because there's nothing left to do.
		return true;
	}
	
	return FAssetEditorToolkit::OnRequestClose(InCloseReason);
}

/** Called when this toolkit is being closed */
void FDataflowEditorToolkit::OnClose()
{
	// Tear down property gizmos before the mode's tools context is destroyed.
	if (PropertyGizmoManager && PropertyGizmoManager->HasActiveGizmos())
	{
		if (UDataflowEditorMode* const DataflowMode = Cast<UDataflowEditorMode>(EditorModeManager->GetActiveScriptableMode(UDataflowEditorMode::EM_DataflowEditorModeId)))
		{
			PropertyGizmoManager->Teardown(DataflowMode->GetInteractiveToolsContext()->ToolManager);
		}
	}

	// Give any active modes a chance to shutdown while the toolkit host is still alive
	// This is super important to do, otherwise currently opened tabs won't be marked as "closed".
	// This results in tabs not being properly recycled upon reopening the editor and tab
	// duplication for each opening event.
	GetEditorModeManager().ActivateDefaultMode();

	FBaseCharacterFXEditorToolkit::OnClose();
}

void FDataflowEditorToolkit::PostInitAssetEditor()
{
	FBaseCharacterFXEditorToolkit::PostInitAssetEditor();

	PropertyGizmoManager = NewObject<UDataflowPropertyGizmoManager>();
	
	auto SetCommonViewportClientOptions = [](FEditorViewportClient* Client)
	{
		// Normally the bIsRealtime flag is determined by whether the connection is remote, but our
		// tools require always being ticked.
		Client->SetRealtime(true);

		// Disable motion blur effects that cause our renders to "fade in" as things are moved
		Client->EngineShowFlags.SetTemporalAA(false);
		Client->EngineShowFlags.SetAntiAliasing(true);
		Client->EngineShowFlags.SetMotionBlur(false);

		// Enable the dithering of occluded selection and portions of gizmos.
		Client->EngineShowFlags.SetOpaqueCompositeEditorPrimitives(false);

		// Disable hardware occlusion queries, which make it harder to use vertex shaders to pull materials
		// toward camera for z ordering because non-translucent materials start occluding themselves (once
		// the component bounds are behind the displaced geometry).
		Client->EngineShowFlags.SetDisableOcclusionQueries(true);

		// Ortho has too many problems with rendering things, unfortunately, so we should use perspective.
		Client->SetViewportType(ELevelViewportType::LVT_Perspective);

		// Lit gives us the most options in terms of the materials we can use.
		Client->SetViewMode(EViewModeIndex::VMI_Lit);

		// We need the viewport client to start out focused, or else it won't get ticked until
		// we click inside it.
		if(Client->Viewport)
		{
			Client->ReceivedFocus(Client->Viewport);
		}
	};
	SetCommonViewportClientOptions(ViewportClient.Get());
	SetCommonViewportClientOptions(SimulationViewportClient.Get());


	UDataflowEditorMode* const DataflowMode = CastChecked<UDataflowEditorMode>(EditorModeManager->GetActiveScriptableMode(UDataflowEditorMode::EM_DataflowEditorModeId));
	const TWeakPtr<FViewportClient> WeakConstructionViewportClient(ViewportClient);
	DataflowMode->SetConstructionViewportClient(StaticCastWeakPtr<FDataflowConstructionViewportClient>(WeakConstructionViewportClient));
	const TWeakPtr<FViewportClient> WeakSimulationViewportClient(SimulationViewportClient);
	DataflowMode->SetSimulationViewportClient(StaticCastWeakPtr<FDataflowSimulationViewportClient>(WeakSimulationViewportClient));

	FDataflowConstructionViewportClient* ConstructionViewportClient = static_cast<FDataflowConstructionViewportClient*>(ViewportClient.Get());
	OnConstructionSelectionChangedDelegateHandle = ConstructionViewportClient->OnSelectionChangedMulticast.AddSP(this, &FDataflowEditorToolkit::OnConstructionViewSelectionChanged);
	OnSimulationSelectionChangedDelegateHandle = SimulationViewportClient->OnSelectionChangedMulticast.AddSP(this, &FDataflowEditorToolkit::OnSimulationViewSelectionChanged);

	// Populate editor toolbar

	FName ParentToolbarName;
	const FName ToolBarName = GetToolMenuToolbarName(ParentToolbarName);
	UToolMenu* const AssetToolbar = UToolMenus::Get()->ExtendMenu(ToolBarName);
	FToolMenuSection& Section = AssetToolbar->FindOrAddSection("DataflowTools");

	AddEvaluationWidget(Section);
	AddDataflowActionWidget(Section);

	// Force scenes to update loaded asset viewer settings
	UAssetViewerSettings::Get()->OnAssetViewerSettingsChanged().Broadcast(NAME_None);

	// Enforce the timeline min size even when restoring an older saved layout that predates the MinSize field.
	// The SSplitter slot binds GetNodeMinSize() as a TAttribute, so setting it here takes effect immediately.
	if (TSharedPtr<SDockTab> TimelineTab = TabManager->FindExistingLiveTab(SimulationTimelineTabId))
	{
		if (TSharedPtr<SDockingTabStack> TimelineStack = TimelineTab->GetParentDockTabStack())
		{
			TimelineStack->SetNodeMinSize(UE::Dataflow::Private::DataflowLayoutTimelineMaxHeight);
		}
	}
}

void FDataflowEditorToolkit::SetEvaluateGraphMode(EDataflowEditorEvaluationMode Mode)
{
	if (Mode != EvaluationMode)
	{
		EvaluationMode = Mode;
		if (UDataflowEditorOptions* const Options = UDataflowEditorOptions::StaticClass()->GetDefaultObject<UDataflowEditorOptions>())
		{
			Options->EditorEvaluationMode = EvaluationMode;
		}
		// when going back to automatic, make sure the graph is up to date
		if (EvaluationMode == EDataflowEditorEvaluationMode::Automatic)
		{
			EvaluateGraph();
		}
	}
}

void FDataflowEditorToolkit::TogglePerfData()
{
	if (UDataflowEditorUserSettings* Settings = GetMutableDefault<UDataflowEditorUserSettings>())
	{
		Settings->bShowPerformanceData = !Settings->bShowPerformanceData;
		Settings->PostEditChange();
		Settings->SaveConfig();
	}
}

bool FDataflowEditorToolkit::IsPerfDataEnabled() const
{
	if (UDataflowEditorUserSettings* Settings = GetMutableDefault<UDataflowEditorUserSettings>())
	{
		return Settings->bShowPerformanceData;
	}
	return false;
}

void FDataflowEditorToolkit::ToggleAsyncEvaluation()
{
	if (UDataflowEditorUserSettings* Settings = GetMutableDefault<UDataflowEditorUserSettings>())
	{
		Settings->bUseAsyncEvaluation = !Settings->bUseAsyncEvaluation;
		Settings->PostEditChange();
		Settings->SaveConfig();
	}
}

bool FDataflowEditorToolkit::IsAsyncEvaluationEnabled() const
{
	if (UDataflowEditorUserSettings* Settings = GetMutableDefault<UDataflowEditorUserSettings>())
	{
		return Settings->bUseAsyncEvaluation;
	}
	return false;
}

void FDataflowEditorToolkit::ClearGraphCache()
{
	if (TObjectPtr<UDataflowBaseContent>& EditorContent = GetEditorContent())
	{
		if (TSharedPtr<UE::Dataflow::FEngineContext> DataflowContext = EditorContent->GetDataflowContext())
		{
			DataflowContext->ClearAllData();
			DataflowContext->ClearAllPerfData();
		}
	}
}

bool FDataflowEditorToolkit::CanClearGraphCache() const
{
	if (const TObjectPtr<UDataflowBaseContent>& EditorContent = GetEditorContent())
	{
		if (TSharedPtr<UE::Dataflow::FEngineContext> DataflowContext = EditorContent->GetDataflowContext())
		{
			return !DataflowContext->IsEmpty();
		}
	}
	return false;
}

FSlateIcon FDataflowEditorToolkit::GetEvaluationStatusImage() const
{
	static const FName CompileStatusBackground("Blueprint.CompileStatus.Background");
	static const FName CompileStatusUnknown("Blueprint.CompileStatus.Overlay.Unknown");
	static const FName CompileStatusGood("Blueprint.CompileStatus.Overlay.Good");
	static const FName CompileStatusWarning("Blueprint.CompileStatus.Overlay.Warning");
	static const FName CompileStatusError("Blueprint.CompileStatus.Overlay.Error");

	FName OverlayIcon;
	if (IsGraphDirty())
	{
		OverlayIcon = CompileStatusUnknown;
	}
	else
	{
		const EMessageSeverity::Type Severity = LogMessages->GetMaxSeverity();
		switch (Severity)
		{
		case EMessageSeverity::Error:
			OverlayIcon = CompileStatusError;
			break;
		case EMessageSeverity::PerformanceWarning:
		case EMessageSeverity::Warning:
			OverlayIcon = CompileStatusWarning;
			break;
		case EMessageSeverity::Info:
		default:
			OverlayIcon = CompileStatusGood;
			break;
		}
	}
	return FSlateIcon(FAppStyle::GetAppStyleSetName(), CompileStatusBackground, NAME_None, OverlayIcon);
}

void FDataflowEditorToolkit::AddEvaluationWidget(FToolMenuSection& InSection)
{
	ToolkitCommands->MapAction(FDataflowEditorCommands::Get().EvaluateGraph,
		FExecuteAction::CreateSP(this, &FDataflowEditorToolkit::EvaluateGraph),
		FCanExecuteAction::CreateSP(this, &FDataflowEditorToolkit::IsEvaluateButtonEnabled)
	);
	ToolkitCommands->MapAction(FDataflowEditorCommands::Get().EvaluateGraphAutomatic,
		FExecuteAction::CreateSP(this, &FDataflowEditorToolkit::SetEvaluateGraphMode, EDataflowEditorEvaluationMode::Automatic),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([this]() { return EvaluationMode == EDataflowEditorEvaluationMode::Automatic; })
	);
	ToolkitCommands->MapAction(FDataflowEditorCommands::Get().EvaluateGraphManual,
		FExecuteAction::CreateSP(this, &FDataflowEditorToolkit::SetEvaluateGraphMode, EDataflowEditorEvaluationMode::Manual),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([this]() { return EvaluationMode == EDataflowEditorEvaluationMode::Manual; })
	);
	ToolkitCommands->MapAction(FDataflowEditorCommands::Get().ClearGraphCache,  //  TODO: Rename ClearGraphCache to ForceGraphEvaluation in 5.7
		FExecuteAction::CreateLambda([this]() { ClearGraphCache(); EvaluateGraph(); }),  // Note: This action used to only clear the cache, but has been changed to force a re evaluation
		FCanExecuteAction()
	);
	ToolkitCommands->MapAction(FDataflowEditorCommands::Get().TogglePerfData,
		FExecuteAction::CreateSP(this, &FDataflowEditorToolkit::TogglePerfData),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FDataflowEditorToolkit::IsPerfDataEnabled)
	);
	ToolkitCommands->MapAction(FDataflowEditorCommands::Get().ToggleAsyncEvaluation,
		FExecuteAction::CreateSP(this, &FDataflowEditorToolkit::ToggleAsyncEvaluation),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FDataflowEditorToolkit::IsAsyncEvaluationEnabled)
	);
	ToolkitCommands->MapAction(FDataflowEditorCommands::Get().ToggleSimulation,
		FExecuteAction::CreateSP(this, &FDataflowEditorToolkit::ToggleDataflowSimulation),
		FCanExecuteAction::CreateSP(this, &FDataflowEditorToolkit::HasSimulationManager),
		FIsActionChecked::CreateSP(this, &FDataflowEditorToolkit::IsSimulationEnabled)
	);
	ToolkitCommands->MapAction(FDataflowEditorCommands::Get().ResetSimulation,
		FExecuteAction::CreateSP(this, &FDataflowEditorToolkit::ResetDataflowSimulation),
		FCanExecuteAction::CreateSP(this, &FDataflowEditorToolkit::HasSimulationManager),
		FIsActionChecked()
	);
	ToolkitCommands->MapAction(FDataflowEditorCommands::Get().StartSimulation,
		FExecuteAction::CreateSP(this, &FDataflowEditorToolkit::StartDataflowSimulation),
		FCanExecuteAction::CreateSP(this, &FDataflowEditorToolkit::IsSimulationDisabled),
		FIsActionChecked()
	);
	ToolkitCommands->MapAction(FDataflowEditorCommands::Get().StopSimulation,
		FExecuteAction::CreateSP(this, &FDataflowEditorToolkit::StopDataflowSimulation),
		FCanExecuteAction::CreateSP(this, &FDataflowEditorToolkit::IsSimulationEnabled),
		FIsActionChecked()
	);
	ToolkitCommands->MapAction(FDataflowEditorCommands::Get().StepSimulation,
		FExecuteAction::CreateSP(this, &FDataflowEditorToolkit::StepDataflowSimulation),
		FCanExecuteAction::CreateSP(this, &FDataflowEditorToolkit::IsSimulationDisabled),
		FIsActionChecked()
	);
	
	InSection.AddDynamicEntry("DataflowEvaluateGraphDynamic", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& DynamicSection)
		{
			using namespace UE::Dataflow::Private;
			if (const TSharedPtr<FDataflowEditorToolkit> Toolkit = GetAssetEditorToolkitFromMenuContext(DynamicSection.FindContext<UAssetEditorToolkitMenuContext>()))
			{
				DynamicSection.AddEntry(FToolMenuEntry::InitToolBarButton(
						FDataflowEditorCommands::Get().EvaluateGraph,
						TAttribute<FText>(),
						TAttribute<FText>(),
						TAttribute<FSlateIcon>(Toolkit.ToSharedRef(), &FDataflowEditorToolkit::GetEvaluationStatusImage)
					));
			}
		}));

	InSection.AddDynamicEntry("DataflowEvaluationOptionsDynamic", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& DynamicSection)
		{
			using namespace UE::Dataflow::Private;
			if (const TSharedPtr<FDataflowEditorToolkit> Toolkit = GetAssetEditorToolkitFromMenuContext(DynamicSection.FindContext<UAssetEditorToolkitMenuContext>()))
			{
				FToolMenuEntry EvaluationOptions = FToolMenuEntry::InitComboButton(
					"DataflowEvaluationOptions",
					FUIAction(),
					FOnGetContent::CreateSP(Toolkit.ToSharedRef(), &FDataflowEditorToolkit::GenerateEvaluationOptionsMenu),
					LOCTEXT("DataflowEvaluationOptions", "Options"),
					LOCTEXT("DataflowEvaluationOptions_ToolbarTooltip", "Options to customize how the Dataflow Graph evaluates"),
					TAttribute<FSlateIcon>(),
					true
				);
				EvaluationOptions.StyleNameOverride = "SlimToolBar";
				DynamicSection.AddEntry(EvaluationOptions);
			}
		}));

	FToolMenuEntry PlayEntry = FToolMenuEntry::InitToolBarButton(
					FDataflowEditorCommands::Get().StartSimulation,
					TAttribute<FText>(),
					TAttribute<FText>(),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "PlayWorld.PlayInViewport"));

	PlayEntry.StyleNameOverride = FName("Toolbar.BackplateLeftPlay");

	InSection.AddEntry(PlayEntry);

	FToolMenuEntry StepEntry = FToolMenuEntry::InitToolBarButton(
	FDataflowEditorCommands::Get().StepSimulation,
					TAttribute<FText>(),
					TAttribute<FText>(),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "PlayWorld.SingleFrameAdvance.Small"));

	StepEntry.StyleNameOverride = FName("Toolbar.BackplateCenter");
				
	InSection.AddEntry(StepEntry);

	FToolMenuEntry StopEntry = FToolMenuEntry::InitToolBarButton(
	FDataflowEditorCommands::Get().StopSimulation,
					TAttribute<FText>(),
					TAttribute<FText>(),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "PlayWorld.StopPlaySession.Small"));

	StopEntry.StyleNameOverride = FName("Toolbar.BackplateCenterStop");

	InSection.AddEntry(StopEntry);

	FToolMenuEntry ResetEntry = FToolMenuEntry::InitToolBarButton(
	FDataflowEditorCommands::Get().ResetSimulation,
				TAttribute<FText>(),
				TAttribute<FText>(),
	FSlateIcon(FDataflowEditorStyle::Get().GetStyleSetName(),"Dataflow.ResetSimulation"));

	ResetEntry.StyleNameOverride = FName("Toolbar.BackplateRight");

	InSection.AddEntry(ResetEntry);

	// load options
	if (UDataflowEditorOptions* const Options = UDataflowEditorOptions::StaticClass()->GetDefaultObject<UDataflowEditorOptions>())
	{
		EvaluationMode = Options->EditorEvaluationMode;
	}
}

TSharedRef<SWidget> FDataflowEditorToolkit::GenerateEvaluationOptionsMenu()
{
	FMenuBuilder MenuBuilder(true, GetToolkitCommands());
	MenuBuilder.BeginSection(TEXT("Section"));
	MenuBuilder.AddMenuEntry(FDataflowEditorCommands::Get().EvaluateGraphAutomatic);
	MenuBuilder.AddMenuEntry(FDataflowEditorCommands::Get().EvaluateGraphManual);
	MenuBuilder.AddSeparator();
	MenuBuilder.AddMenuEntry(FDataflowEditorCommands::Get().ClearGraphCache);
	MenuBuilder.AddSeparator();
	MenuBuilder.AddMenuEntry(FDataflowEditorCommands::Get().TogglePerfData);
	MenuBuilder.AddMenuEntry(FDataflowEditorCommands::Get().ToggleAsyncEvaluation);
	MenuBuilder.EndSection();
	return MenuBuilder.MakeWidget();
}

bool FDataflowEditorToolkit::IsEvaluateButtonEnabled() const
{
	// Button is always enabled in manual mode or in automatic mode if the graph is dirty
	return (EvaluationMode == EDataflowEditorEvaluationMode::Manual || IsGraphDirty());
}

void FDataflowEditorToolkit::AddDataflowActionWidget(FToolMenuSection& InSection)
{
	InSection.AddSeparator(TEXT("Dataflow Actions"));

	// Convert to embedded Dataflow button
	ToolkitCommands->MapAction(FDataflowEditorCommands::Get().ConvertToEmbeddedDataflow,
		FExecuteAction::CreateSP(this, &FDataflowEditorToolkit::ConvertToEmbeddedDataflow),
		FCanExecuteAction::CreateSP(this, &FDataflowEditorToolkit::IsExternalDataflow),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateSP(this, &FDataflowEditorToolkit::IsExternalDataflow)
	);

	InSection.AddDynamicEntry("DataflowActionConvertToEmbed", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& DynamicSection)
		{
			DynamicSection.AddEntry(FToolMenuEntry::InitToolBarButton(
				FDataflowEditorCommands::Get().ConvertToEmbeddedDataflow,
				TAttribute<FText>(),
				TAttribute<FText>(),
				FSlateIcon()
			));
		}));

	// Convert to external Dataflow button
	ToolkitCommands->MapAction(FDataflowEditorCommands::Get().ConvertToExternalDataflow,
		FExecuteAction::CreateSP(this, &FDataflowEditorToolkit::ConvertToExternalDataflow),
		FCanExecuteAction::CreateSP(this, &FDataflowEditorToolkit::IsEmbeddedDataflow),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateSP(this, &FDataflowEditorToolkit::IsEmbeddedDataflow)
	);

	InSection.AddDynamicEntry("DataflowActionConvertToExternal", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& DynamicSection)
		{
			DynamicSection.AddEntry(FToolMenuEntry::InitToolBarButton(
				FDataflowEditorCommands::Get().ConvertToExternalDataflow,
				TAttribute<FText>(),
				TAttribute<FText>(),
				FSlateIcon()
			));
		}));

	// Lock/Unlock editing button
	ToolkitCommands->MapAction(FDataflowEditorCommands::Get().ToggleDataflowEditingLock,
		FExecuteAction::CreateSP(this, &FDataflowEditorToolkit::ToggleDataflowEditingLock),
		FCanExecuteAction::CreateSP(this, &FDataflowEditorToolkit::IsExternalDataflow),
		FIsActionChecked::CreateSP(this, &FDataflowEditorToolkit::IsEditable),
		FIsActionButtonVisible::CreateSP(this, &FDataflowEditorToolkit::IsEditingLockButtonVisible)
	);

	InSection.AddDynamicEntry("DataflowActionLockUnlockGraph", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& DynamicSection)
		{
			using namespace UE::Dataflow::Private;
			if (const TSharedPtr<FDataflowEditorToolkit> Toolkit = GetAssetEditorToolkitFromMenuContext(DynamicSection.FindContext<UAssetEditorToolkitMenuContext>()))
			{
				const FSlateIcon UnlockedIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Unlock");
				const FSlateIcon LockedIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Lock");
				const FText UnlockedText = LOCTEXT("LockEditing_Text", "Lock Editing");
				const FText UnlockedTooltip = LOCTEXT("lockEditing_Tooltip", "Lock Editing for this external Dataflow asset so that other assets using will not be affected by any potential changes (Default behavior)");
				const FText LockedText = LOCTEXT("UnlockEditing_Text", "Unlock Editing");
				const FText LockedTooltip = LOCTEXT("UnlockEditing_Tooltip", "Unlock Editing for this external Dataflow asset\nWarning : this asset may be used by other asset and changes made will affect them");

				TWeakPtr<FDataflowEditorToolkit> WeakToolkit = Toolkit;
				DynamicSection.AddEntry(FToolMenuEntry::InitToolBarButton(
					FDataflowEditorCommands::Get().ToggleDataflowEditingLock,
					TAttribute<FText>::CreateLambda([WeakToolkit, UnlockedText, LockedText]() 
						{ 
							if (TSharedPtr<FDataflowEditorToolkit> ToolkitPtr = WeakToolkit.Pin())
							{
								return ToolkitPtr->AllowExternalDataflowEditing() ? UnlockedText : LockedText;
							}
							return UnlockedText;
						}),
					TAttribute<FText>::CreateLambda([WeakToolkit, UnlockedTooltip, LockedTooltip]() 
						{ 
							if (TSharedPtr<FDataflowEditorToolkit> ToolkitPtr = WeakToolkit.Pin())
							{
								return ToolkitPtr->AllowExternalDataflowEditing() ? UnlockedTooltip : LockedTooltip;
							}
							return UnlockedTooltip;
						}),
					TAttribute<FSlateIcon>::CreateLambda([WeakToolkit, UnlockedIcon, LockedIcon]()
						{ 
							if (TSharedPtr<FDataflowEditorToolkit> ToolkitPtr = WeakToolkit.Pin())
							{
								return ToolkitPtr->AllowExternalDataflowEditing() ? UnlockedIcon : LockedIcon;
							}
							return UnlockedIcon;
						})
				));
			}
		}));

	// Open reference asset button 
	InSection.AddDynamicEntry("DataflowActionOpenReferenceAsset", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& DynamicSection)
		{
			using namespace UE::Dataflow::Private;
			if (const TSharedPtr<FDataflowEditorToolkit> Toolkit = GetAssetEditorToolkitFromMenuContext(DynamicSection.FindContext<UAssetEditorToolkitMenuContext>()))
			{
				TWeakPtr<FDataflowEditorToolkit> WeakToolkit = Toolkit;

				DynamicSection.AddEntry(FToolMenuEntry::InitToolBarButton(
					TEXT("DataflowOpenReferenceAsset"),
					FUIAction(
						FExecuteAction::CreateLambda([WeakToolkit]()
							{
								if (TSharedPtr<FDataflowEditorToolkit> ToolkitPtr = WeakToolkit.Pin())
								{
									if (UObject* ReferenceAsset = ToolkitPtr->GetReferenceAsset())
									{
										GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(ReferenceAsset);
									}
								}
							}
						),
						FCanExecuteAction::CreateLambda([WeakToolkit]()
							{
								if (TSharedPtr<FDataflowEditorToolkit> ToolkitPtr = WeakToolkit.Pin())
								{
									return !ToolkitPtr->IsReferenceAssetNullOrSameAsBoundAsset();
								}
								return false;
							}
						)
					),
					TAttribute<FText>(LOCTEXT("OpenReferenceAssetLabel", "Open Reference Asset")),
					LOCTEXT("OpenReferenceAssetTooltip", "Open reference asset in an editor"),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "GraphEditor.EventGraph_16x")
				));
			}
		}));
}

bool FDataflowEditorToolkit::IsEmbeddedDataflow() const
{
	if (const TObjectPtr<UDataflowBaseContent>& EditorContent = GetEditorContent())
	{
		if (const UDataflow* DataflowAsset = EditorContent->GetDataflowAsset())
		{
			return !DataflowAsset->IsAsset();
		}
	}
	return false;
}

bool FDataflowEditorToolkit::IsExternalDataflow() const
{
	return !IsEmbeddedDataflow();
}

UObject* FDataflowEditorToolkit::GetReferenceAsset() const
{
	if (const TObjectPtr<UDataflowBaseContent>& EditorContent = GetEditorContent())
	{
		if (const UDataflow* DataflowAsset = EditorContent->GetDataflowAsset())
		{
			return DataflowAsset->ReferenceAsset;
		}
	}
	return nullptr;
}

bool FDataflowEditorToolkit::IsReferenceAssetNullOrSameAsBoundAsset() const
{
	UObject* ReferenceAsset = nullptr;
	bool bIsReferenceAssetNullOrSameAsBoundAsset = true;
	if (const TObjectPtr<UDataflowBaseContent>& EditorContent = GetEditorContent())
	{
		if (const UDataflow* DataflowAsset = EditorContent->GetDataflowAsset())
		{
			ReferenceAsset = DataflowAsset->ReferenceAsset;
		}
		TObjectPtr<UObject> BoundAsset = EditorContent->GetDataflowOwner();
		bIsReferenceAssetNullOrSameAsBoundAsset = (ReferenceAsset == nullptr) || (BoundAsset == ReferenceAsset);
	}
	return bIsReferenceAssetNullOrSameAsBoundAsset;
}

bool FDataflowEditorToolkit::AllowExternalDataflowEditing() const
{
	return IsExternalDataflow() && bAllowExternalDataflowEditing && IsReferenceAssetNullOrSameAsBoundAsset();
}

bool FDataflowEditorToolkit::IsEditable() const
{
	return (IsEmbeddedDataflow() || AllowExternalDataflowEditing() || bDataflowForceGraphEditable);
}

bool FDataflowEditorToolkit::IsEditingLockButtonVisible() const
{
	return IsExternalDataflow() && IsReferenceAssetNullOrSameAsBoundAsset() && !bDataflowForceGraphEditable;
}

void FDataflowEditorToolkit::ConvertToEmbeddedDataflow()
{
	ensure(IsExternalDataflow());

	if (const TObjectPtr<UDataflowBaseContent>& EditorContent = GetEditorContent())
	{
		if (TObjectPtr<UObject> BoundAsset = EditorContent->GetDataflowOwner())
		{
			const TObjectPtr<UDataflow> PreviousDataflowAsset = EditorContent->GetDataflowAsset();

			UDataflow* NewDataflowAsset = nullptr;
			if (UE::DataflowAssetDefinitionHelpers::EmbedDataflowAsset(BoundAsset, NewDataflowAsset))
			{
				BoundAsset->Modify();

				// Make sure the previous dataflow asset is no longer in the list of editing object to avoid confusing the save/save commands
				if (PreviousDataflowAsset)
				{
					RemoveEditingObject(PreviousDataflowAsset);
				}

				ResetEditor();
			}
		}
	}
}

void FDataflowEditorToolkit::ConvertToExternalDataflow()
{
	ensure(IsEmbeddedDataflow());
	if (const TObjectPtr<UDataflowBaseContent>& EditorContent = GetEditorContent())
	{
		if (TObjectPtr<UObject> BoundAsset = EditorContent->GetDataflowOwner())
		{
			const TObjectPtr<UDataflow> PreviousDataflowAsset = EditorContent->GetDataflowAsset();

			UDataflow* NewDataflowAsset = nullptr;
			if (UE::DataflowAssetDefinitionHelpers::MakeShareableDataflowAsset(BoundAsset, NewDataflowAsset))
			{
				// Make sure the previous dataflow asset is no longer in the list of editing object to avoid confusing the save/save commands
				if (PreviousDataflowAsset)
				{
					RemoveEditingObject(PreviousDataflowAsset);
				}

				ResetEditor();
			}
		}
	}
}

void FDataflowEditorToolkit::ToggleDataflowEditingLock()
{
	if (IsExternalDataflow() && IsReferenceAssetNullOrSameAsBoundAsset())
	{
		bAllowExternalDataflowEditing = !bAllowExternalDataflowEditing;
	}
}

void FDataflowEditorToolkit::InitializeEdMode(UBaseCharacterFXEditorMode* EdMode)
{
	UDataflowEditorMode* DataflowMode = Cast<UDataflowEditorMode>(EdMode);
	check(DataflowMode);
	DataflowMode->SetDataflowEditor(DataflowEditor);

	// We first set the preview scene in order to store the dynamic mesh elements
	// generated by the tools
	DataflowMode->SetDataflowConstructionScene(ConstructionScene);

	checkf(SimulationScene, TEXT("Expected SimulationScene to have been created in FDataflowEditorToolkit constructor"));
	DataflowMode->SetDataflowSimlationScene(SimulationScene);

	// Set of the graph editor to be able to add nodes
	DataflowMode->SetDataflowGraphEditor(GraphEditor);
	TArray<TObjectPtr<UObject>> ObjectsToEdit;
	OwningAssetEditor->GetObjectsToEdit(MutableView(ObjectsToEdit));
	DataflowMode->InitializeTargets(ObjectsToEdit);

	if (TSharedPtr<FModeToolkit> ModeToolkit = DataflowMode->GetToolkit().Pin())
	{
		FDataflowEditorModeToolkit* DataflowModeToolkit = static_cast<FDataflowEditorModeToolkit*>(ModeToolkit.Get());
		DataflowModeToolkit->SetConstructionViewportWidget(DataflowConstructionViewport);
		DataflowModeToolkit->SetSimulationViewportWidget(DataflowSimulationViewport);
	}

	// @todo(brice) : This used to crash when comnmented out. 
	FBaseCharacterFXEditorToolkit::InitializeEdMode(EdMode);
}

void FDataflowEditorToolkit::CreateEditorModeUILayer()
{
	FBaseCharacterFXEditorToolkit::CreateEditorModeUILayer();
}

void FDataflowEditorToolkit::GetSaveableObjects(TArray<UObject*>& OutObjects) const
{
	FBaseCharacterFXEditorToolkit::GetSaveableObjects(OutObjects);

	if (ensure(GetEditorContent()))
	{
		if (UDataflow* DataflowAsset = GetEditorContent()->GetDataflowAsset())
		{
			if (DataflowAsset->IsAsset())
			{
				OutObjects.AddUnique(DataflowAsset);
			}
			if (DataflowAsset->PreviewCacheAsset.IsValid())
			{
				OutObjects.AddUnique(DataflowAsset->PreviewCacheAsset.Get());
			}
			if (DataflowAsset->PreviewGeometryCacheAsset.IsValid())
			{
				OutObjects.AddUnique(DataflowAsset->PreviewGeometryCacheAsset.Get());
			}
		}
	}
}

void FDataflowEditorToolkit::OnAssetsSavedAs(const TArray<UObject*>& SavedObjects)
{
	// Find new assets 
	UDataflow* NewDataflowAsset = nullptr;
	UObject* NewAssetWithInterface = nullptr;
	IDataflowInstanceInterface* NewDataflowInterface = nullptr;
	UChaosCacheCollection* NewChaosCache = nullptr;
	UGeometryCache* NewGeomCache = nullptr;
	for (UObject* const SavedObj : SavedObjects)
	{
		if (SavedObj->IsA<UDataflow>())
		{
			NewDataflowAsset = Cast<UDataflow>(SavedObj);
		}
		else if (IDataflowInstanceInterface* Interface = UE::Dataflow::InstanceUtils::GetDataflowInterfaceFromObject(SavedObj))
		{
			NewAssetWithInterface = SavedObj;
			NewDataflowInterface = Interface;
		}
		else if (SavedObj->IsA<UChaosCacheCollection>())
		{
			NewChaosCache = Cast<UChaosCacheCollection>(SavedObj);
		}
		else if (SavedObj->IsA<UGeometryCache>())
		{
			NewGeomCache = Cast<UGeometryCache>(SavedObj);
		}
	}

	TArray<UPackage*> PackagesToSave;
	if (NewDataflowAsset && (NewChaosCache || NewGeomCache))
	{
		NewDataflowAsset->PreviewCacheAsset = NewChaosCache;
		NewDataflowAsset->PreviewGeometryCacheAsset = NewGeomCache;

		// Now save the new Dataflow asset again since we've updated its Property
		NewDataflowAsset->MarkPackageDirty();
		PackagesToSave.AddUnique(NewDataflowAsset->GetOutermost());
	}
	if (NewDataflowAsset && NewDataflowInterface && NewAssetWithInterface)
	{
		NewDataflowInterface->GetDataflowInstance().SetDataflowAsset(NewDataflowAsset);

		// Now save the new IDataflowInstanceInterface asset again since we've updated its Property
		NewAssetWithInterface->MarkPackageDirty();
		PackagesToSave.Add(NewAssetWithInterface->GetOutermost());
	}

	if (!PackagesToSave.IsEmpty())
	{
		constexpr bool bPromptToSave = false;
		FEditorFileUtils::PromptForCheckoutAndSave(PackagesToSave, bCheckDirtyOnAssetSave, bPromptToSave);

		// Reload the package so the editor can update anything that references the Dataflow asset
		// (Sadly this function is only called after the editor re-launches as part of the Save As process, so the editor is already open at this point.)
		UPackageTools::ReloadPackages(PackagesToSave);
	}
}

bool FDataflowEditorToolkit::ShouldReopenEditorForSavedAsset(const UObject* Asset) const
{
	// Do not re-open known dependent assets.
	if (Asset->IsA<UDataflow>())
	{
		return UE::DataflowAssetDefinitionHelpers::CanOpenDataflowAssetInEditor(Asset);
	}
	if (Asset->IsA<UChaosCacheCollection>())
	{
		return false;
	}
	if (Asset->IsA<UGeometryCache>())
	{
		return false;
	}
	return true;
}

//~ End FBaseCharacterFXEditorToolkit overrides

class FDataflowPreviewSceneDescriptionCustomization : public IDetailCustomization
{
public:
	FDataflowPreviewSceneDescriptionCustomization(const TArray<UDataflowBaseContent*>& DataflowContents);

	virtual ~FDataflowPreviewSceneDescriptionCustomization() {}

	
	/**Customize details for the description */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

	private :
		/** List of dataflow contents to preview */
		TMap<FString,TArray<UObject*>> ContentTypesObjects;
};

FDataflowPreviewSceneDescriptionCustomization::FDataflowPreviewSceneDescriptionCustomization(const TArray<UDataflowBaseContent*>& DataflowContents) :
	IDetailCustomization(), ContentTypesObjects()
{
	static const FString PreviewCategory = TEXT("Preview");
	TArray<UObject*>& PreviewObjects = ContentTypesObjects.FindOrAdd(PreviewCategory);
	for(UDataflowBaseContent* DataflowContent : DataflowContents)
	{
		if(DataflowContent)
		{
			PreviewObjects.Add(DataflowContent); 
		}
	}
}

void FDataflowPreviewSceneDescriptionCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	FAddPropertyParams PropertyParams;
	PropertyParams.AllowChildren(true);
	PropertyParams.CreateCategoryNodes(false);
	PropertyParams.HideRootObjectNode(true);
	for(TPair<FString, TArray<UObject*>>& ContentTypeObjects : ContentTypesObjects)
	{
		DetailBuilder.EditCategory(*ContentTypeObjects.Key).AddExternalObjects(ContentTypeObjects.Value, EPropertyLocation::Common, PropertyParams);
	}
}

TSharedRef<class IDetailCustomization> FDataflowEditorToolkit::CustomizePreviewSceneDescription() const
{
	const TArray<UDataflowBaseContent*> SimulationContents  = TArray<UDataflowBaseContent*>{SimulationScene->GetEditorContent()};
	return MakeShareable(new FDataflowPreviewSceneDescriptionCustomization(SimulationContents));
}

TSharedRef<class IDetailCustomization> FDataflowEditorToolkit::CustomizeAssetViewer() const
{
	return MakeShareable(new FDataflowAssetViewerSettingsCustomization(SimulationSceneProfileIndexStorage));
}


//~ Begin FBaseAssetToolkit overrides

void FDataflowEditorToolkit::CreateWidgets()
{
	FBaseCharacterFXEditorToolkit::CreateWidgets();

	if (const TObjectPtr<UDataflowBaseContent>& EditorContent = GetEditorContent())
	{
		if (UDataflow* DataflowAsset = EditorContent->GetDataflowAsset())
		{
			NodeDetailsEditor = CreateNodeDetailsEditorWidget(EditorContent->GetDataflowAsset());
			if(EditorContent->GetDataflowOwner() != EditorContent->GetDataflowAsset())
			{
				AssetDetailsEditor = CreateAssetDetailsEditorWidget({EditorContent->GetDataflowOwner(),EditorContent->GetDataflowAsset()});
			}
			else
			{
				AssetDetailsEditor = CreateAssetDetailsEditorWidget({EditorContent->GetDataflowAsset()});
			}
			GraphEditor = CreateGraphEditorWidget(DataflowAsset, NodeDetailsEditor);

			// Synchronize the EditorContent's selected node with the GraphEditor
			UDataflowEdNode* const InitialSelectedNode = Cast<UDataflowEdNode>(GraphEditor->GetSingleSelectedNode());
			EditorContent->SetSelectedNode(InitialSelectedNode);

			CreateSimulationViewportClient();
		
			TArray<FAdvancedPreviewSceneModule::FDetailCustomizationInfo> DetailsCustomizations;

			DetailsCustomizations.Add({ UDataflowSimulationSceneDescription::StaticClass(),
				FOnGetDetailCustomizationInstance::CreateSP(const_cast<FDataflowEditorToolkit*>(this), &FDataflowEditorToolkit::CustomizePreviewSceneDescription) });

			DetailsCustomizations.Add({ UAssetViewerSettings::StaticClass(),
				FOnGetDetailCustomizationInstance::CreateSP(const_cast<FDataflowEditorToolkit*>(this), &FDataflowEditorToolkit::CustomizeAssetViewer) });

			AdvancedPreviewSettingsWidget = SNew(SDataflowAdvancedPreviewDetailsTab, SimulationScene.ToSharedRef())
				.AdditionalSettings(SimulationScene->GetPreviewSceneDescription())
				.ProfileIndexStorage(SimulationSceneProfileIndexStorage)
				.DetailCustomizations(DetailsCustomizations)
				.PropertyTypeCustomizations(TArray<FAdvancedPreviewSceneModule::FPropertyTypeCustomizationInfo>())
				.Delegates(TArray<FAdvancedPreviewSceneModule::FDetailDelegates>());
			
		}
	}
}

// Called from FBaseAssetToolkit::CreateWidgets. The delegate call path goes through FAssetEditorToolkit::InitAssetEditor
// and FBaseAssetToolkit::SpawnTab_Viewport.
AssetEditorViewportFactoryFunction FDataflowEditorToolkit::GetViewportDelegate()
{
	AssetEditorViewportFactoryFunction TempViewportDelegate = [this](FAssetEditorViewportConstructionArgs InArgs)
	{
		TSharedRef<SDataflowConstructionViewport> Viewport = SAssignNew(DataflowConstructionViewport, SDataflowConstructionViewport, InArgs)
			.ViewportClient(StaticCastSharedPtr<FDataflowConstructionViewportClient>(ViewportClient));
		
		if (UDataflowEditorMode* const DataflowMode = Cast<UDataflowEditorMode>(EditorModeManager->GetActiveScriptableMode(UDataflowEditorMode::EM_DataflowEditorModeId)))
		{
			if (TSharedPtr<FModeToolkit> ModeToolkit = DataflowMode->GetToolkit().Pin())
			{
				if (FDataflowEditorModeToolkit* DataflowModeToolkit = static_cast<FDataflowEditorModeToolkit*>(ModeToolkit.Get()))
				{
					DataflowModeToolkit->SetConstructionViewportWidget(DataflowConstructionViewport);
				}
			}
		}
		return Viewport;
	};

	return TempViewportDelegate;
}

// Called from FBaseAssetToolkit::CreateWidgets to populate ViewportClient, but otherwise only used 
// in our own viewport delegate.
TSharedPtr<FEditorViewportClient> FDataflowEditorToolkit::CreateEditorViewportClient() const
{
	// Note that we can't reliably adjust the viewport client here because we will be passing it
	// into the viewport created by the viewport delegate we get from GetViewportDelegate(), and
	// that delegate may (will) affect the settings based on FAssetEditorViewportConstructionArgs,
	// namely ViewportType.
	// Instead, we do viewport client adjustment in PostInitAssetEditor().
	check(EditorModeManager.IsValid());
	TSharedPtr<FDataflowConstructionViewportClient> LocalConstructionClient = MakeShared<FDataflowConstructionViewportClient>(
		EditorModeManager.Get(), ConstructionScene, true);
	LocalConstructionClient->SetDataflowEditorToolkit(StaticCastSharedRef<FDataflowEditorToolkit>(
		const_cast<FDataflowEditorToolkit*>(this)->AsShared()));

	return LocalConstructionClient;
}

void FDataflowEditorToolkit::CreateSimulationViewportClient()
{
	SimulationTabContent = MakeShareable(new FEditorViewportTabContent());
	SimulationViewportClient = MakeShared<FDataflowSimulationViewportClient>(SimulationModeManager.Get(),
		SimulationScene, false);
	
	SimulationViewportClient->SetDataflowEditorToolkit(StaticCastSharedRef<FDataflowEditorToolkit>(this->AsShared()));

	SimulationViewportDelegate = [this](FAssetEditorViewportConstructionArgs InArgs)
	{
		TSharedRef<SDataflowSimulationViewport> Viewport = SAssignNew(DataflowSimulationViewport, SDataflowSimulationViewport, InArgs)
			.ViewportClient(StaticCastSharedPtr<FDataflowSimulationViewportClient>(SimulationViewportClient))
			.CommandList(GetToolkitCommands().ToSharedPtr());;

		if (UDataflowEditorMode* const DataflowMode = Cast<UDataflowEditorMode>(EditorModeManager->GetActiveScriptableMode(UDataflowEditorMode::EM_DataflowEditorModeId)))
		{
			if (TSharedPtr<FModeToolkit> ModeToolkit = DataflowMode->GetToolkit().Pin())
			{
				if (FDataflowEditorModeToolkit* DataflowModeToolkit = static_cast<FDataflowEditorModeToolkit*>(ModeToolkit.Get()))
				{
					DataflowModeToolkit->SetSimulationViewportWidget(DataflowSimulationViewport);
				}
			}
		}
		return Viewport;
	};
}

//~ End FBaseAssetToolkit overrides

void FDataflowEditorToolkit::UpdateDebugDraw()
{
	if(TObjectPtr<UDataflowBaseContent>& EditorContent = GetEditorContent())
	{
		if(EditorContent->GetDataflowAsset() && EditorContent->GetDataflowAsset()->Type == EDataflowType::Construction)
		{
			static const FString RootElementsName = "Construction Elements";
			UE::Dataflow::Private::UpdateDebugDrawComponent<FDataflowNode>(
				EditorContent, RootElementsName, ConstructionScene, true, DebugDrawOverlayString, EditorModeManager);
		}
		else
		{
			static const FString RootElementsName = "Simulation Elements";
			UE::Dataflow::Private::UpdateDebugDrawComponent<FDataflowSimulationNode>(
				EditorContent, RootElementsName, SimulationScene, false, DebugDrawOverlayString, SimulationModeManager);
		}
	}
}

namespace UE::Dataflow::Private
{
	static void ShowNotificationMessage(const FText& Message, const SNotificationItem::ECompletionState CompletionState)
	{
		FNotificationInfo Info(Message);
		Info.ExpireDuration = 5.0f;
		Info.bUseLargeFont = false;
		Info.bUseThrobber = false;
		Info.bUseSuccessFailIcons = false;
		TSharedPtr<SNotificationItem> Notification = FSlateNotificationManager::Get().AddNotification(Info);
		if (Notification.IsValid())
		{
			Notification->SetCompletionState(CompletionState);
		}
	}
}

void FDataflowEditorToolkit::OnPropertyValueChanged(const FPropertyChangedEvent& PropertyChangedEvent)
{
	if (const TObjectPtr<UDataflowBaseContent>& EditorContent = GetEditorContent())
	{
		ensure(EditorContent);
		if (UDataflow* DataflowAsset = EditorContent->GetDataflowAsset())
		{
			TSharedPtr<UE::Dataflow::FEngineContext> DataflowContext = EditorContent->GetDataflowContext();
			UE::Dataflow::FTimestamp LastNodeTimestamp = EditorContent->GetLastModifiedTimestamp();
			
			FDataflowEditorCommands::OnPropertyValueChanged(DataflowAsset, DataflowContext, LastNodeTimestamp, PropertyChangedEvent, SelectedDataflowNodes);

			if (!SelectedComments.IsEmpty())
			{
				FDataflowEditorCommands::OnCommentPropertyValueChanged(PropertyChangedEvent, SelectedComments);
			}

			// Refresh graph display to display nodes with warning/error
			GraphEditor->NotifyGraphChanged();
		}
	}

	// Keep viewport gizmos in sync with the updated property values.
	if (PropertyGizmoManager)
	{
		UpdateGizmos();
	}
}

void FDataflowEditorToolkit::OnDataflowAssetChanged()
{
	ResetEditor();
}

void FDataflowEditorToolkit::ResetEditor()
{
	if (const TObjectPtr<UDataflowBaseContent>& EditorContent = GetEditorContent())
	{
		if (EditorContent)
		{
			if (IDataflowInstanceInterface* DataflowInstanceInterface = UE::Dataflow::InstanceUtils::GetDataflowInterfaceFromObject(EditorContent->GetDataflowOwner()))
			{
				// close all subgraphs tabs and selection to make sure no other widget refer to this graphs data
				const TArray<TSharedPtr<SDockTab>> Tabs = DocumentManager->GetAllDocumentTabs();
				for (TSharedPtr<SDockTab> Tab : Tabs)
				{
					if (Tab)
					{
						Tab->RequestCloseTab();
					}
				}
				if (GraphEditor)
				{
					GraphEditor->ClearSelectionSet();
				}
				bViewsNeedRefresh = true;

				// change the asset and open a new graph in the tab 
				if (UDataflow* NewDataflowAsset = DataflowInstanceInterface->GetDataflowInstance().GetDataflowAsset())
				{
					NewDataflowAsset->Schema = UDataflowSchema::StaticClass();
					EditorContent->SetDataflowAsset(NewDataflowAsset);
					GraphEditor = CreateGraphEditorWidget(NewDataflowAsset, NodeDetailsEditor);
					if (GraphEditorTab)
					{
						if (GraphEditor)
						{
							GraphEditorTab->SetContent(GraphEditor.ToSharedRef());
						}
						else
						{
							GraphEditorTab->SetContent(SNew(SSpacer));
						}
					}
				}
				else
				{
					// Clear the GraphEditor area
					// (Can't have a SDataflowGraphEditor with a null UDataflow, so just put down Spacers if we have no Dataflow)
					GraphEditor.Reset();
					if (GraphEditorTab)
					{
						GraphEditorTab->SetContent(SNew(SSpacer));
					}
				}
				if (EditorModeManager)
				{
					if (UDataflowEditorMode* const DataflowMode = Cast<UDataflowEditorMode>(EditorModeManager->GetActiveScriptableMode(UDataflowEditorMode::EM_DataflowEditorModeId)))
					{
						// Make sure DataflowMode gets the up to date version of the GraphEditor, otherwise this may cause issue with launching the tools if the weakptr is stale 
						DataflowMode->SetDataflowGraphEditor(GraphEditor);
					}
				}
			}
		}
	}

	// Reset the editing state for external dataflows
	bAllowExternalDataflowEditing = false;
}

void FDataflowEditorToolkit::OnAssetPropertyValueChanged(const FPropertyChangedEvent& PropertyChangedEvent)
{
	if (const TObjectPtr<UDataflowBaseContent>& EditorContent = GetEditorContent())
	{
		if (PropertyChangedEvent.GetPropertyName()== FDataflowInstance::GetDataflowAssetPropertyName())
		{
			ResetEditor();
		}
		if (ensure(EditorContent))
		{
			FDataflowEditorCommands::OnAssetPropertyValueChanged(EditorContent, PropertyChangedEvent);
		}
	}
}

bool FDataflowEditorToolkit::OnNodeVerifyTitleCommit(const FText& NewText, UEdGraphNode* GraphNode, FText& OutErrorMessage)
{
	return FDataflowEditorCommands::OnNodeVerifyTitleCommit(NewText, GraphNode, OutErrorMessage);
}

void FDataflowEditorToolkit::OnNodeTitleCommitted(const FText& InNewText, ETextCommit::Type InCommitType, UEdGraphNode* GraphNode)
{
	FDataflowEditorCommands::OnNodeTitleCommitted(InNewText, InCommitType, GraphNode);
}

void FDataflowEditorToolkit::OnNodeDoubleClicked(UEdGraphNode* ClickedNode)
{
	// if the node is a call to subgraph , open the subgraph tab 
	if (UDataflowEdNode* DataflowEdNode = Cast<UDataflowEdNode>(ClickedNode))
	{
		if (TSharedPtr<FDataflowNode> DataflowNode = DataflowEdNode->GetDataflowNode())
		{
			if (FDataflowCallSubGraphNode* CallSubGraphNode = DataflowNode->AsType<FDataflowCallSubGraphNode>())
			{
				if (UDataflowSubGraph* Subgraph = GetSubGraph(CallSubGraphNode->GetSubGraphGuid()))
				{
					OpenSubGraphTab(Subgraph);
				}
			}
		}
	}
}

TSet<TObjectPtr<UDataflowEdNode>> FDataflowEditorToolkit::FilterDataflowEdNodesFromSet(const TSet<UObject*>& Set)
{
	TSet<TObjectPtr<UDataflowEdNode>> DataflowEdNodes;
	for (UObject* ObjectPtr : Set)
	{
		if (UDataflowEdNode* DataflowEdNode = Cast<UDataflowEdNode>(ObjectPtr))
		{
			DataflowEdNodes.Add(DataflowEdNode);
		}
	}
	return DataflowEdNodes;
}

const TObjectPtr<UDataflowEdNode> FDataflowEditorToolkit::GetOnlyFromSet(const TSet<TObjectPtr<UDataflowEdNode>>& Set)
{
	if (Set.Num() == 1)
	{
		return *Set.CreateConstIterator();
	}
	return nullptr;
}

void FDataflowEditorToolkit::OnNodeSelectionChanged(const TSet<UObject*>& InNewSelection)
{
	// update the current selection set and keep the previous one
	TSet<TObjectPtr<UDataflowEdNode>> PreviouslySelectedNodes = SelectedDataflowNodes;
	SelectedDataflowNodes = FilterDataflowEdNodesFromSet(InNewSelection);

	SelectedComments.Reset();
	auto IsACommentNode = [](UObject* Obj) { return Cast<UEdGraphNode_Comment>(Obj) != nullptr; };
	auto CastToCommentNode = [](UObject* Obj) { return Cast<UEdGraphNode_Comment>(Obj); };
	Algo::TransformIf(InNewSelection, SelectedComments, IsACommentNode, CastToCommentNode);

	// compute the various delta sets
	TSet<TObjectPtr<UDataflowEdNode>> DeselectedNodes = PreviouslySelectedNodes.Difference(SelectedDataflowNodes);
	TSet<TObjectPtr<UDataflowEdNode>> StillSelectedNodes = PreviouslySelectedNodes.Intersect(SelectedDataflowNodes);
	TSet<TObjectPtr<UDataflowEdNode>> NewlySelectedNodes = SelectedDataflowNodes.Difference(PreviouslySelectedNodes);

	// update the render flag 
	auto SetShouldRenderNodeFlag = [](const TSet<TObjectPtr<UDataflowEdNode>>& Nodes, bool bValue)
		{
			for (UDataflowEdNode* const Node : Nodes)
			{
				if (Node)
				{
					Node->SetShouldRenderNode(bValue);
				}
			}
		};

	const TObjectPtr<UDataflowEdNode> PrevSelectedNode = GetOnlyFromSet(PreviouslySelectedNodes);
	const TObjectPtr<UDataflowEdNode> NewSelectedNode = GetOnlyFromSet(SelectedDataflowNodes);
	const bool bSingleSelectionChanged = (PrevSelectedNode != NewSelectedNode);

	SetShouldRenderNodeFlag(DeselectedNodes, false);
	SetShouldRenderNodeFlag({ NewSelectedNode }, true);

	// todo : move the tool logic somewhere else: we should not be activelly looking for a collection here, we should have some sort of interface to talk to the tools
	UDataflowEditorMode* const DataflowMode = Cast<UDataflowEditorMode>(EditorModeManager->GetActiveScriptableMode(UDataflowEditorMode::EM_DataflowEditorModeId));

	// First close any running tool by accepting the changes.
	// New tool will start at the end of this method if a single node was selected
	if (DataflowMode && bSingleSelectionChanged)
	{
		DataflowMode->ShutdownActiveToolIfNeeded(EToolShutdownType::Accept);
	}

	if (const TObjectPtr<UDataflowBaseContent>& EditorContent = GetEditorContent(); EditorContent->GetDataflowAsset())
	{
		EditorContent->SetSelectedNode(NewSelectedNode);
		EditorContent->SetSelectedCollection(nullptr, /*bCollectionIsInput=*/ false);

		if (bSingleSelectionChanged)
		{
			// if alt is pressed, select make sure the first component is selected 
			if (GetDataflowGraphEditor()->IsAltDown())
			{
				if (ConstructionScene)
				{
					ConstructionScene->SelectNodeComponents(NewSelectedNode);
				}
			}
		}
	}

	// refresh the views
	UpdateViewsFromNode(NewSelectedNode);


	// Check if the current view mode can render the selected node. If not, try to find a view mode that can.
	if (DataflowMode)
	{
		DataflowMode->SetConstructionViewModeForNode(NewSelectedNode);
		DataflowMode->SetPendingNodeSelectionChanged(false);

		// if we have one single element selected, start the associated tool
		if (bSingleSelectionChanged && SelectedDataflowNodes.Num() == 1)
		{
			// Start the corresponding tool
			DataflowMode->StartToolForSelectedNode(NewSelectedNode);
		}
	}

	UpdateGizmos();
}

bool FDataflowEditorToolkit::IsGraphDirty() const
{
	using namespace UE::Dataflow;

	// we need to check the specific terminal for this asset and see if it's up to date 
	if (const TObjectPtr<UDataflowBaseContent>& EditorContent = GetEditorContent())
	{
		const FTimestamp LastModifiedTimestamp = EditorContent->GetLastModifiedTimestamp();
		const FName TerminalNodeName(EditorContent->GetDataflowTerminal());

		if (UDataflow* DataflowAsset = EditorContent->GetDataflowAsset())
		{
			if (TSharedPtr<FGraph> DataflowGraph = DataflowAsset->GetDataflow())
			{
				if (FGraph::FConstNodeSharedPtr Node = DataflowGraph->FindBaseNode(TerminalNodeName))
				{
					if (const FDataflowTerminalNode* TerminalNode = Node->AsType<FDataflowTerminalNode>())
					{
						if (LastModifiedTimestamp < TerminalNode->GetTimestamp())
						{
							return true;
						}
					}
					else
					{
						// no main terminal means always dirty
						return true;
					}
				}
			}
		}
	}
	return false;
}

void FDataflowEditorToolkit::ResetDataflowSimulation() const
{
	if(SimulationScene)
	{
		const bool bSimulationWasEnabled = SimulationScene->IsSimulationEnabled();
		SimulationScene->RebuildSimulationScene();
		SimulationScene->SetSimulationEnabled(bSimulationWasEnabled);
	}
}

void FDataflowEditorToolkit::ToggleDataflowSimulation() const
{
	if(HasSimulationManager())
	{
		UDataflowSimulationManager* SimulationManager = SimulationScene->GetWorld()->GetSubsystem<UDataflowSimulationManager>();
		
		const bool SimulationFlag = SimulationManager->GetSimulationEnabled();
		SimulationManager->SetSimulationEnabled(!SimulationFlag);
	}
}

void FDataflowEditorToolkit::StartDataflowSimulation() const
{
	if(HasSimulationManager())
	{
		UDataflowSimulationManager* SimulationManager = SimulationScene->GetWorld()->GetSubsystem<UDataflowSimulationManager>();
		
		SimulationManager->SetSimulationEnabled(true);
	}
}

void FDataflowEditorToolkit::StepDataflowSimulation() const
{
	if(HasSimulationManager())
	{
		UDataflowSimulationManager* SimulationManager = SimulationScene->GetWorld()->GetSubsystem<UDataflowSimulationManager>();

		SimulationManager->SetSimulationEnabled(true);
		SimulationManager->SetSimulationStepping(true);
	}
}

void FDataflowEditorToolkit::PauseDataflowSimulation() const
{
	if(HasSimulationManager())
	{
		UDataflowSimulationManager* SimulationManager = SimulationScene->GetWorld()->GetSubsystem<UDataflowSimulationManager>();
		
		SimulationManager->SetSimulationEnabled(false);
	}
}

void FDataflowEditorToolkit::StopDataflowSimulation() const
{
	if(HasSimulationManager())
	{
		UDataflowSimulationManager* SimulationManager = SimulationScene->GetWorld()->GetSubsystem<UDataflowSimulationManager>();
		
		SimulationManager->SetSimulationEnabled(false);
	}
}

bool FDataflowEditorToolkit::IsSimulationEnabled() const
{
	if(HasSimulationManager())
	{
		UDataflowSimulationManager* SimulationManager = SimulationScene->GetWorld()->GetSubsystem<UDataflowSimulationManager>();
		return SimulationManager->GetSimulationEnabled();
	}
	return false;
}

bool FDataflowEditorToolkit::IsSimulationDisabled() const
{
	return !IsSimulationEnabled();
}

bool FDataflowEditorToolkit::HasSimulationManager() const
{
	return SimulationScene && SimulationScene->GetWorld() && SimulationScene->GetWorld()->GetSubsystem<UDataflowSimulationManager>();
}

void FDataflowEditorToolkit::EvaluateGraph()
{
	if (const TObjectPtr<UDataflowBaseContent>& EditorContent = GetEditorContent())
	{
		if (UDataflow* DataflowAsset = EditorContent->GetDataflowAsset())
		{
			if (TSharedPtr<UE::Dataflow::FGraph> DataflowGraph = DataflowAsset->GetDataflow())
			{
				for (const TSharedPtr<FDataflowNode>& DataflowNode : DataflowGraph->GetFilteredNodes(FDataflowTerminalNode::StaticType()))
				{
					if (const FDataflowTerminalNode* TerminalNode = DataflowNode ? DataflowNode->AsType<FDataflowTerminalNode>() : nullptr)
					{
						NodesToEvaluateOnTick.Add(TerminalNode->GetGuid());
					}
				}
			}
		}
	}
}

void FDataflowEditorToolkit::OnNodeInvalidated(UDataflow& DataflowAsset, FDataflowNode& Node)
{
	if (EvaluationMode == EDataflowEditorEvaluationMode::Automatic)
	{
		if (const TObjectPtr<UDataflowBaseContent>& EditorContent = GetEditorContent())
		{
			if (EditorContent->GetDataflowAsset() == &DataflowAsset)
			{
				// evaluate only active terminals nodes 
				if (FDataflowTerminalNode* TerminalNode = Node.AsType<FDataflowTerminalNode>())
				{
					NodesToEvaluateOnTick.Add(TerminalNode->GetGuid());
				}
				else
				{
					// also add "terminal" normal nodes ( no output connections )
					bool bHasOutputConnection = false;
					Node.ForEachConnection([&bHasOutputConnection](const FDataflowConnection* Connection)
						{
							if (Connection->GetDirection() == UE::Dataflow::FPin::EDirection::OUTPUT)
							{
								bHasOutputConnection |= Connection->IsConnected();
							}
							return !bHasOutputConnection;
						});
					if (!bHasOutputConnection)
					{
						NodesToEvaluateOnTick.Add(Node.GetGuid());
					}
				}
			}
		}
	}

	LogMessages->RemoveForNode(Node, Node.GetTimestamp());
}

void FDataflowEditorToolkit::OnNodeDeleted(const TSet<UObject*>& NewSelection)
{
	const TSet<TObjectPtr<UDataflowEdNode>> DeletedNodes = FilterDataflowEdNodesFromSet(NewSelection);

	FGraphPanelSelectionSet DeletedNodesSet;
	for (UDataflowEdNode* Node : DeletedNodes)
	{
		if (SelectedDataflowNodes.Contains(Node))
		{
			SelectedDataflowNodes.Remove(Node);
		}
		DeletedNodesSet.Add(Node);
	}

	// remove from the logs
	for (UDataflowEdNode* Node : DeletedNodes)
	{
		if (TSharedPtr<const FDataflowNode> DataflowNode = Node->GetDataflowNode())
		{
			LogMessages->RemoveForNode(*DataflowNode);
		}
	}

	if (UDataflowEditorMode* DataflowEdMode = Cast<UDataflowEditorMode>(EditorModeManager->GetActiveScriptableMode(UDataflowEditorMode::EM_DataflowEditorModeId)))
	{
		DataflowEdMode->OnDataflowNodeDeleted(DeletedNodesSet);
	}
}

void FDataflowEditorToolkit::OnConstructionViewSelectionChanged(const TArray<UPrimitiveComponent*>& SelectedComponents, const TArray<FDataflowBaseElement*>& SelectedElements)
{
	for(IDataflowViewListener* Listener : ViewListeners)
	{
		Listener->OnConstructionViewSelectionChanged(SelectedComponents, SelectedElements);
	}
}

void FDataflowEditorToolkit::OnSimulationViewSelectionChanged(const TArray<UPrimitiveComponent*>& SelectedComponents, const TArray<FDataflowBaseElement*>& SelectedElements)
{
	for(IDataflowViewListener* Listener : ViewListeners)
	{
		Listener->OnSimulationViewSelectionChanged(SelectedComponents, SelectedElements);
	}
}

void FDataflowEditorToolkit::OnNodeBeginEvaluate(const FDataflowNode* Node, const FDataflowOutput* Output)
{
	if (bDataflowLogNodeEvaluation)
	{
		if (const TObjectPtr<UDataflowBaseContent>& EditorContent = GetEditorContent())
		{
			ensure(EditorContent);
			if (TSharedPtr<UE::Dataflow::FEngineContext> DataflowContext = EditorContent->GetDataflowContext())
			{
				DataflowContext->Info(TEXT("Begin Evaluate"), Node, Output);
			}
		}
	}
}

void FDataflowEditorToolkit::OnNodeFinishEvaluate(const FDataflowNode* Node, const FDataflowOutput* Output)
{
	if (bDataflowLogNodeEvaluation)
	{
		if (const TObjectPtr<UDataflowBaseContent>& EditorContent = GetEditorContent())
		{
			ensure(EditorContent);
			if (TSharedPtr<UE::Dataflow::FEngineContext> DataflowContext = EditorContent->GetDataflowContext())
			{
				DataflowContext->Info(TEXT("End Evaluate"), Node, Output);
			}
		}
	}
}

void FDataflowEditorToolkit::SetDataflowPathFromNodeAndOutput(const FDataflowNode* Node, const FDataflowOutput* Output, FDataflowPath& OutPath) const
{
	if (const TObjectPtr<UDataflowBaseContent>& EditorContent = GetEditorContent())
	{
		if (const TObjectPtr<UDataflow> DataflowAsset = EditorContent->GetDataflowAsset())
		{
			FName GraphName = GetGraphLogName();
			if (Node)
			{
				if (const UDataflowEdNode* EdNode = DataflowAsset->FindEdNodeByDataflowNodeGuid(Node->GetGuid()))
				{
					if (const UDataflowSubGraph* SubGraph = Cast<UDataflowSubGraph>(EdNode->GetGraph()))
					{
						GraphName = SubGraph->GetFName();
					}
				}
			}
			OutPath.SetGraph(GraphName.ToString());
			OutPath.SetNode(Node ? Node->GetName().ToString() : FString());
			OutPath.SetOutput(Output ? Output->GetName().ToString() : FString());
		}
	}
}

void FDataflowEditorToolkit::LogMessage(const EMessageSeverity::Type Severity, const FDataflowNode* Node, const FDataflowOutput* Output, const FString& Message) const
{
	FText TextMessage = FText::FromString(Message);
	LogMessage(Severity, Node, Output, TextMessage);
}

void FDataflowEditorToolkit::LogMessage(const EMessageSeverity::Type Severity, const FDataflowNode* Node, const FDataflowOutput* Output, const FText& Message) const
{
	if (DataflowOutputLog)
	{
		FDataflowPath Path;
		SetDataflowPathFromNodeAndOutput(Node, Output, Path);
		DataflowOutputLog->AddMessage(Severity, Message, Path);
	}
}

void FDataflowEditorToolkit::RefreshLogMessageView()
{
	if (DataflowOutputLog)
	{
		DataflowOutputLog->ClearMessageLog();

		UE::Dataflow::FContext::FLogMessage Message;
		const int32 NumMessages = LogMessages->Num();
		for (int32 Index = 0; Index < NumMessages; ++Index)
		{
			LogMessages->GetMessage(Index, Message);
			TSharedPtr<const FDataflowNode> Node = Message.Node.Pin();
			const FDataflowOutput* Output = Node ? Node->FindOutput(Message.OutputGuid) : nullptr;
			LogMessage(Message.Severity, Node.Get(), Output, Message.Text);
		}
	}
}

void FDataflowEditorToolkit::OnContextLog(const UE::Dataflow::FContext::FLogMessage& InLogMessage)
{
	LogMessages->Add(InLogMessage);
}

namespace UE::Dataflow::Private
{
	static int32 GetMilliseconds(const FTimespan& InTimespan)
	{
		const int64 Ticks = InTimespan.GetTicks();
		return (int32)((Ticks / ETimespan::TicksPerMillisecond) % 1000);
	}

	static FString GetElapsedTimeString(const FDateTime& InGraphEvaluationFinished, const FDateTime& InGraphEvaluationBegin)
	{
		const FTimespan ElapsedTimespan = InGraphEvaluationFinished - InGraphEvaluationBegin;
		const FString ElapsedTimeString = FString::Printf(TEXT("%02dm %02ds %02dms"), ElapsedTimespan.GetMinutes(), ElapsedTimespan.GetSeconds(), GetMilliseconds(ElapsedTimespan));

		return ElapsedTimeString;
	}
}

void FDataflowEditorToolkit::OnOutputLogMessageTokenClicked(const FString TokenString)
{
	FDataflowPath Path;
	Path.DecodePath(TokenString);
	const FString NodeName = Path.GetNode();

	FString Output;
	if (!NodeName.IsEmpty() && Path.PathHasOutput())
	{
		Output = Path.GetOutput();
	}

	TSharedPtr<SDataflowGraphEditor> ActiveGraphEditor = GraphEditor;


	// Select node from TokenString
	if (UDataflow* DataflowAsset = GetEditorContent()->GetDataflowAsset())
	{
		const FName GraphName(Path.GetGraph());
		if (UDataflowSubGraph* SubGraph = DataflowAsset->FindSubGraphByName(GraphName))
		{
			OpenSubGraphTab(GraphName);
			ActiveGraphEditor = ActiveSubGraphEditorWeakPtr.Pin();
		}
		else
		{
			if (GraphEditorTab)
			{
				GraphEditorTab->DrawAttention();
			}
		}

		// Clear node selection
		ActiveGraphEditor->ClearSelectionSet();

		// now jump to the relevant node 
		if (TSharedPtr<UE::Dataflow::FGraph> DataflowGraph = DataflowAsset->GetDataflow())
		{
			if (TSharedPtr<FDataflowNode> DataflowNode = DataflowGraph->FindBaseNode(FName(*NodeName)))
			{
				if (TObjectPtr<UDataflowEdNode> DataflowUEdNode = DataflowAsset->FindEdNodeByDataflowNodeGuid(DataflowNode->GetGuid()))
				{
					ActiveGraphEditor->JumpToNode(DataflowUEdNode);
				}
			}
		}
	}
}

void FDataflowEditorToolkit::Tick(float DeltaTime)
{
	if (const TObjectPtr<UDataflowBaseContent> EditorContent = GetEditorContent())
	{
		if (EditorContent->GetDataflowAsset())
		{
			UE::Dataflow::FTimestamp InitTimeStamp = EditorContent->GetLastModifiedTimestamp();
			if (!EditorContent->GetDataflowContext())
			{
				EditorContent->SetDataflowContext(MakeShared<UE::Dataflow::FEngineContext>(EditorContent->GetDataflowOwner()));
				InitTimeStamp = UE::Dataflow::FTimestamp::Invalid;
			}
			// Sync the global settings
			if (UDataflowEditorUserSettings* Settings = GetMutableDefault<UDataflowEditorUserSettings>())
			{
				if (TSharedPtr<UE::Dataflow::FEngineContext> Context = EditorContent->GetDataflowContext())
				{
					Context->EnablePerfData(Settings->bShowPerformanceData);
				}
			}

			// Update the list of dataflow terminal contents 
			DataflowEditor->UpdateTerminalContents(InitTimeStamp);

			// Evaluate the nodes that are requested from NodesToEvaluateOnTick
			const TSet<FGuid, DefaultKeyFuncs<FGuid>, TInlineSetAllocator<4>> NodesToEvaluateThisTick(NodesToEvaluateOnTick); // EvaluateNode can re-enter and Add to the set
			NodesToEvaluateOnTick.Reset();
			for (const FGuid& NodeToEvaluateGuid : NodesToEvaluateThisTick)
			{
				if (UDataflow* DataflowAsset = EditorContent->GetDataflowAsset())
				{
					if (TSharedPtr<UE::Dataflow::FGraph> DataflowGraph = DataflowAsset->GetDataflow())
					{
						if (TSharedPtr<const FDataflowNode> Node = DataflowGraph->FindBaseNode(NodeToEvaluateGuid))
						{
							UE::Dataflow::FTimestamp DummyNodeTimeStamp = UE::Dataflow::FTimestamp::Invalid;
							EvaluateNode(Node.Get(), nullptr, DummyNodeTimeStamp);
							bViewsNeedRefresh = true;
						}
					}
				}
			}
		}
	}
	RefreshViewsIfNeeded();
}

TStatId FDataflowEditorToolkit::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FDataflowEditorToolkit, STATGROUP_Tickables);
}

void FDataflowEditorToolkit::EvaluateTerminalNode(const FDataflowTerminalNode& TerminalNode)
{
	using namespace UE::Dataflow;

	UE_LOGF(LogChaosDataflow, VeryVerbose, "FDataflowEditorToolkit::EvaluateTerminalNode(): Node [%ls]", *TerminalNode.GetName().ToString());

	if (const TObjectPtr<UDataflowBaseContent>& EditorContent = GetEditorContent())
	{
		FTimestamp TerminalNodeTimeStamp = FTimestamp::Invalid;
		EvaluateNode(&TerminalNode, nullptr, TerminalNodeTimeStamp);
	}
}

void FDataflowEditorToolkit::EvaluateNode(const FDataflowNode* Node, const FDataflowOutput* Output, UE::Dataflow::FTimestamp& InOutTimestamp)
{
	UE_LOGF(LogChaosDataflow, VeryVerbose, "FDataflowEditorToolkit::EvaluateNode(): Node [%ls], Output [%ls]", Node ? *Node->GetName().ToString() : TEXT("nullptr"), Output ? *Output->GetName().ToString() : TEXT("nullptr"));

	if (!Node)
	{
		UE_LOGF(LogChaosDataflow, VeryVerbose, "FDataflowEditorToolkit::EvaluateNode(): Node is null, skipping evaluation");
		return;
	}

	// do not evaluate disabled terminal nodes
	if (!Node->IsActive())
	{
		UE_LOGF(LogChaosDataflow, VeryVerbose, "FDataflowEditorToolkit::EvaluateNode(): Node [%ls] is inactive skipping evaluation", *Node->GetName().ToString());
		return;
	}

	if (bDataflowEnableGraphEval)
	{
		const bool bIsInPIEOrSimulate = GEditor->PlayWorld || GEditor->bIsSimulatingInEditor;
		const bool bCanEvaluate = bAllowEvaluationInPIE || !bIsInPIEOrSimulate;
		if (bCanEvaluate)
		{
			if (const TObjectPtr<UDataflowBaseContent>& EditorContent = GetEditorContent())
			{
				const FDateTime StartEvaluationTime = FDateTime::Now();

				// when evaluation completes, refresh the views
				TWeakObjectPtr<UDataflowBaseContent> WeakEditorContent = EditorContent;
				TWeakPtr<FDataflowEditorToolkit> WeakToolkit = SharedThis(this);
				TWeakPtr<const FDataflowNode> WeakNode = Node->AsWeak();
				auto OnEvaluationCompleted = [WeakEditorContent, WeakToolkit, WeakNode, StartEvaluationTime](UE::Dataflow::FContext& Context)
					{
						check(IsInGameThread());

						const FDateTime EndEvaluationTime = FDateTime::Now();
						// refresh views 
						if (TSharedPtr<FDataflowEditorToolkit> ToolKit = WeakToolkit.Pin())
						{
							ToolKit->bViewsNeedRefresh = true;
							ToolKit->GraphEditor->NotifyGraphChanged();
						}
						if (TSharedPtr<const FDataflowNode> Node = WeakNode.Pin())
						{
							const FText ElapsedTimeText = FText::FromString(UE::Dataflow::Private::GetElapsedTimeString(EndEvaluationTime, StartEvaluationTime));
							const FText NodeNameText = FText::FromName(Node->GetName());
							const FText MessageText = FText::Format(LOCTEXT("EvalElapsedLog", "Graph evaluation from node [{0}] time : {1}"), NodeNameText, ElapsedTimeText);
							Context.Info(MessageText, Node.Get());

							if (TStrongObjectPtr<UDataflowBaseContent> EditorContent = WeakEditorContent.Pin())
							{
								if (Node->GetName().ToString() == EditorContent->GetDataflowTerminal())
								{
									EditorContent->SetLastModifiedTimestamp(Node->GetTimestamp());
								}
								EditorContent->SetConstructionDirty(true);
							}
						}
					};
				bool bUseAsyncEvaluation = false;
				if (UDataflowEditorUserSettings* Settings = GetMutableDefault<UDataflowEditorUserSettings>())
				{
					bUseAsyncEvaluation = Settings->bUseAsyncEvaluation;
				}
				if (bUseAsyncEvaluation)
				{
					// TODO : maybe this should be in the commands EvaluateNode as there's logic with the timestamp
					EditorContent->EvaluateGraph(OnEvaluationCompleted);
				}
				else
				{
					// If Node is null, the terminal node with the given name will be used instead
					FDataflowEditorCommands::EvaluateNode(*EditorContent->GetDataflowContext().Get(), *Node, Output, EditorContent->GetTerminalAsset(), InOutTimestamp, OnEvaluationCompleted);
				}
			}
		}
	}
}

void FDataflowEditorToolkit::RefreshViewsIfNeeded(bool bForce)
{
	if (ConstructionScene->IsSceneDirty() || SimulationScene->IsSceneDirty())
	{
		bViewsNeedRefresh = true;
	}

	if (bForce || bViewsNeedRefresh)
	{
		for (IDataflowViewListener* Listener : ViewListeners)
		{
			Listener->RefreshView();
		}

		UpdateDebugDraw();
		RefreshLogMessageView();

		bViewsNeedRefresh = false;
	}

	// reset dirty flags
	if (ConstructionScene->IsSceneDirty() || SimulationScene->IsSceneDirty())
	{
		ConstructionScene->ResetDirtyFlag();
		SimulationScene->ResetDirtyFlag();
	}

}

void FDataflowEditorToolkit::UpdateViewsFromNode(UDataflowEdNode* Node)
{
	for (IDataflowViewListener* Listener : ViewListeners)
	{
		Listener->OnSelectedNodeChanged(Node);
	}

	if (const TObjectPtr<UDataflowBaseContent>& EditorContent = GetEditorContent())
	{
		EditorContent->SetConstructionDirty(true);
	}

	UpdateDebugDraw();
};

TSharedRef<SDataflowGraphEditor> FDataflowEditorToolkit::CreateGraphEditorWidget(UEdGraph* GraphToEdit, TSharedPtr<IStructureDetailsView> InNodeDetailsEditor)
{
	ensure(GraphToEdit);
	using namespace UE::Dataflow;

	const SDataflowGraphEditor::FGraphEvaluationCallback Evaluate =
		[this](const FDataflowNode* Node, const FDataflowOutput* Output)
		{
			// This method is called when a node is explicily called to be evaluated form the UI 
			// Evaluate may already have happen when the node was invalidated in the graph editor code if the node is part of a branch that 
			// ends with a terminal node and evaluation mode is automatic ( see OnNodeInvalidated )
			// So to avoid double evaluation, we only evaluate if the node is not terminal or if we are in manual evaluation mode
			// todo(dataflow) We should refactor this post 5.6 to make this simpler and less dependent on other method logic
			const bool bIsTerminalNode = (Node && Node->AsType<FDataflowTerminalNode>());
			if (!bIsTerminalNode || EvaluationMode == EDataflowEditorEvaluationMode::Manual)
			{
				if (const TObjectPtr<UDataflowBaseContent>& EditorContent = GetEditorContent())
				{
					FTimestamp NodeTimeStamp = FTimestamp::Invalid;
					EvaluateNode(Node, Output, NodeTimeStamp);
				}
			}
		};

	DataflowEditor->UpdateTerminalContents(FTimestamp::Invalid);
	
	SGraphEditor::FGraphEditorEvents InEvents;
	InEvents.OnVerifyTextCommit = FOnNodeVerifyTextCommit::CreateSP(this, &FDataflowEditorToolkit::OnNodeVerifyTitleCommit);
	InEvents.OnTextCommitted = FOnNodeTextCommitted::CreateSP(this, &FDataflowEditorToolkit::OnNodeTitleCommitted);
	InEvents.OnNodeDoubleClicked = FSingleNodeEvent::CreateSP(this, &FDataflowEditorToolkit::OnNodeDoubleClicked);


	UDataflow* DataflowAsset = UDataflow::GetDataflowAssetFromEdGraph(GraphToEdit);
	check(DataflowAsset);

	TSharedRef<SDataflowGraphEditor> NewGraphEditor = SNew(SDataflowGraphEditor, DataflowAsset)
		.GraphToEdit(GraphToEdit)
		.GraphEvents(InEvents)
		.DetailsView(InNodeDetailsEditor)
		.EvaluateGraph(Evaluate)
		.IsEditable([this]() { return IsEditable(); })
		.DataflowEditor(DataflowEditor);

	OnSelectionChangedMulticastDelegateHandle = NewGraphEditor->OnSelectionChangedMulticast.AddSP(this, &FDataflowEditorToolkit::OnNodeSelectionChanged);
	OnNodeDeletedMulticastDelegateHandle = NewGraphEditor->OnNodeDeletedMulticast.AddSP(this, &FDataflowEditorToolkit::OnNodeDeleted);
	
	NewGraphEditor->SetLogMessageStorage(LogMessages);

	UnregisterContextHandlers();
	RegisterContextHandlers();

	return NewGraphEditor;
}

TSharedPtr<IStructureDetailsView> FDataflowEditorToolkit::CreateNodeDetailsEditorWidget(UObject* ObjectToEdit)
{
	ensure(ObjectToEdit);
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));

	FDetailsViewArgs DetailsViewArgs;
	{
		DetailsViewArgs.bAllowSearch = false;
		DetailsViewArgs.bHideSelectionTip = true;
		DetailsViewArgs.bLockable = false;
		DetailsViewArgs.bSearchInitialKeyFocus = true;
		DetailsViewArgs.bUpdatesFromSelection = false;
		DetailsViewArgs.NotifyHook = this;
		DetailsViewArgs.bShowOptions = true;
		DetailsViewArgs.bShowModifiedPropertiesOption = false;
		DetailsViewArgs.bShowScrollBar = false;
	}

	FStructureDetailsViewArgs StructureViewArgs;
	{
		StructureViewArgs.bShowObjects = true;
		StructureViewArgs.bShowAssets = true;
		StructureViewArgs.bShowClasses = true;
		StructureViewArgs.bShowInterfaces = true;
	}
	TSharedPtr<IStructureDetailsView> LocalDetailsView = PropertyEditorModule.CreateStructureDetailView(DetailsViewArgs, StructureViewArgs, nullptr);
	LocalDetailsView->GetDetailsView()->SetObject(ObjectToEdit);
	OnFinishedChangingPropertiesDelegateHandle = LocalDetailsView->GetOnFinishedChangingPropertiesDelegate().AddSP(this, &FDataflowEditorToolkit::OnPropertyValueChanged);

	LocalDetailsView->GetDetailsView()->SetIsPropertyEditingEnabledDelegate(FIsPropertyEditingEnabled::CreateRaw(this, &FDataflowEditorToolkit::IsEditable));

	NodeDetailsExtensionHandler = MakeShared<UE::Dataflow::FDataflowNodeDetailExtensionHandler>();
	LocalDetailsView->GetDetailsView()->SetExtensionHandler(NodeDetailsExtensionHandler);

	return LocalDetailsView;
}

TSharedPtr<IDetailsView> FDataflowEditorToolkit::CreateAssetDetailsEditorWidget(const TArray<UObject*>& ObjectsToEdit)
{
	ensure(ObjectsToEdit.Num() > 0);
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));

	FDetailsViewArgs DetailsViewArgs;
	{
		DetailsViewArgs.bAllowSearch = true;
		DetailsViewArgs.bLockable = false;
		DetailsViewArgs.bUpdatesFromSelection = false;
		DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
		DetailsViewArgs.NotifyHook = this;
		DetailsViewArgs.bAllowMultipleTopLevelObjects = true;
	}

	TSharedPtr<IDetailsView> LocalDetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	LocalDetailsView->SetObjects(ObjectsToEdit, true);

	OnFinishedChangingAssetPropertiesDelegateHandle = LocalDetailsView->OnFinishedChangingProperties().AddSP(this, &FDataflowEditorToolkit::OnAssetPropertyValueChanged);

	return LocalDetailsView;

}

TSharedPtr<SDataflowMembersWidget> FDataflowEditorToolkit::CreateDataflowMembersWidget()
{
	using namespace UE::Dataflow;

	TSharedRef<SDataflowMembersWidget> NewMembersWidget = SNew(SDataflowMembersWidget, SharedThis(this));

	return NewMembersWidget;

}

TSharedRef<SDockTab> FDataflowEditorToolkit::SpawnTab_AssetDetails(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == DetailsTabID);

	return SNew(SDockTab)
		.Label(LOCTEXT("DataflowEditor_AssetDetails_TabTitle", "Asset Details"))
		[
			AssetDetailsEditor->AsShared()
		];
}

TSharedRef<SDockTab> FDataflowEditorToolkit::SpawnTab_SimulationViewport(const FSpawnTabArgs& Args)
{
	TSharedRef<SDockTab> DockableTab = SNew(SDockTab);
	if(SimulationTabContent)
	{
		SimulationTabContent->Initialize(SimulationViewportDelegate, DockableTab, SimulationViewportTabId.ToString());
	}
	return DockableTab;
}

TSharedRef<SDockTab> FDataflowEditorToolkit::SpawnTab_PreviewScene(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == PreviewSceneTabId);

	return SNew(SDockTab)
		.Label(LOCTEXT("DataflowEditor_PreviewScene_TabTitle", "Preview Scene"))
		[
			AdvancedPreviewSettingsWidget->AsShared()
		];
}

TSharedRef<SDockTab> FDataflowEditorToolkit::SpawnTab_GraphCanvas(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == GraphCanvasTabId);

	SAssignNew(GraphEditorTab, SDockTab)
		.Label(LOCTEXT("DataflowEditor_DataflowGraph_TabTitle", "Dataflow Graph"));

	if (GraphEditor)
	{
		GraphEditorTab.Get()->SetContent(GraphEditor.ToSharedRef());
	}

	return GraphEditorTab.ToSharedRef();
}

TSharedRef<SDockTab> FDataflowEditorToolkit::SpawnTab_SubGraphTab(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == SubGraphCanvasTabId);

	TSharedPtr<SWidget> SubgraphEditor = SNullWidget::NullWidget;
	if (const TObjectPtr<UDataflowBaseContent>& EditorContent = GetEditorContent())
	{
		if (UDataflow* DataflowAsset = EditorContent->GetDataflowAsset())
		{
			SubgraphEditor = CreateGraphEditorWidget(DataflowAsset, NodeDetailsEditor);
		}
	}
	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab)
		.Label(LOCTEXT("DataflowEditor_DataflowSubgraph_TabTitle", "Dataflow Subgraph"))
		[
			SubgraphEditor.ToSharedRef()
		];

	return SpawnedTab;
}

TSharedRef<SDockTab> FDataflowEditorToolkit::SpawnTab_NodeDetails(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == NodeDetailsTabId);

	return SNew(SDockTab)
		.Label(LOCTEXT("DataflowEditor_NodeDetails_TabTitle", "Node Details"))
		[
			NodeDetailsEditor->GetWidget()->AsShared()
		];
}

TSharedRef<SDockTab> FDataflowEditorToolkit::SpawnTab_SkeletonView(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == SkeletonViewTabId);
	check(DataflowEditor);
	check(DataflowEditor->GetEditorContent());

	SkeletonEditorView = MakeShared<FDataflowSkeletonView>(DataflowEditor->GetEditorContent());
	ViewListeners.Add(SkeletonEditorView.Get());

	FSkeletonTreeArgs SkeletonTreeArgs;
	SkeletonTreeArgs.bShowBlendProfiles = false;
	SkeletonTreeArgs.bShowFilterMenu = true;
	SkeletonTreeArgs.bShowDebugVisualizationOptions = false;
	SkeletonTreeArgs.bAllowMeshOperations = false;
	SkeletonTreeArgs.bAllowSkeletonOperations = false;
	SkeletonTreeArgs.bHideBonesByDefault = false;
	SkeletonTreeArgs.OnSelectionChanged = FOnSkeletonTreeSelectionChanged::CreateSP(SkeletonEditorView.ToSharedRef(), &FDataflowSkeletonView::SkeletonViewSelectionChanged);
	SkeletonTreeArgs.ContextName = GetToolkitFName();

	TSharedPtr<ISkeletonTree> SkeletonEditor = SkeletonEditorView->CreateEditor(SkeletonTreeArgs);
	TSharedRef<SDockTab> DockableTab = SNew(SDockTab)
		.Label(LOCTEXT("DataflowEditor_SkeletonTree_TabTitle", "Skeleton Tree"))
		[
			SkeletonEditor.ToSharedRef()
		];

	DockableTab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateRaw(this, &FDataflowEditorToolkit::OnTabClosed));

	return DockableTab;
}

TSharedRef<SDockTab> FDataflowEditorToolkit::SpawnTab_OutlinerView(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == OutlinerViewTabId);
	check(DataflowEditor);
	check(DataflowEditor->GetEditorContent());

	DataflowOutlinerView = MakeShared<FDataflowOutlinerView>(ConstructionScene, SimulationScene, DataflowEditor->GetEditorContent());
	ViewListeners.Add(DataflowOutlinerView.Get());

	TSharedPtr<ISceneOutliner> DataflowOutliner = DataflowOutlinerView->CreateWidget();
	
	TSharedRef<SDockTab> DockableTab = SNew(SDockTab)
		.Label(LOCTEXT("DataflowEditor_SceneOutliner_TabTitle", "Scene Outliner"))
		[
			DataflowOutliner.ToSharedRef()

		];

	DockableTab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateRaw(this, &FDataflowEditorToolkit::OnTabClosed));

	return DockableTab;
}

TSharedRef<SDockTab> FDataflowEditorToolkit::SpawnTab_SelectionView(const FSpawnTabArgs& Args)
{
	//	check(Args.GetTabId().TabType == SelectionViewTabId_1);
	check(DataflowEditor);
	check(DataflowEditor->GetEditorContent());

	if (Args.GetTabId() == SelectionViewTabId_1)
	{
		DataflowSelectionView_1 = MakeShared<FDataflowSelectionView>(DataflowEditor->GetEditorContent());
		if (DataflowSelectionView_1.IsValid())
		{
			ViewListeners.Add(DataflowSelectionView_1.Get());
		}
	}
	else if (Args.GetTabId() == SelectionViewTabId_2)
	{
		DataflowSelectionView_2 = MakeShared<FDataflowSelectionView>(DataflowEditor->GetEditorContent());
		if (DataflowSelectionView_2.IsValid())
		{
			ViewListeners.Add(DataflowSelectionView_2.Get());
		}
	}
	else if (Args.GetTabId() == SelectionViewTabId_3)
	{
		DataflowSelectionView_3 = MakeShared<FDataflowSelectionView>(DataflowEditor->GetEditorContent());
		if (DataflowSelectionView_3.IsValid())
		{
			ViewListeners.Add(DataflowSelectionView_3.Get());
		}
	}
	else if (Args.GetTabId() == SelectionViewTabId_4)
	{
		DataflowSelectionView_4 = MakeShared<FDataflowSelectionView>(DataflowEditor->GetEditorContent());
		if (DataflowSelectionView_4.IsValid())
		{
			ViewListeners.Add(DataflowSelectionView_4.Get());
		}
	}

	TSharedPtr<SSelectionViewWidget> SelectionViewWidget;

	TSharedRef<SDockTab> DockableTab = SNew(SDockTab)
	[
		SAssignNew(SelectionViewWidget, SSelectionViewWidget)
	];

	if (SelectionViewWidget)
	{
		if (const TObjectPtr<UDataflowBaseContent>& EditorContent = GetEditorContent())
		{
			if (Args.GetTabId() == SelectionViewTabId_1)
			{
				DataflowSelectionView_1->SetSelectionView(SelectionViewWidget);
			}
			else if (Args.GetTabId() == SelectionViewTabId_2)
			{
				DataflowSelectionView_2->SetSelectionView(SelectionViewWidget);
			}
			else if (Args.GetTabId() == SelectionViewTabId_3)
			{
				DataflowSelectionView_3->SetSelectionView(SelectionViewWidget);
			}
			else if (Args.GetTabId() == SelectionViewTabId_4)
			{
				DataflowSelectionView_4->SetSelectionView(SelectionViewWidget);
			}
		}
	}

	DockableTab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateRaw(this, &FDataflowEditorToolkit::OnTabClosed));

	return DockableTab;
}

TSharedRef<SDockTab> FDataflowEditorToolkit::SpawnTab_CollectionSpreadSheet(const FSpawnTabArgs& Args)
{
	check(DataflowEditor);
	check(DataflowEditor->GetEditorContent());

	if (Args.GetTabId() == CollectionSpreadSheetTabId_1)
	{
		DataflowCollectionSpreadSheet_1 = MakeShared<FDataflowCollectionSpreadSheet>(DataflowEditor->GetEditorContent());
		if (DataflowCollectionSpreadSheet_1.IsValid())
		{
			ViewListeners.Add(DataflowCollectionSpreadSheet_1.Get());
		}
	}
	else if (Args.GetTabId() == CollectionSpreadSheetTabId_2)
	{
		DataflowCollectionSpreadSheet_2 = MakeShared<FDataflowCollectionSpreadSheet>(DataflowEditor->GetEditorContent());
		if (DataflowCollectionSpreadSheet_2.IsValid())
		{
			ViewListeners.Add(DataflowCollectionSpreadSheet_2.Get());
		}
	}
	else if (Args.GetTabId() == CollectionSpreadSheetTabId_3)
	{
		DataflowCollectionSpreadSheet_3 = MakeShared<FDataflowCollectionSpreadSheet>(DataflowEditor->GetEditorContent());
		if (DataflowCollectionSpreadSheet_3.IsValid())
		{
			ViewListeners.Add(DataflowCollectionSpreadSheet_3.Get());
		}
	}
	else if (Args.GetTabId() == CollectionSpreadSheetTabId_4)
	{
		DataflowCollectionSpreadSheet_4 = MakeShared<FDataflowCollectionSpreadSheet>(DataflowEditor->GetEditorContent());
		if (DataflowCollectionSpreadSheet_4.IsValid())
		{
			ViewListeners.Add(DataflowCollectionSpreadSheet_4.Get());
		}
	}

	TSharedPtr<SCollectionSpreadSheetWidget> CollectionSpreadSheetWidget;

	TSharedRef<SDockTab> DockableTab = SNew(SDockTab)
	[
		SAssignNew(CollectionSpreadSheetWidget, SCollectionSpreadSheetWidget)
	];

	if (CollectionSpreadSheetWidget)
	{
		if (const TObjectPtr<UDataflowBaseContent>& EditorContent = GetEditorContent())
		{
			if (Args.GetTabId() == CollectionSpreadSheetTabId_1)
			{
				DataflowCollectionSpreadSheet_1->SetCollectionSpreadSheet(CollectionSpreadSheetWidget);
			}
			else if (Args.GetTabId() == CollectionSpreadSheetTabId_2)
			{
				DataflowCollectionSpreadSheet_2->SetCollectionSpreadSheet(CollectionSpreadSheetWidget);
			}
			else if (Args.GetTabId() == CollectionSpreadSheetTabId_3)
			{
				DataflowCollectionSpreadSheet_3->SetCollectionSpreadSheet(CollectionSpreadSheetWidget);
			}
			else if (Args.GetTabId() == CollectionSpreadSheetTabId_4)
			{
				DataflowCollectionSpreadSheet_4->SetCollectionSpreadSheet(CollectionSpreadSheetWidget);
			}
		}
	}

	DockableTab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateRaw(this, &FDataflowEditorToolkit::OnTabClosed));

	return DockableTab;
}

TSharedPtr<SWidget> FDataflowEditorToolkit::CreateSimulationVisualizationWidget()
{
	FMenuBuilder MenuBuilder(false, nullptr);

	using namespace UE::Dataflow;
	for (const TPair<FName, TUniquePtr<IDataflowSimulationVisualization>>& Visualization : FDataflowSimulationVisualizationRegistry::GetInstance().GetVisualizations())
	{
		Visualization.Value->ExtendSimulationVisualizationMenu(SimulationViewportClient, MenuBuilder);
	}
	return MenuBuilder.MakeWidget();
}

TSharedRef<SDockTab> FDataflowEditorToolkit::SpawnTab_SimulationTimeline(const FSpawnTabArgs& Args)
{
	TSharedRef<SDockTab> SimulationTimelineTab = SNew(SDockTab)
		.Label(LOCTEXT("DataflowEditor_SimulationTimeline_TabTitle", "Simulation Timeline"));

	SimulationBinding = MakeShared<FDataflowSimulationBinding>(SimulationScene);
	SimulationTimelineWidget = SNew(SDataflowSimulationTimeline, SimulationBinding.ToSharedRef());
	SimulationTimelineTab->SetContent(SimulationTimelineWidget.ToSharedRef());
	
	return SimulationTimelineTab;
}

TSharedRef<SDockTab> FDataflowEditorToolkit::SpawnTab_SimulationVisualization(const FSpawnTabArgs& Args)
{
	TSharedRef<SDockTab> SimulationVisualizationTab = SNew(SDockTab)
		.Label(LOCTEXT("DataflowEditor_SimulationVisualisation_TabTitle", "Simulation Visualization"));

	SimulationVisualizationWidget = CreateSimulationVisualizationWidget();
	SimulationVisualizationTab->SetContent(SimulationVisualizationWidget.ToSharedRef());

	// Re-create the visualization panel when the simulation scene changes
	OnSimulationSceneChangedDelegateHandle = SimulationScene->GetPreviewSceneDescription()->DataflowSimulationSceneDescriptionChanged.AddLambda([this, SimulationVisualizationTab]()
	{
		SimulationVisualizationWidget = CreateSimulationVisualizationWidget();
		SimulationVisualizationTab->SetContent(SimulationVisualizationWidget.ToSharedRef());
	});

	return SimulationVisualizationTab;
}


TSharedRef<SDockTab> FDataflowEditorToolkit::SpawnTab_MembersWidget(const FSpawnTabArgs& Args)
{
	TSharedRef<SDockTab> DataflowMembersTab = SNew(SDockTab)
		.Label(LOCTEXT("DataflowEditor_DataflowMembers_TabTitle", "Dataflow Members"));

	MembersWidget = CreateDataflowMembersWidget();
	DataflowMembersTab->SetContent(MembersWidget.ToSharedRef());

	return DataflowMembersTab;
}

TSharedRef<SDockTab> FDataflowEditorToolkit::SpawnTab_OutputLog(const FSpawnTabArgs& Args)
{
	check(DataflowEditor);
	check(DataflowEditor->GetEditorContent());

	if (Args.GetTabId() == OutputLogTabId)
	{
		DataflowOutputLog = MakeShared<FDataflowOutputLog>(DataflowEditor->GetEditorContent());
	}

	using namespace UE::Dataflow;

	TSharedRef<SDockTab> DockableTab = SNew(SDockTab)
	[
		DataflowOutputLog->GetOutputLogWidget().ToSharedRef()
	];

	if (DataflowOutputLog->GetOutputLogWidget())
	{
		OnOutputLogMessageTokenClickedDelegateHandle = DataflowOutputLog->GetOnOutputLogMessageTokenClickedDelegate().AddRaw(this, &FDataflowEditorToolkit::OnOutputLogMessageTokenClicked);
	}

	DockableTab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateRaw(this, &FDataflowEditorToolkit::OnTabClosed));

	WeakOutputLogDockTab = DockableTab;

	return DockableTab;
}

void FDataflowEditorToolkit::RegisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
{
	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);

	EditorMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenu_DataflowEditor", "Dataflow Editor"));
	const TSharedRef<FWorkspaceItem> SelectionViewWorkspaceMenuCategoryRef = EditorMenuCategory->AddGroup(LOCTEXT("WorkspaceMenu_SelectionView", "Selection View"), FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Outliner"));
	const TSharedRef<FWorkspaceItem> CollectionSpreadSheetWorkspaceMenuCategoryRef = EditorMenuCategory->AddGroup(LOCTEXT("WorkspaceMenu_CollectionSpreadSheet", "Collection SpreadSheet"), FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Outliner"));
	
	InTabManager->RegisterTabSpawner(ViewportTabID, FOnSpawnTab::CreateSP(this, &FDataflowEditorToolkit::SpawnTab_Viewport))
		.SetDisplayName(LOCTEXT("ConstructionViewportTab", "Construction Viewport"))
		.SetGroup(EditorMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Viewports"));

	InTabManager->RegisterTabSpawner(SimulationViewportTabId, FOnSpawnTab::CreateSP(this, &FDataflowEditorToolkit::SpawnTab_SimulationViewport))
		.SetDisplayName(LOCTEXT("SimulationViewportTab", "Simulation Viewport"))
		.SetGroup(EditorMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Viewports"));

	InTabManager->RegisterTabSpawner(DetailsTabID, FOnSpawnTab::CreateSP(this, &FDataflowEditorToolkit::SpawnTab_AssetDetails))
		.SetDisplayName(LOCTEXT("AssetDetailsTab", "Asset Details"))
		.SetGroup(EditorMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));

	InTabManager->RegisterTabSpawner(PreviewSceneTabId, FOnSpawnTab::CreateSP(this, &FDataflowEditorToolkit::SpawnTab_PreviewScene))
		.SetDisplayName(LOCTEXT("PreviewSceneTab", "Preview Scene"))
		.SetGroup(EditorMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));
	
	InTabManager->RegisterTabSpawner(GraphCanvasTabId, FOnSpawnTab::CreateSP(this, &FDataflowEditorToolkit::SpawnTab_GraphCanvas))
		.SetDisplayName(LOCTEXT("DataflowGraphTab", "Dataflow Graph"))
		.SetGroup(EditorMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "GraphEditor.EventGraph_16x"));

	InTabManager->RegisterTabSpawner(SubGraphCanvasTabId, FOnSpawnTab::CreateSP(this, &FDataflowEditorToolkit::SpawnTab_SubGraphTab))
		.SetDisplayName(LOCTEXT("DataflowSubgraphTab", "Dataflow Subgraph"))
		.SetGroup(EditorMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "GraphEditor.EventGraph_16x"))
		.SetMenuType(ETabSpawnerMenuType::Hidden); // hide it from menus as those tabs can only be open by the toolkit itself 

	InTabManager->RegisterTabSpawner(NodeDetailsTabId, FOnSpawnTab::CreateSP(this, &FDataflowEditorToolkit::SpawnTab_NodeDetails))
		.SetDisplayName(LOCTEXT("NodeDetailsTab", "Node Details"))
		.SetGroup(EditorMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));

	if (bDataflowEnableSkeletonView)
	{
		InTabManager->RegisterTabSpawner(SkeletonViewTabId, FOnSpawnTab::CreateSP(this, &FDataflowEditorToolkit::SpawnTab_SkeletonView))
			.SetDisplayName(LOCTEXT("SkeletonTreeTab", "Skeleton Tree"))
			.SetGroup(EditorMenuCategory.ToSharedRef())
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Outliner"));
	}
	
	InTabManager->RegisterTabSpawner(OutlinerViewTabId, FOnSpawnTab::CreateSP(this, &FDataflowEditorToolkit::SpawnTab_OutlinerView))
			.SetDisplayName(LOCTEXT("SceneOutlinerTab", "Scene Outliner"))
			.SetGroup(EditorMenuCategory.ToSharedRef())
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Outliner"));

	InTabManager->RegisterTabSpawner(SelectionViewTabId_1, FOnSpawnTab::CreateSP(this, &FDataflowEditorToolkit::SpawnTab_SelectionView))
		.SetDisplayName(LOCTEXT("SelectionViewTab1", "Selection View 1"))
		.SetGroup(SelectionViewWorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Outliner"));

	InTabManager->RegisterTabSpawner(SelectionViewTabId_2, FOnSpawnTab::CreateSP(this, &FDataflowEditorToolkit::SpawnTab_SelectionView))
		.SetDisplayName(LOCTEXT("SelectionViewTab2", "Selection View 2"))
		.SetGroup(SelectionViewWorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Outliner"));

	InTabManager->RegisterTabSpawner(SelectionViewTabId_3, FOnSpawnTab::CreateSP(this, &FDataflowEditorToolkit::SpawnTab_SelectionView))
		.SetDisplayName(LOCTEXT("SelectionViewTab3", "Selection View 3"))
		.SetGroup(SelectionViewWorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Outliner"));

	InTabManager->RegisterTabSpawner(SelectionViewTabId_4, FOnSpawnTab::CreateSP(this, &FDataflowEditorToolkit::SpawnTab_SelectionView))
		.SetDisplayName(LOCTEXT("SelectionViewTab4", "Selection View 4"))
		.SetGroup(SelectionViewWorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Outliner"));

	InTabManager->RegisterTabSpawner(CollectionSpreadSheetTabId_1, FOnSpawnTab::CreateSP(this, &FDataflowEditorToolkit::SpawnTab_CollectionSpreadSheet))
		.SetDisplayName(LOCTEXT("CollectionSpreadSheetTab1", "Collection SpreadSheet 1"))
		.SetGroup(CollectionSpreadSheetWorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Outliner"));

	InTabManager->RegisterTabSpawner(CollectionSpreadSheetTabId_2, FOnSpawnTab::CreateSP(this, &FDataflowEditorToolkit::SpawnTab_CollectionSpreadSheet))
		.SetDisplayName(LOCTEXT("CollectionSpreadSheetTab2", "Collection SpreadSheet 2"))
		.SetGroup(CollectionSpreadSheetWorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Outliner"));

	InTabManager->RegisterTabSpawner(CollectionSpreadSheetTabId_3, FOnSpawnTab::CreateSP(this, &FDataflowEditorToolkit::SpawnTab_CollectionSpreadSheet))
		.SetDisplayName(LOCTEXT("CollectionSpreadSheetTab3", "Collection SpreadSheet 3"))
		.SetGroup(CollectionSpreadSheetWorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Outliner"));

	InTabManager->RegisterTabSpawner(CollectionSpreadSheetTabId_4, FOnSpawnTab::CreateSP(this, &FDataflowEditorToolkit::SpawnTab_CollectionSpreadSheet))
		.SetDisplayName(LOCTEXT("CollectionSpreadSheetTab4", "Collection SpreadSheet 4"))
		.SetGroup(CollectionSpreadSheetWorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Outliner"));
		
	InTabManager->RegisterTabSpawner(SimulationTimelineTabId, FOnSpawnTab::CreateSP(this, &FDataflowEditorToolkit::SpawnTab_SimulationTimeline))
		.SetDisplayName(LOCTEXT("SimulationTimelineTab", "Simulation Timeline"))
		.SetGroup(AssetEditorTabsCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Outliner"));
	
	InTabManager->RegisterTabSpawner(MembersWidgetTabId, FOnSpawnTab::CreateSP(this, &FDataflowEditorToolkit::SpawnTab_MembersWidget))
		.SetDisplayName(LOCTEXT("DataflowMembersTab", "Dataflow Members"))
		.SetGroup(AssetEditorTabsCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Outliner"));

	// Create a document manager to be able to spawn subgraph tabs
	DocumentManager = MakeShareable(new FDocumentTracker(SubGraphCanvasTabId));
	DocumentManager->Initialize(SharedThis(this));
	DocumentManager->SetTabManager(InTabManager);

	TSharedPtr<FDataflowEditorSubGraphTabSummoner> SubGraphTabSummoner = 
		MakeShareable( new FDataflowEditorSubGraphTabSummoner(
			SharedThis(this),
			FDataflowEditorSubGraphTabSummoner::FOnCreateGraphEditorWidget::CreateSP(this, &FDataflowEditorToolkit::CreateSubGraphEditorWidget)
		));
	DocumentManager->RegisterDocumentFactory(SubGraphTabSummoner);
	
	InTabManager->RegisterTabSpawner(OutputLogTabId, FOnSpawnTab::CreateSP(this, &FDataflowEditorToolkit::SpawnTab_OutputLog))
		.SetDisplayName(LOCTEXT("OutputLogTab", "Output Log"))
		.SetGroup(EditorMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "Log.TabIcon"));
}

void FDataflowEditorToolkit::UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	FBaseAssetToolkit::UnregisterTabSpawners(InTabManager);

	InTabManager->UnregisterTabSpawner(ViewportTabID);
	InTabManager->UnregisterTabSpawner(SimulationViewportTabId);
	InTabManager->UnregisterTabSpawner(DetailsTabID);
	InTabManager->UnregisterTabSpawner(PreviewSceneTabId);
	InTabManager->UnregisterTabSpawner(GraphCanvasTabId);
	InTabManager->UnregisterTabSpawner(SubGraphCanvasTabId);
	InTabManager->UnregisterTabSpawner(NodeDetailsTabId);
	InTabManager->UnregisterTabSpawner(SkeletonViewTabId);
	InTabManager->UnregisterTabSpawner(OutlinerViewTabId);
	InTabManager->UnregisterTabSpawner(SelectionViewTabId_1);
	InTabManager->UnregisterTabSpawner(SelectionViewTabId_2);
	InTabManager->UnregisterTabSpawner(SelectionViewTabId_3);
	InTabManager->UnregisterTabSpawner(SelectionViewTabId_4);
	InTabManager->UnregisterTabSpawner(CollectionSpreadSheetTabId_1);
	InTabManager->UnregisterTabSpawner(CollectionSpreadSheetTabId_2);
	InTabManager->UnregisterTabSpawner(CollectionSpreadSheetTabId_3);
	InTabManager->UnregisterTabSpawner(CollectionSpreadSheetTabId_4);
	InTabManager->UnregisterTabSpawner(SimulationTimelineTabId);
	InTabManager->UnregisterTabSpawner(MembersWidgetTabId);
	InTabManager->UnregisterTabSpawner(OutputLogTabId);
}

void FDataflowEditorToolkit::OnTabClosed(TSharedRef<SDockTab> Tab)
{
	if (Tab->GetTabLabel().EqualTo(FText::FromString("Selection View 1")))
	{
		ViewListeners.Remove(DataflowSelectionView_1.Get());
	}
	else if (Tab->GetTabLabel().EqualTo(FText::FromString("Selection View 2")))
	{
		ViewListeners.Remove(DataflowSelectionView_2.Get());
	}
	else if (Tab->GetTabLabel().EqualTo(FText::FromString("Selection View 3")))
	{
		ViewListeners.Remove(DataflowSelectionView_3.Get());
	}
	else if (Tab->GetTabLabel().EqualTo(FText::FromString("Selection View 4")))
	{
		ViewListeners.Remove(DataflowSelectionView_4.Get());
	}
	else if (Tab->GetTabLabel().EqualTo(FText::FromString("Collection SpreadSheet 1")))
	{
		ViewListeners.Remove(DataflowCollectionSpreadSheet_1.Get());
	}
	else if (Tab->GetTabLabel().EqualTo(FText::FromString("Collection SpreadSheet 2")))
	{
		ViewListeners.Remove(DataflowCollectionSpreadSheet_2.Get());
	}
	else if (Tab->GetTabLabel().EqualTo(FText::FromString("Collection SpreadSheet 3")))
	{
		ViewListeners.Remove(DataflowCollectionSpreadSheet_3.Get());
	}
	else if (Tab->GetTabLabel().EqualTo(FText::FromString("Collection SpreadSheet 4")))
	{
		ViewListeners.Remove(DataflowCollectionSpreadSheet_4.Get());
	}
	else if (Tab->GetTabLabel().EqualTo(FText::FromString("Skeleton Tree")))
	{
		ViewListeners.Remove(SkeletonEditorView.Get());
	}
	else if (Tab->GetTabLabel().EqualTo(FText::FromString("Scene Outliner")))
	{
		ViewListeners.Remove(DataflowOutlinerView.Get());
	}
}

void FDataflowEditorToolkit::SetSubGraphTabActiveState(TSharedPtr<SDataflowGraphEditor> SubGraphEditor, bool bActive)
{
	if (bActive)
	{
		ActiveSubGraphEditorWeakPtr = SubGraphEditor;
	}
	else
	{
		// only reset to null if this was th previously active subgraph
		if (ActiveSubGraphEditorWeakPtr == SubGraphEditor)
		{
			ActiveSubGraphEditorWeakPtr.Reset();
		}
	}
}

UDataflowSubGraph* FDataflowEditorToolkit::GetSubGraph(FName SubGraphName) const
{
	if (const TObjectPtr<UDataflowBaseContent>& EditorContent = GetEditorContent())
	{
		ensure(EditorContent);
		if (UDataflow* DataflowAsset = EditorContent->GetDataflowAsset())
		{
			return DataflowAsset->FindSubGraphByName(SubGraphName);
		}
	}
	return nullptr;
}

UDataflowSubGraph* FDataflowEditorToolkit::GetSubGraph(const FGuid& SubGraphGuid) const
{
	if (const TObjectPtr<UDataflowBaseContent>& EditorContent = GetEditorContent())
	{
		ensure(EditorContent);
		if (UDataflow* DataflowAsset = EditorContent->GetDataflowAsset())
		{
			return DataflowAsset->FindSubGraphByGuid(SubGraphGuid);
		}
	}
	return nullptr;
}

void FDataflowEditorToolkit::OpenSubGraphTab(FName SubGraphName)
{
	OpenSubGraphTab(GetSubGraph(SubGraphName));
}

void FDataflowEditorToolkit::OpenSubGraphTab(const UDataflowSubGraph* SubGraph)
{
	if (SubGraph)
	{
		TSharedRef<FTabPayload_UObject> Payload = FTabPayload_UObject::Make(SubGraph);
		DocumentManager->OpenDocument(Payload, FDocumentTracker::EOpenDocumentCause::OpenNewDocument);
	}
}

void FDataflowEditorToolkit::CloseSubGraphTab(const UDataflowSubGraph* SubGraph)
{
	if (SubGraph)
	{
		TSharedRef<FTabPayload_UObject> Payload = FTabPayload_UObject::Make(SubGraph);
		DocumentManager->CloseTab(Payload);
	}
}

void FDataflowEditorToolkit::ReOpenSubGraphTab(const UDataflowSubGraph* SubGraph)
{
	CloseSubGraphTab(SubGraph);
	OpenSubGraphTab(SubGraph);
}

TSharedRef<SGraphEditor> FDataflowEditorToolkit::CreateSubGraphEditorWidget(TSharedRef<FTabInfo> InTabInfo, UDataflowSubGraph* InGraph)
{
	if (const TObjectPtr<UDataflowBaseContent>& EditorContent = GetEditorContent())
	{
		return CreateGraphEditorWidget(InGraph, NodeDetailsEditor);
	}
	return SNew(SGraphEditor);
}

void FDataflowEditorToolkit::FindAllVariableNodeInGraph(UEdGraph* EdGraph, FName VariableName, TArray<UDataflowEdNode*>& OutEdNodes)
{
	for (UEdGraphNode* EdNode : EdGraph->Nodes)
	{
		if (UDataflowEdNode* DataflowEdNode = Cast<UDataflowEdNode>(EdNode))
		{
			if (TSharedPtr<FDataflowNode> DataflowNode = DataflowEdNode->GetDataflowNode())
			{
				if (FGetDataflowVariableNode* VariableNode = DataflowNode->AsType<FGetDataflowVariableNode>())
				{
					if (VariableNode->GetVariableName() == VariableName)
					{
						OutEdNodes.Add(DataflowEdNode);
					}
				}
			}
		}
	}
}

FGuid FDataflowEditorToolkit::FocusOnNextVariableNode(FName VariableName, const FGuid& LastTimeNodeGuid)
{
	if (const TObjectPtr<UDataflowBaseContent>& EditorContent = GetEditorContent())
	{
		if (UDataflow* DataflowAsset = EditorContent->GetDataflowAsset())
		{
			// first collect all the nodes that match the specific variable 
			TArray<UDataflowEdNode*> EdNodes;
			FindAllVariableNodeInGraph(DataflowAsset, VariableName, EdNodes);
			for (TObjectPtr<UDataflowSubGraph> SubGraphs : DataflowAsset->GetSubGraphs())
			{
				FindAllVariableNodeInGraph(SubGraphs, VariableName, EdNodes);
			}

			const int32 LastTimeIndex = EdNodes.IndexOfByPredicate(
				[&LastTimeNodeGuid](const UDataflowEdNode* EdNode)
				{
					return (EdNode && EdNode->GetDataflowNodeGuid() == LastTimeNodeGuid);
				});

			int32 IndexToUse = 0;
			if (LastTimeIndex != INDEX_NONE && EdNodes.Num() > 0)
			{
				IndexToUse = (LastTimeIndex + 1) % EdNodes.Num();
			}

			if (EdNodes.IsValidIndex(IndexToUse))
			{
				if (UDataflowEdNode* NextVariableNode = EdNodes[IndexToUse])
				{
					TSharedPtr<SDataflowGraphEditor> ActiveEditor = GraphEditor;
					if (UDataflowSubGraph* SubGraph = Cast<UDataflowSubGraph>(NextVariableNode->GetGraph()))
					{
						OpenSubGraphTab(SubGraph);
						ActiveEditor = ActiveSubGraphEditorWeakPtr.Pin();
					}
					else
					{
						if (GraphEditorTab)
						{
							GraphEditorTab->DrawAttention();
						}
					}
					if (ActiveEditor)
					{
						// Clear node selection in the newly active graph
						ActiveEditor->ClearSelectionSet();

						// now jump to the relevant node 
						ActiveEditor->JumpToNode(NextVariableNode);
					}
					return NextVariableNode->GetDataflowNodeGuid();
				}
			}
		}
	}
	return FGuid();
}

FName FDataflowEditorToolkit::GetToolkitFName() const
{
	return FName("DataflowEditor");
}

FText FDataflowEditorToolkit::GetToolkitName() const
{
	if (const TObjectPtr<UDataflowBaseContent>& EditorContent = GetEditorContent())
	{
		if(EditorContent->GetDataflowOwner())
		{
			// If we have an asset userdata type it is potentially a wrapper around a dataflow asset.
			// In this case, check for an underlying asset and prefer to use the asset name rather
			// than the name of the wrapper for the tab label
			const UAssetUserData* AsUserData = Cast<UAssetUserData>(EditorContent->GetDataflowOwner());

			if(!AsUserData || !EditorContent->GetDataflowAsset())
			{
				return  GetLabelForObject(EditorContent->GetDataflowOwner());
			}
		}

		if (EditorContent->GetDataflowAsset())
		{
			return  GetLabelForObject(EditorContent->GetDataflowAsset());
		}
	}
	return  LOCTEXT("ToolkitName", "Empty Dataflow Editor");
}

FText FDataflowEditorToolkit::GetBaseToolkitName() const
{
	return LOCTEXT("AppLabel", "Dataflow Editor");
}

FText FDataflowEditorToolkit::GetToolkitToolTipText() const
{
	return  LOCTEXT("ToolkitToolTipText", "Dataflow Editor");
}

FString FDataflowEditorToolkit::GetWorldCentricTabPrefix() const
{
	return LOCTEXT("WorldCentricTabPrefix", "Dataflow").ToString();
}

FLinearColor FDataflowEditorToolkit::GetWorldCentricTabColorScale() const
{
	return FLinearColor(0.3f, 0.2f, 0.5f, 0.5f);
}

void FDataflowEditorToolkit::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObjects(SelectedDataflowNodes);
	Collector.AddReferencedObject(PropertyGizmoManager);
	//Collector.AddReferencedObject(PrimarySelection);
	if(ConstructionScene)
	{
		Collector.AddReferencedObject(ConstructionScene->ModifyDebugDrawComponent());
	}
	if(SimulationScene)
	{
		Collector.AddReferencedObject(SimulationScene->ModifyDebugDrawComponent());
	}
}

const FString& FDataflowEditorToolkit::GetDebugDrawOverlayString() const
{
	return DebugDrawOverlayString;
}

void FDataflowEditorToolkit::DrawPropertyGizmoLabels(FCanvas* Canvas, const FSceneView* SceneView) const
{
	if (PropertyGizmoManager)
	{
		PropertyGizmoManager->DrawPropertyLabels(Canvas, SceneView);
	}
}

void FDataflowEditorToolkit::OnConstructionViewModeChanged(const UE::Dataflow::IDataflowConstructionViewMode* NewConstructionViewMode)
{
	UpdateGizmos();
}

void FDataflowEditorToolkit::UpdateGizmos()
{
	// Update property gizmos whenever the view is updated
	if (PropertyGizmoManager)
	{
		bool bCanDisplayGizmo = false;
		if (UDataflowEditorMode* const DataflowMode = Cast<UDataflowEditorMode>(EditorModeManager->GetActiveScriptableMode(UDataflowEditorMode::EM_DataflowEditorModeId)))
		{
			if (const UE::Dataflow::IDataflowConstructionViewMode* const ConstructionViewMode = DataflowMode->GetConstructionViewMode())
			{
				bCanDisplayGizmo = ConstructionViewMode->CanDisplayGizmo();
			}

			UInteractiveToolManager* ToolManager = DataflowMode->GetInteractiveToolsContext()->ToolManager;
			PropertyGizmoManager->Teardown(ToolManager);
			if (bCanDisplayGizmo)
			{
				if (SelectedDataflowNodes.Num() == 1)
				{
					PropertyGizmoManager->Setup(ToolManager, *SelectedDataflowNodes.begin());
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
