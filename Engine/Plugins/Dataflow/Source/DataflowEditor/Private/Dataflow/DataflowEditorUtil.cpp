// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowEditorUtil.h"

#include "Animation/Skeleton.h"
#include "Dataflow/DataflowEditor.h"
#include "Dataflow/DataflowContent.h"
#include "Dataflow/DataflowEditorToolkit.h"
#include "Dataflow/DataflowEdNode.h"
#include "Dataflow/DataflowInstance.h"
#include "Dataflow/DataflowObject.h"
#include "Dataflow/DataflowRenderingFactory.h"
#include "Dataflow/DataflowRenderingViewMode.h"
#include "DataflowRendering/DataflowRenderableTypeInstance.h"
#include "DynamicMesh/MeshNormals.h"
#include "Engine/SkeletalMesh.h"
#include "Animation/Skeleton.h"
#include "Misc/SlowTask.h"
#include "GeometryCollection/Facades/CollectionRenderingFacade.h"
#include "Materials/Material.h"

using namespace UE::Geometry;

namespace UE::Dataflow::Private
{
	// disable slow task for the progress tracking as this keep on stealing focus from other operation 
	// we need to shift to a true asynchronous model for the Dataflow editor so that the game thread is free to update progress toast notification
	static bool bUseSlowTaskForProgressNotification = false;
	static FAutoConsoleVariableRef CVarUseSlowTaskForProgressNotification(TEXT("p.Dataflow.UseSlowTaskForProgressNotification"), bUseSlowTaskForProgressNotification, TEXT("When enabled the progress notification class will use the SlowTask system to diaply progress"));


	bool HasSkeletalMesh(UObject* InObject)
	{
		if (UClass* Class = InObject->GetClass())
		{
			return Class->FindPropertyByName(FName("SkeletalMesh")) &&
				   Class->FindPropertyByName(FName("Skeleton"));
		}
		return false;
	}
	
	bool HasDataflowAsset(UObject* InObject)
	{
		return UE::Dataflow::InstanceUtils::HasValidDataflowAsset(InObject);
	}
	
	UDataflow* GetDataflowAssetFrom(UObject* InObject)
	{
		return UE::Dataflow::InstanceUtils::GetDataflowAssetFromObject(InObject);
	}

	USkeletalMesh* GetSkeletalMeshFrom(UObject* InObject)
	{
		if (UClass* Class = InObject->GetClass())
		{
			if (FProperty* Property = Class->FindPropertyByName(FName("SkeletalMesh")))
			{
				return *Property->ContainerPtrToValuePtr<USkeletalMesh*>(InObject);
			}
		}
		return nullptr;
	}

	USkeleton* GetSkeletonFrom(UObject* InObject)
	{
		if (UClass* Class = InObject->GetClass())
		{
			if (FProperty* Property = Class->FindPropertyByName(FName("Skeleton")))
			{
				return *Property->ContainerPtrToValuePtr<USkeleton*>(InObject);
			}
		}
		return nullptr;
	}
	
	UAnimationAsset* GetAnimationAssetFrom(UObject* InObject)
	{
		if (UClass* Class = InObject->GetClass())
		{
			if (FProperty* Property = Class->FindPropertyByName(FName("AnimationAsset")))
			{
				return *Property->ContainerPtrToValuePtr<UAnimationAsset*>(InObject);
			}
		}
		return nullptr;
	}

	FString GetDataflowTerminalFrom(UObject* InObject)
	{
		return UE::Dataflow::InstanceUtils::GetTerminalNodeNameFromObject(InObject).ToString();
	}
};


namespace UE
{
	namespace Material
	{
		UMaterial* LoadMaterialFromPath(const FName& InPath, UObject* Outer)
		{
			if (InPath == NAME_None) return nullptr;

			return Cast<UMaterial>(StaticLoadObject(UMaterial::StaticClass(), Outer, *InPath.ToString()));
		}
	}

}

namespace UE::Dataflow
{
	TSharedPtr<FEngineContext> GetContext(TObjectPtr<UDataflowBaseContent> Content)
	{
		if (Content)
		{
			if (!Content->GetDataflowContext())
			{
				Content->SetDataflowContext(MakeShared<FEngineContext>(Content->GetDataflowOwner()));
			}
			return Content->GetDataflowContext();
		}

		ensure(false);
		return MakeShared<FEngineContext>(nullptr);
	}

	bool IsViewModeSupported(const UDataflowEdNode& EdNode, const UDataflowBaseContent& EditorContent, const IDataflowConstructionViewMode& ViewMode, bool bCheckDebugDraw)
	{
		if (const TSharedPtr<FEngineContext> Context = EditorContent.GetDataflowContext())
		{
			if (TSharedPtr<const FDataflowNode> DataflowNode = EdNode.GetDataflowNode())
			{
				// first check the rendering collection rendering system (legacy)
				if (const FRenderingFactory* const Factory = FRenderingFactory::GetInstance())
				{
					for (const FRenderingParameter& Parameter : EdNode.GetRenderParameters())
					{
						if (Factory->CanRenderNodeOutput(FGraphRenderingState{ EdNode.GetDataflowNodeGuid(), DataflowNode.Get(), Parameter, *Context, ViewMode }))
						{
							return true;
						}
					}
				}

				// check if the node needs debug draw
				if (bCheckDebugDraw && DataflowNode->CanDebugDrawViewMode(ViewMode.GetName()))
				{
					return true;
				}

				// check the component based rendering system (new)
				TArray<FRenderableTypeInstance> RenderableInstances;
				FRenderableTypeInstance::GetRenderableInstancesForNode(*Context, ViewMode, *DataflowNode, /*bEvaluateOutputs*/false, RenderableInstances);
				if (RenderableInstances.Num() > 0)
				{
					return true;
				}
			}
		}
		return false;
	}

	bool CanRenderNodeOutput(const UDataflowEdNode& EdNode, const UDataflowBaseContent& EditorContent, const IDataflowConstructionViewMode& ViewMode, bool bCheckDebugDraw)
	{
		if (const TSharedPtr<FEngineContext> Context = EditorContent.GetDataflowContext())
		{
			if (TSharedPtr<const FDataflowNode> NodeTarget = EdNode.GetDataflowGraph()->FindBaseNode(FName(EdNode.GetName())))
			{
				if (const FRenderingFactory* const Factory = FRenderingFactory::GetInstance())
				{
					for (const FRenderingParameter& Parameter : EdNode.GetRenderParameters())
					{
						if (Factory->CanRenderNodeOutput(FGraphRenderingState{ EdNode.GetDataflowNodeGuid(), NodeTarget.Get(), Parameter, *Context.Get(), ViewMode }))
						{
							return true;
						}
					}
				}
				if (bCheckDebugDraw && NodeTarget->CanDebugDrawViewMode(ViewMode.GetName()))
				{
					return true;
				}
			}
		}

		return false;
	}

	bool RenderNodeOutput(GeometryCollection::Facades::FRenderingFacade& RenderingFacade, const UDataflowEdNode& Node, const UDataflowBaseContent& EditorContent, const bool bEvaluateOutputs)
	{
		const TObjectPtr<UDataflow>& DataflowAsset = EditorContent.GetDataflowAsset();
		const IDataflowConstructionViewMode* ConstructionViewMode = EditorContent.GetConstructionViewMode();
		const TSharedPtr<FEngineContext>& DataflowContext = EditorContent.GetDataflowContext();

		bool bHasRenderCollectionPrimitives = false;
		if (DataflowAsset && DataflowContext && ConstructionViewMode)
		{
			if (FRenderingFactory* const Factory = FRenderingFactory::GetInstance())
			{
				for (const FRenderingParameter& Parameter : Node.GetRenderParameters())
				{
					if (const TSharedPtr<FGraph> Graph = DataflowAsset->GetDataflow())
					{
						if (TSharedPtr<const FDataflowNode> NodeTarget = Graph->FindBaseNode(FName(Node.GetName())))
						{
							Factory->RenderNodeOutput(RenderingFacade, FGraphRenderingState{ Node.GetDataflowNodeGuid(), NodeTarget.Get(), Parameter, *DataflowContext, *ConstructionViewMode, bEvaluateOutputs });
							bHasRenderCollectionPrimitives = bHasRenderCollectionPrimitives || NodeTarget->HasRenderCollectionPrimitives();
						}
					}
				}
			}
		}
		return bHasRenderCollectionPrimitives;
	}


	void GetViewModesForNode(const UDataflowEdNode& EdNode, const UDataflowBaseContent& EditorContent, TArray<FName>& OutValidViewModeNames)
	{
		TArray<FName> AllViewModeNames;
		FRenderingViewModeFactory::GetInstance().GetViewModes().GetKeys(AllViewModeNames);

		for (const FName& ViewModeName : AllViewModeNames)
		{
			if (const IDataflowConstructionViewMode* const ViewMode = FRenderingViewModeFactory::GetInstance().GetViewMode(ViewModeName))
			{
				if (IsViewModeSupported(EdNode, EditorContent, *ViewMode))
				{
					OutValidViewModeNames.Add(ViewModeName);
				}
			}
		}
	}

	FScopedProgressNotification::FScopedProgressNotification(const FText& Title, int32 InMaxProgress, float DelayInSeconds)
		: MaxProgress(InMaxProgress)
	{
		if (Private::bUseSlowTaskForProgressNotification)
		{
			SlowTask = MakeUnique<FSlowTask>(static_cast<float>(MaxProgress), Title);
			SlowTask->Initialize();
			if (DelayInSeconds != 0)
			{
				SlowTask->MakeDialogDelayed(DelayInSeconds);
			}
			else
			{
				SlowTask->MakeDialog(false);
			}
		}
	}

	FScopedProgressNotification::~FScopedProgressNotification()
	{
		if (SlowTask)
		{
			SlowTask->Destroy();
		}
	}

	void FScopedProgressNotification::SetProgress(float InProgress, const FText& Message)
	{
		const float ClampedProgress = FMath::Clamp(InProgress, 0.f, MaxProgress);

		const float DeltaProgressSinceLastUpdate = FMath::Max(0, (ClampedProgress - LastUpdateProgress));
		if (UpdateSteps <= 0 || DeltaProgressSinceLastUpdate > UpdateSteps)
		{
			const float DeltaProgress = FMath::Max(0, InProgress - CurrentProgress);
			if (SlowTask)
			{
				SlowTask->EnterProgressFrame(DeltaProgress, Message);
			}
			LastUpdateProgress = ClampedProgress;
		}
		CurrentProgress = ClampedProgress;
	}

	void FScopedProgressNotification::AddProgress(float InDeltaProgress, const FText& Message)
	{
		SetProgress((CurrentProgress + InDeltaProgress), Message);
	}

	void FScopedProgressNotification::SetUpdateSteps(float InUpdateSteps)
	{
		UpdateSteps = InUpdateSteps;
	}
}