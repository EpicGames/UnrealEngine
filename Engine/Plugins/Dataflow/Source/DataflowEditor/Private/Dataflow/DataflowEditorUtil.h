// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/UnrealString.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Dataflow/DataflowObjectInterface.h"
#include "Templates/SharedPointer.h"
#include "Framework/Notifications/NotificationManager.h"

struct FSlowTask;
class UDataflow;
class UObject;
class UDataflowBaseContent;
class USkeletalMesh;
class USkeleton;
class UAnimationAsset;
class UMaterial;
class UDataflowEdNode;

namespace UE::Dataflow { class IDataflowConstructionViewMode; }
namespace GeometryCollection::Facades
{
	class FRenderingFacade;
};

namespace UE::Dataflow::Private
{
	bool HasSkeletalMesh(UObject* InObject);

	bool HasDataflowAsset(UObject* InObject);
	
	UDataflow* GetDataflowAssetFrom(UObject* InObject);

	USkeletalMesh* GetSkeletalMeshFrom(UObject* InObject);

	USkeleton* GetSkeletonFrom(UObject* InObject);

	UAnimationAsset* GetAnimationAssetFrom(UObject* InObject);

	FString GetDataflowTerminalFrom(UObject* InObject);
};

namespace UE
{
	namespace Material
	{
		UMaterial* LoadMaterialFromPath( const FName& Path, UObject* Outer);
	}
}

namespace UE::Dataflow
{
	TSharedPtr<UE::Dataflow::FEngineContext> GetContext(TObjectPtr<UDataflowBaseContent> Content);

	bool IsViewModeSupported(const UDataflowEdNode& EdNode, const UDataflowBaseContent& EditorContent, const IDataflowConstructionViewMode& ViewMode, bool bCheckDebugDraw = true);

	bool CanRenderNodeOutput(const UDataflowEdNode& EdNode, const UDataflowBaseContent& EditorContent, const UE::Dataflow::IDataflowConstructionViewMode& ViewMode, bool bCheckDebugDraw = true);

	/** This function fills the managed array collection linked to the rendering facade from the callbacks.
	 * Returns true if the resulting collection should be directly rendered of false if the collection will be used to generate a primitive component */
	bool RenderNodeOutput(GeometryCollection::Facades::FRenderingFacade& RenderingFacade, const UDataflowEdNode& Node, const UDataflowBaseContent& EditorContent, const bool bEvaluateOutputs);
	void GetViewModesForNode(const UDataflowEdNode& EdNode, const UDataflowBaseContent& EditorContent, TArray<FName>& OutValidViewModeNames);


	struct FScopedProgressNotification
	{
	public:
		FScopedProgressNotification(const FText& Title, int32 InMaxProgress = 1.0f, float DelayInSeconds = 0.1f);
		~FScopedProgressNotification();

		void SetUpdateSteps(float InUpdateSteps);

		void SetProgress(float InProgress, const FText& Message = FText::GetEmpty());
		void AddProgress(float InDeltaProgress, const FText& Message = FText::GetEmpty());

	private:
		TUniquePtr<FSlowTask> SlowTask;
		float CurrentProgress = 0.0f;
		float LastUpdateProgress = 0.0f;
		float UpdateSteps = 0.0f;
		float MaxProgress = 1.0f;
	};
}