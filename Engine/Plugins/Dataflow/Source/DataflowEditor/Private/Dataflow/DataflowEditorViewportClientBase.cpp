// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowEditorViewportClientBase.h"
#include "Dataflow/DataflowEditorPreviewSceneBase.h"
#include "EditorModeManager.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "BaseBehaviors/ClickDragBehavior.h"
#include "AssetViewerSettings.h"
#include "Dataflow/DataflowElement.h"
#include "Dataflow/DataflowElement.h"

#define LOCTEXT_NAMESPACE "DataflowEditorViewportClientBase"

FDataflowEditorViewportClientBase::FDataflowEditorViewportClientBase(FEditorModeTools* InModeTools,
                                                             const TWeakPtr<FDataflowPreviewSceneBase> InPreviewScene, const bool bCouldTickScene,
                                                             const TWeakPtr<SEditorViewport> InEditorViewportWidget)
	: FEditorViewportClient(InModeTools, InPreviewScene.Pin().Get(), InEditorViewportWidget)
	, DataflowPreviewScene(InPreviewScene)
{
	BehaviorSet = NewObject<UInputBehaviorSet>();

	// The ClickDragBehavior is used to intercept non-alt left-mouse-button drag inputs, but still allow single-click for select/deselect operation
	const TObjectPtr<ULocalClickDragInputBehavior> ClickDragBehavior = NewObject<ULocalClickDragInputBehavior>();
	ClickDragBehavior->Initialize();

	// We'll have the priority of our viewport behaviors be lower (i.e. higher numerically) than both the gizmo default and the tool default
	constexpr int ViewportBehaviorPriority = FMath::Max(FInputCapturePriority::DEFAULT_GIZMO_PRIORITY, FInputCapturePriority::DEFAULT_TOOL_PRIORITY) + 1;
	ClickDragBehavior->SetDefaultPriority(ViewportBehaviorPriority);

	ClickDragBehavior->ModifierCheckFunc = [](const FInputDeviceState& InputState) 
	{
		return !FInputDeviceState::IsAltKeyDown(InputState);
	};

	ClickDragBehavior->CanBeginClickDragFunc = [](const FInputDeviceRay& InputDeviceRay)
	{
		return FInputRayHit(TNumericLimits<float>::Max()); // bHit is true. Depth is max to lose the standard tiebreaker.
	};

	ClickDragBehavior->OnClickPressFunc = [this](const FInputDeviceRay& ClickPos)
	{
		HHitProxy* const HitProxy = Viewport->GetHitProxy(ClickPos.ScreenPosition[0], ClickPos.ScreenPosition[1]);
		OnViewportClicked(HitProxy);
	};

	BaseBehaviors.Add(ClickDragBehavior);
	BehaviorSet->Add(ClickDragBehavior);

	ModeTools->GetInteractiveToolsContext()->InputRouter->RegisterSource(this);

	RegisterDelegates();
}

FDataflowEditorViewportClientBase::~FDataflowEditorViewportClientBase()
{
	DeregisterDelegates();
}

const UInputBehaviorSet* FDataflowEditorViewportClientBase::GetInputBehaviors() const
{
	return BehaviorSet;
}

void FDataflowEditorViewportClientBase::AddReferencedObjects(FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(Collector);
	Collector.AddReferencedObjects(BaseBehaviors);
	Collector.AddReferencedObject(BehaviorSet);
}

void FDataflowEditorViewportClientBase::RegisterDelegates()
{
	// Remove any existing delegate in case this function is called twice
	UAssetViewerSettings::Get()->OnAssetViewerSettingsChanged().Remove(OnAssetViewerSettingsChangedDelegateHandle);

	auto SyncEngineShowFlags = [this]()
		{
			const TSharedPtr<FDataflowPreviewSceneBase> DataflowPreviewScenePtr = DataflowPreviewScene.Pin();

			if (ensureMsgf(DataflowPreviewScenePtr, TEXT("Expected valid DataflowPreviewScene pointer")))
			{
				const int32 CurrentProfileIndex = DataflowPreviewScenePtr->GetCurrentProfileIndex();

				const UAssetViewerSettings* const DefaultSettings = UAssetViewerSettings::Get();

				checkf(DefaultSettings && DefaultSettings->Profiles.IsValidIndex(CurrentProfileIndex), TEXT("Invalid default settings pointer or current profile index"));

				const FPreviewSceneProfile& Profile = DefaultSettings->Profiles[CurrentProfileIndex];
				DrawHelper.bDrawGrid = EngineShowFlags.Grid;

				if (Profile.bPostProcessingEnabled)
				{
					EngineShowFlags.EnableAdvancedFeatures();
				}
				else
				{
					EngineShowFlags.DisableAdvancedFeatures();
				}
			}
		};

	// Run the lambda once to get things in sync
	SyncEngineShowFlags();

	OnAssetViewerSettingsChangedDelegateHandle = UAssetViewerSettings::Get()->OnAssetViewerSettingsChanged().AddLambda([this, SyncEngineShowFlags](const FName& InPropertyName)
	{
		if ((InPropertyName == GET_MEMBER_NAME_CHECKED(FPreviewSceneProfile, bPostProcessingEnabled)) ||
			(InPropertyName == NAME_None))		// NAME_None is passed in when the current profile changes
		{
			SyncEngineShowFlags();
		}
	});

	if (const TSharedPtr<FDataflowPreviewSceneBase> DataflowPreviewScenePtr = DataflowPreviewScene.Pin())
	{
		OnFocusRequestDelegateHandle = DataflowPreviewScenePtr->OnFocusRequest().AddRaw(this, &FDataflowEditorViewportClientBase::HandleFocusRequest);
	}
}

void FDataflowEditorViewportClientBase::DeregisterDelegates()
{
	UAssetViewerSettings::Get()->OnAssetViewerSettingsChanged().Remove(OnAssetViewerSettingsChangedDelegateHandle);

	if (const TSharedPtr<FDataflowPreviewSceneBase> DataflowPreviewScenePtr = DataflowPreviewScene.Pin())
	{
		DataflowPreviewScenePtr->OnFocusRequest().Remove(OnFocusRequestDelegateHandle);
	}
}

void FDataflowEditorViewportClientBase::HandleFocusRequest(const FBox& BoundingBox)
{
	FocusViewportOnBox(BoundingBox);
}

void FDataflowEditorViewportClientBase::Draw(const FSceneView* View, FPrimitiveDrawInterface* PDI)
{
	Super::Draw(View, PDI);
}

void FDataflowEditorViewportClientBase::GetSelectedElements(HHitProxy* HitProxy, TArray<FDataflowBaseElement*>& SelectedElements) const
{
	TArray<FDataflowBaseElement*> DataflowElements;
	if (const TSharedPtr<FDataflowPreviewSceneBase> DataflowPreviewScenePtr = DataflowPreviewScene.Pin())
	{
		for(IDataflowDebugDrawInterface::FDataflowElementsType::ElementType& DataflowElement : DataflowPreviewScenePtr->ModifySceneElements())
		{
			if(DataflowElement.IsValid())
			{
				DataflowElement->bIsSelected = false;
			}
		}
		if (HitProxy && HitProxy->IsA(HDataflowElementHitProxy::StaticGetType()))
		{
			if(const HDataflowElementHitProxy* DataflowElementProxy = static_cast<HDataflowElementHitProxy*>(HitProxy))
			{
				if(DataflowElementProxy->ElementIndex < DataflowPreviewScenePtr->ModifySceneElements().Num())
				{
					IDataflowDebugDrawInterface::FDataflowElementsType::ElementType& DataflowElement = DataflowPreviewScenePtr->ModifySceneElements()[DataflowElementProxy->ElementIndex];

					if(DataflowElement.IsValid())
					{
						DataflowElement->bIsSelected = true;
						SelectedElements.Add(DataflowElement.Get());
					}
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE 
