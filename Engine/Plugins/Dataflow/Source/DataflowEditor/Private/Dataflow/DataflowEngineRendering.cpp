// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowEngineRendering.h"

#include "Dataflow/DataflowConnectionTypes.h"
#include "Dataflow/DataflowEditorModule.h"
#include "Dataflow/DataflowRenderingFactory.h"
#include "Dataflow/DataflowMesh.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/MeshNormals.h"
#include "Field/FieldSystemTypes.h"
#include "GeometryCollection/GeometryCollectionUtility.h"
#include "GeometryCollection/Facades/CollectionRenderingFacade.h"
#include "GeometryCollection/Facades/CollectionExplodedVectorFacade.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "Materials/MaterialInterface.h"
#include "GeometryCollection/Facades/CollectionUVFacade.h"
#include "GeometryCollection/Facades/CollectionMeshFacade.h"
#include "GeometryCollection/Facades/CollectionVertexBoneWeightsFacade.h"
#include "UDynamicMesh.h"

namespace UE::Dataflow
{
	namespace Private
	{
		int32 DataflowPointRenderLimit = 100000;
		FAutoConsoleVariableRef CVARDataflowPointRenderLimit(TEXT("p.Dataflow.PointRenderLimit"), DataflowPointRenderLimit,
			TEXT("Limit for the number of points rendered in a dataflow node visualization. Set to -1 to remove limit."));


		void RenderDynamicMesh(GeometryCollection::Facades::FRenderingFacade& RenderCollection, 
			const FString& GeometryGroupName, 
			const UE::Geometry::FDynamicMesh3& DynamicMesh, 
			const TArray<TObjectPtr<UMaterialInterface>>& Materials)
		{
			const int32 NumVertices = DynamicMesh.VertexCount();
			const int32 NumTriangles = DynamicMesh.TriangleCount();

			if (NumVertices > 0 && NumTriangles > 0)
			{
				// This will contain the valid triangles only
				TArray<FIntVector> Tris; Tris.Reserve(DynamicMesh.TriangleCount());
				TArray<int32> MaterialIDs; MaterialIDs.Reserve(DynamicMesh.TriangleCount());
				const UE::Geometry::FDynamicMeshMaterialAttribute* const MaterialAttribute = DynamicMesh.Attributes()->GetMaterialID();

				// DynamicMesh.TrianglesItr() returns the valid triangles only
				for (const int32 TriangleID : DynamicMesh.TriangleIndicesItr())
				{
					const UE::Geometry::FIndex3i Tri = DynamicMesh.GetTriangle(TriangleID);
					Tris.Add(FIntVector(Tri.A, Tri.B, Tri.C));
					if (MaterialAttribute)
					{
						MaterialIDs.Add(MaterialAttribute->GetValue(TriangleID));
					}
				}

				// This will contain all the vertices (invalid ones too)
				// Otherwise the IDs need to be remaped
				TArray<FVector3f> Vertices; Vertices.AddZeroed(DynamicMesh.MaxVertexID());

				// DynamicMesh.VertexIndicesItr() returns the valid vertices only
				for (int32 VertexID : DynamicMesh.VertexIndicesItr())
				{
					Vertices[VertexID] = (FVector3f)DynamicMesh.GetVertex(VertexID);
				}

				TArray<FVector3f> VertexNormals;
				VertexNormals.AddUninitialized(Vertices.Num());
				if (DynamicMesh.HasVertexNormals())
				{
					for (int32 VertexID : DynamicMesh.VertexIndicesItr())
					{
						VertexNormals[VertexID] = DynamicMesh.GetVertexNormal(VertexID);
					}
				}
				else if (DynamicMesh.HasAttributes() && DynamicMesh.Attributes()->PrimaryNormals())
				{
					const UE::Geometry::FDynamicMeshNormalOverlay* const NormalOverlay = DynamicMesh.Attributes()->PrimaryNormals();
					for (int32 VertexID : DynamicMesh.VertexIndicesItr())
					{
						TArray<int> OverlayElements;
						NormalOverlay->GetVertexElements(VertexID, OverlayElements);

						FVector3f AvgNormal(0.0f, 0.0f, 0.0f);
						if (OverlayElements.Num() > 0)
						{
							for (int32 ElementID : OverlayElements)
							{
								AvgNormal += NormalOverlay->GetElement(ElementID);
							}
							AvgNormal /= (float)OverlayElements.Num();
						}

						VertexNormals[VertexID] = AvgNormal;
					}
				}
				else
				{
					// No vertex normals and no overlay: compute per-vertex normals
					UE::Geometry::FMeshNormals MeshNormals(&DynamicMesh);
					MeshNormals.ComputeVertexNormals();
					const TArray<FVector3d>& ComputedNormals = MeshNormals.GetNormals();
					check(ComputedNormals.Num() == Vertices.Num());

					for (int32 VertexIndex = 0; VertexIndex < Vertices.Num(); ++VertexIndex)
					{
						VertexNormals[VertexIndex] = (FVector3f)ComputedNormals[VertexIndex];
					}
				}

				// Add VertexNormal and VertexColor
				TArray<FLinearColor> VertexColors; VertexColors.AddUninitialized(Vertices.Num());
				
				if (DynamicMesh.HasVertexColors())
				{
					for (int32 VertexID : DynamicMesh.VertexIndicesItr())
					{
						VertexColors[VertexID] = DynamicMesh.GetVertexColor(VertexID);
					}
				}
				else
				{
					for (int32 VertexIdx = 0; VertexIdx < VertexNormals.Num(); ++VertexIdx)
					{
						VertexColors[VertexIdx] = FLinearColor(FDataflowEditorModule::SurfaceColor);
					}
				}
				const int32 NumUVLayers = DynamicMesh.Attributes()->NumUVLayers();

				const int32 GeometryIndex = RenderCollection.StartGeometryGroup(GeometryGroupName);

				if (MaterialIDs.Num() != Tris.Num() || Materials.IsEmpty() || NumUVLayers == 0)
				{
					// No materials.
					RenderCollection.AddSurface(MoveTemp(Vertices), MoveTemp(Tris), MoveTemp(VertexNormals), MoveTemp(VertexColors));
				}
				else
				{
					// GetUVs and materials.
					// The render facade currently assumes there are no internal uv seams (i.e., each vertex has one UV, instead of wedges). 
					// This will be the case for meshes that come from SKMs/SMs/Cloth
					TArray<const UE::Geometry::FDynamicMeshUVOverlay*> UVLayers;
					UVLayers.SetNumUninitialized(NumUVLayers);
					for (int32 Layer = 0; Layer < NumUVLayers; ++Layer)
					{
						UVLayers[Layer] = DynamicMesh.Attributes()->GetUVLayer(Layer);
					}

					TArray<FVector2f> DefaultUVs;
					DefaultUVs.SetNumZeroed(NumUVLayers);

					TArray<TArray<FVector2f>> UVs;
					UVs.Init(DefaultUVs, Vertices.Num()); // RenderFacade->DynamicMesh code expects all vertices to have full UV sets.

					for (const int32 TriangleIndex : DynamicMesh.TriangleIndicesItr())
					{
						const UE::Geometry::FIndex3i Tri = DynamicMesh.GetTriangle(TriangleIndex);

						for (int32 TriangleVertexIndex = 0; TriangleVertexIndex < 3; ++TriangleVertexIndex)
						{
							const int32 VertexIndex = Tri[TriangleVertexIndex];

							for (int32 Layer = 0; Layer < NumUVLayers; ++Layer)
							{
								if (UVLayers[Layer])
								{
									UVs[VertexIndex][Layer] = UVLayers[Layer]->GetElementAtVertex(TriangleIndex, VertexIndex);
								}
							}
						}
					}

					TArray<FString> MaterialPaths;
					MaterialPaths.Reserve(Materials.Num());
					for (const TObjectPtr<UMaterialInterface>& Material : Materials)
					{
						MaterialPaths.Add(Material ? Material->GetPathName() : "");
					}

					RenderCollection.AddSurface(MoveTemp(Vertices), MoveTemp(Tris), MoveTemp(VertexNormals), MoveTemp(VertexColors), MoveTemp(UVs), MoveTemp(MaterialIDs), MoveTemp(MaterialPaths));

				}

				RenderCollection.EndGeometryGroup(GeometryIndex);
			}
		}


		void RenderDynamicMeshUV(GeometryCollection::Facades::FRenderingFacade& RenderCollection,
			const FString& GeometryGroupName,
			const UE::Geometry::FDynamicMesh3& DynamicMesh,
			int32 UVChannel)
		{
			using namespace UE::Geometry;

			const int32 NumVertices = DynamicMesh.VertexCount();
			const int32 NumTriangles = DynamicMesh.TriangleCount();
			const int32 NumUVLayers = DynamicMesh.Attributes()->NumUVLayers();

			if (NumVertices > 0 && NumTriangles > 0 && NumUVLayers > UVChannel)
			{
				const FDynamicMeshUVOverlay* const UVOverlay = DynamicMesh.Attributes()->GetUVLayer(UVChannel);

				TArray<FIntVector> Tris; 
				Tris.Reserve(DynamicMesh.TriangleCount());
				for (const int32 TriangleID : DynamicMesh.TriangleIndicesItr())
				{
					const FIndex3i UVTriangle = UVOverlay->GetTriangle(TriangleID);
					Tris.Add(FIntVector(UVTriangle.A, UVTriangle.B, UVTriangle.C));
				}

				TArray<FVector3f> UVVertices; 
				UVVertices.AddZeroed(UVOverlay->ElementCount());
				for (int32 ElementID : UVOverlay->ElementIndicesItr())
				{
					UVVertices[ElementID] = FVector3f(UVOverlay->GetElement(ElementID)[0], UVOverlay->GetElement(ElementID)[1], 0.0f);
				}

				TArray<FVector3f> VertexNormals;
				VertexNormals.Init(FVector3f(0.0, 0.0, 1.0), UVVertices.Num());

				TArray<FLinearColor> VertexColors;
				VertexColors.Init(FLinearColor(0,0,0,0), UVVertices.Num());

				const int32 GeometryIndex = RenderCollection.StartGeometryGroup(GeometryGroupName);

				RenderCollection.AddSurface(MoveTemp(UVVertices), MoveTemp(Tris), MoveTemp(VertexNormals), MoveTemp(VertexColors));

				RenderCollection.EndGeometryGroup(GeometryIndex);
			}
		}

	}

	void RenderBasicGeometryCollection(GeometryCollection::Facades::FRenderingFacade& RenderCollection, const UE::Dataflow::FGraphRenderingState& State, TArray<FLinearColor>* VertexColorOverride = nullptr)
	{
		FManagedArrayCollection Default;
		FName PrimaryOutput = State.GetRenderOutputs()[0]; // "Collection"
		const FManagedArrayCollection& Collection = State.GetValue<FManagedArrayCollection>(PrimaryOutput, Default);

		const TManagedArray<int32>& BoneIndex = Collection.GetAttribute<int32>("BoneMap", FGeometryCollection::VerticesGroup);
		const TManagedArray<int32>& Parents = Collection.GetAttribute<int32>(FTransformCollection::ParentAttribute, FTransformCollection::TransformGroup);
		const TManagedArray<FTransform3f>& Transforms = Collection.GetAttribute<FTransform3f>(FTransformCollection::TransformAttribute, FTransformCollection::TransformGroup);

		TArray<FTransform> M;
		GeometryCollectionAlgo::GlobalMatrices(Transforms, Parents, M);

		// If Collection has "ExplodedVector" attribute then use it to modify the global matrices (ExplodedView node creates it)
		GeometryCollection::Facades::FCollectionExplodedVectorFacade ExplodedViewFacade(Collection);
		ExplodedViewFacade.UpdateGlobalMatricesWithExplodedVectors(M);

		auto ToD = [](FVector3f V) { return FVector3d(V.X, V.Y, V.Z); };
		auto ToF = [](FVector3d V) { return FVector3f(V.X, V.Y, V.Z); };


		const TManagedArray<FVector3f>& Vertex = Collection.GetAttribute<FVector3f>("Vertex", FGeometryCollection::VerticesGroup);
		const TManagedArray<FIntVector>& Faces = Collection.GetAttribute<FIntVector>("Indices", FGeometryCollection::FacesGroup);
		const TManagedArray<bool>* FaceVisible = Collection.FindAttribute<bool>("Visible", FGeometryCollection::FacesGroup);

		TArray<FVector3f> Vertices; Vertices.AddUninitialized(Vertex.Num());
		TArray<FIntVector> Tris; Tris.AddUninitialized(Faces.Num());
		TArray<bool> Visited; Visited.Init(false, Vertices.Num());

		int32 Tdx = 0;
		for (int32 FaceIdx = 0; FaceIdx < Faces.Num(); ++FaceIdx)
		{
			if (FaceVisible && !(*FaceVisible)[FaceIdx]) continue;

			const FIntVector& Face = Faces[FaceIdx];

			FIntVector Tri = FIntVector(Face[0], Face[1], Face[2]);
			FTransform Ms[3] = { M[BoneIndex[Tri[0]]], M[BoneIndex[Tri[1]]], M[BoneIndex[Tri[2]]] };

			Tris[Tdx++] = Tri;
			if (!Visited[Tri[0]]) Vertices[Tri[0]] = ToF(Ms[0].TransformPosition(ToD(Vertex[Tri[0]])));
			if (!Visited[Tri[1]]) Vertices[Tri[1]] = ToF(Ms[1].TransformPosition(ToD(Vertex[Tri[1]])));
			if (!Visited[Tri[2]]) Vertices[Tri[2]] = ToF(Ms[2].TransformPosition(ToD(Vertex[Tri[2]])));

			Visited[Tri[0]] = true; Visited[Tri[1]] = true; Visited[Tri[2]] = true;
		}

		Tris.SetNum(Tdx);

		// Maybe these buffers should be shrunk, but there are unused vertices in the buffer. 
		for (int i = 0; i < Visited.Num(); i++) if (!Visited[i]) Vertices[i] = FVector3f(0);

		// Copy VertexNormals from the Collection if exists otherwise compute and set it
		TArray<FVector3f> VertexNormals; VertexNormals.AddUninitialized(Vertex.Num());
		if (const TManagedArray<FVector3f>* VertexNormal = Collection.FindAttribute<FVector3f>("Normal", FGeometryCollection::VerticesGroup))
		{
			for (int32 VertexIdx = 0; VertexIdx < VertexNormals.Num(); ++VertexIdx)
			{
				VertexNormals[VertexIdx] = (*VertexNormal)[VertexIdx];
			}
		}
		else
		{
			for (int32 VertexIdx = 0; VertexIdx < VertexNormals.Num(); ++VertexIdx)
			{
				// TODO: Compute the normal
				VertexNormals[VertexIdx] = FVector3f(0.f);
			}
		}

		// Copy VertexColors from the Collection if exists otherwise set it to FDataflowEditorModule::SurfaceColor
		TArray<FLinearColor> VertexColors; VertexColors.AddUninitialized(Vertex.Num());
		if (VertexColorOverride && VertexColorOverride->Num() == Vertex.Num())
		{
			for (int32 VertexIdx = 0; VertexIdx < VertexColors.Num(); ++VertexIdx)
			{
				VertexColors[VertexIdx] = (*VertexColorOverride)[VertexIdx];
			}
		}
		else
		{
			if (const TManagedArray<FLinearColor>* VertexColorManagedArray = Collection.FindAttribute<FLinearColor>("Color", FGeometryCollection::VerticesGroup))
			{
				for (int32 VertexIdx = 0; VertexIdx < VertexColors.Num(); ++VertexIdx)
				{
					VertexColors[VertexIdx] = (*VertexColorManagedArray)[VertexIdx];
				}
			}
			else
			{
				for (int32 VertexIdx = 0; VertexIdx < VertexColors.Num(); ++VertexIdx)
				{
					VertexColors[VertexIdx] = FLinearColor(FDataflowEditorModule::SurfaceColor);
				}
			}
		}

		// Set the data on the RenderCollection
		int32 GeometryIndex = RenderCollection.StartGeometryGroup(State.GetGuid().ToString());
		RenderCollection.AddSurface(MoveTemp(Vertices), MoveTemp(Tris), MoveTemp(VertexNormals), MoveTemp(VertexColors));
		RenderCollection.EndGeometryGroup(GeometryIndex);

	}

	void RenderMeshIndexedGeometryCollection(GeometryCollection::Facades::FRenderingFacade& RenderCollection, const UE::Dataflow::FGraphRenderingState& State, TArray<FLinearColor>* VertexColorOverride = nullptr )
	{
		auto ToD = [](FVector3f V) { return FVector3d(V.X, V.Y, V.Z); };
		auto ToF = [](FVector3d V) { return FVector3f(V.X, V.Y, V.Z); };

		FManagedArrayCollection Default;
		FName PrimaryOutput = State.GetRenderOutputs()[0]; // "Collection"
		const FManagedArrayCollection& Collection = State.GetValue<FManagedArrayCollection>(PrimaryOutput, Default);

		const TManagedArray<int32>& BoneIndex = Collection.GetAttribute<int32>("BoneMap", FGeometryCollection::VerticesGroup);
		const TManagedArray<int32>& Parents = Collection.GetAttribute<int32>(FTransformCollection::ParentAttribute, FTransformCollection::TransformGroup);
		const TManagedArray<FTransform3f>& Transforms = Collection.GetAttribute<FTransform3f>(FTransformCollection::TransformAttribute, FTransformCollection::TransformGroup);
		const TManagedArray<FString>& BoneNames = Collection.GetAttribute<FString>("BoneName", FGeometryCollection::TransformGroup);
		const TManagedArray<FVector3f>& Vertex = Collection.GetAttribute<FVector3f>("Vertex", FGeometryCollection::VerticesGroup);
		const TManagedArray<FIntVector>& Faces = Collection.GetAttribute<FIntVector>("Indices", FGeometryCollection::FacesGroup);
		const TManagedArray<bool>* FaceVisible = Collection.FindAttribute<bool>("Visible", FGeometryCollection::FacesGroup);

		const TManagedArray<int32>& VertexStart = Collection.GetAttribute<int32>("VertexStart", FGeometryCollection::GeometryGroup);
		const TManagedArray<int32>& VertexCount = Collection.GetAttribute<int32>("VertexCount", FGeometryCollection::GeometryGroup);
		const TManagedArray<int32>& FacesStart = Collection.GetAttribute<int32>("FaceStart", FGeometryCollection::GeometryGroup);
		const TManagedArray<int32>& FacesCount = Collection.GetAttribute<int32>("FaceCount", FGeometryCollection::GeometryGroup);
		int32 TotalVertices = Collection.NumElements(FGeometryCollection::VerticesGroup);

		TArray<FTransform> M;
		GeometryCollectionAlgo::GlobalMatrices(Transforms, Parents, M);
		GeometryCollection::Facades::FCollectionExplodedVectorFacade ExplodedViewFacade(Collection);
		ExplodedViewFacade.UpdateGlobalMatricesWithExplodedVectors(M);

		for (int Gdx = 0; Gdx < Collection.NumElements(FGeometryCollection::GeometryGroup); Gdx++)
		{
			TArray<FVector3f> Vertices; Vertices.AddUninitialized(VertexCount[Gdx]);
			TArray<FIntVector> Tris; Tris.AddUninitialized(FacesCount[Gdx]);
			TArray<bool> Visited; Visited.Init(false, VertexCount[Gdx]);

			int32 Tdx = 0;
			int32 LastFaceIndex = FacesStart[Gdx] + FacesCount[Gdx];
			for (int32 FaceIdx = FacesStart[Gdx]; FaceIdx < LastFaceIndex; ++FaceIdx)
			{
				if (FaceVisible && !(*FaceVisible)[FaceIdx]) continue;

				const FIntVector& Face = Faces[FaceIdx];

				FIntVector Tri = FIntVector(Face[0], Face[1], Face[2]);
				FTransform Ms[3] = { M[BoneIndex[Tri[0]]], M[BoneIndex[Tri[1]]], M[BoneIndex[Tri[2]]] };
				FIntVector MovedTri = FIntVector(Face[0] - VertexStart[Gdx], Face[1] - VertexStart[Gdx], Face[2] - VertexStart[Gdx]);

				Tris[Tdx++] = MovedTri;
				if (!Visited[MovedTri[0]]) Vertices[Tri[0] - VertexStart[Gdx]] = ToF(Ms[0].TransformPosition(ToD(Vertex[Tri[0]])));
				if (!Visited[MovedTri[1]]) Vertices[Tri[1] - VertexStart[Gdx]] = ToF(Ms[1].TransformPosition(ToD(Vertex[Tri[1]])));
				if (!Visited[MovedTri[2]]) Vertices[Tri[2] - VertexStart[Gdx]] = ToF(Ms[2].TransformPosition(ToD(Vertex[Tri[2]])));

				Visited[MovedTri[0]] = true; Visited[MovedTri[1]] = true; Visited[MovedTri[2]] = true;
			}

			Tris.SetNum(Tdx);

			// move the unused points too. Need to keep them for vertex alignment with ediior tools. 
			for (int i = 0; i < Visited.Num(); i++)
			{
				if (!Visited[i])
				{
					Vertices[i] = ToF(M[BoneIndex[i + VertexStart[Gdx]]].TransformPosition(ToD(Vertex[i + VertexStart[Gdx]])));
				}
			}

			// Copy VertexNormals from the Collection if exists otherwise compute and set it
			ensure(VertexCount[Gdx] == Vertices.Num());
			TArray<FVector3f> VertexNormals; VertexNormals.AddUninitialized(Vertices.Num());
			if (const TManagedArray<FVector3f>* VertexNormal = Collection.FindAttribute<FVector3f>("Normal", FGeometryCollection::VerticesGroup))
			{
				int32 LastVertIndex = VertexStart[Gdx] + VertexCount[Gdx];
				for (int32 VertexIdx = VertexStart[Gdx], SrcVertexIdx = 0; VertexIdx < LastVertIndex; ++VertexIdx, ++SrcVertexIdx)
				{
					VertexNormals[SrcVertexIdx] = (*VertexNormal)[VertexIdx];
				}
			}
			else
			{
				for (int32 VertexIdx = 0; VertexIdx < Vertices.Num(); ++VertexIdx)
				{
					// TODO: Compute the normal
					VertexNormals[VertexIdx] = FVector3f(0.f);
				}
			}

			// Copy VertexColors from the Collection if exists otherwise set it to FDataflowEditorModule::SurfaceColor
			TArray<FLinearColor> VertexColors; VertexColors.AddUninitialized(Vertices.Num());
			if (VertexColorOverride && VertexColorOverride->Num() == TotalVertices)
			{
				int32 LastVertIndex = VertexStart[Gdx] + VertexCount[Gdx];
				for (int32 VertexIdx = VertexStart[Gdx], SrcVertexIdx = 0; VertexIdx < LastVertIndex; ++VertexIdx, ++SrcVertexIdx)
				{
					VertexColors[SrcVertexIdx] = (*VertexColorOverride)[VertexIdx];
				}
			}
			else
			{
				if (const TManagedArray<FLinearColor>* VertexColorManagedArray = Collection.FindAttribute<FLinearColor>("Color", FGeometryCollection::VerticesGroup))
				{
					int32 LastVertIndex = VertexStart[Gdx] + VertexCount[Gdx];
					for (int32 VertexIdx = VertexStart[Gdx], SrcVertexIdx = 0; VertexIdx < LastVertIndex; ++VertexIdx, ++SrcVertexIdx)
					{
						VertexColors[SrcVertexIdx] = (*VertexColorManagedArray)[VertexIdx];
					}
				}
				else
				{
					for (int32 VertexIdx = 0; VertexIdx < VertexColors.Num(); ++VertexIdx)
					{
						VertexColors[VertexIdx] = FLinearColor(FDataflowEditorModule::SurfaceColor);
					}
				}
			}

			TArray<TArray<float>> RenderBoneWeights;
			TArray<TArray<int32>> RenderBoneIndices;

			using FVertexBoneWeightsFacade = GeometryCollection::Facades::FVertexBoneWeightsFacade;

			const TManagedArray<TArray<float>>* BoneWeights = Collection.FindAttributeTyped<TArray<float>>(FVertexBoneWeightsFacade::BoneWeightsAttributeName, FGeometryCollection::VerticesGroup);
			const TManagedArray<TArray<int32>>* BoneIndices = Collection.FindAttributeTyped<TArray<int32>>(FVertexBoneWeightsFacade::BoneIndicesAttributeName, FGeometryCollection::VerticesGroup);
			if (BoneWeights && BoneIndices)
			{
				RenderBoneWeights.SetNum(Vertices.Num());
				RenderBoneIndices.SetNum(Vertices.Num());

				int32 LastVertIndex = VertexStart[Gdx] + VertexCount[Gdx];
				for (int32 VertexIdx = VertexStart[Gdx], SrcVertexIdx = 0; VertexIdx < LastVertIndex; ++VertexIdx, ++SrcVertexIdx)
				{
					RenderBoneWeights[SrcVertexIdx] = (*BoneWeights)[VertexIdx];
					RenderBoneIndices[SrcVertexIdx] = (*BoneIndices)[VertexIdx];
				}
			}

			// Set the data on the RenderCollection
			if (Vertices.Num() && Tris.Num())
			{
				FString GeometryName = State.GetGuid().ToString(); GeometryName.AppendChar('.').AppendInt(Gdx);
				if (BoneIndex[VertexStart[Gdx]] != INDEX_NONE)
				{
					GeometryName = BoneNames[BoneIndex[VertexStart[Gdx]]];
				}
				int32 GeometryIndex = RenderCollection.StartGeometryGroup(GeometryName);
				RenderCollection.AddSurface(MoveTemp(Vertices), MoveTemp(Tris), MoveTemp(VertexNormals), MoveTemp(VertexColors));
				if (BoneWeights && BoneIndices)
				{
					RenderCollection.AddSurfaceBoneWeightsAndIndices(MoveTemp(RenderBoneWeights), MoveTemp(RenderBoneIndices));
				}
				RenderCollection.EndGeometryGroup(GeometryIndex);
			}
		}
	}

	void RenderGeometryCollectionUV(GeometryCollection::Facades::FRenderingFacade& RenderCollection, const UE::Dataflow::FGraphRenderingState& State, TArray<FLinearColor>* VertexColorOverride = nullptr)
	{
		const TArray<FName>& RenderOutputs = State.GetRenderOutputs();
		if (RenderOutputs.Num() == 0)
		{
			return; // no outputs
		}
		const FName PrimaryOutput = RenderOutputs[0]; // "Collection"
		const FName UVChannelOutput = (RenderOutputs.Num() > 1)? RenderOutputs[1]: NAME_None; // "UVChannel"

		const FManagedArrayCollection Default;
		const FManagedArrayCollection& Collection = State.GetValue<FManagedArrayCollection>(PrimaryOutput, Default);

		const int32 DefaultUVChannel = 0;
		int32 UVChannel = (UVChannelOutput.IsNone())? DefaultUVChannel: State.GetValue<int32>(UVChannelOutput, DefaultUVChannel);

		const GeometryCollection::Facades::FCollectionUVFacade UVFacade(Collection);
		if (!UVFacade.IsValid())
		{
			return; // no UV data 
		}
		if (UVFacade.FindUVLayer(UVChannel) == nullptr)
		{
			UVChannel = 0;
		}

		const GeometryCollection::Facades::FCollectionMeshFacade MeshFacade(Collection);
		if (!MeshFacade.IndicesAttribute.IsValid())
		{
			return; // no face nothing to render
		}

		TArray<FIntVector> Triangles;
		Triangles.Reserve(MeshFacade.IndicesAttribute.Num());
		for (int32 TriangleIdx = 0; TriangleIdx < MeshFacade.IndicesAttribute.Num(); ++TriangleIdx)
		{
			const bool IsVisible = (MeshFacade.VisibleAttribute.IsValid()) ? MeshFacade.VisibleAttribute.Get()[TriangleIdx] : true;
			if (IsVisible)
			{
				Triangles.Add(MeshFacade.IndicesAttribute[TriangleIdx]);
			}
		}

		const TManagedArray<FVector2f>& UVs = UVFacade.GetUVLayer(UVChannel);
		TArray<FVector3f> UVVertices;
		UVVertices.Reserve(UVs.Num());
		for (int32 UvIdx = 0; UvIdx < UVs.Num(); ++UvIdx)
		{
			const FVector2f& Uv = UVs[UvIdx];
			UVVertices.Add(FVector3f(Uv.X, Uv.Y, 0.f));
		}

		TArray<FVector3f> VertexNormals;
		VertexNormals.Init(FVector3f(0.0, 0.0, 1.0), UVVertices.Num());

		TArray<FLinearColor> VertexColors;
		VertexColors.Init(FLinearColor(0, 0, 0, 0), UVVertices.Num());

		const int32 GeometryIndex = RenderCollection.StartGeometryGroup(FGeometryCollection::GeometryGroup.ToString());
		RenderCollection.AddSurface(MoveTemp(UVVertices), MoveTemp(Triangles), MoveTemp(VertexNormals), MoveTemp(VertexColors));
		RenderCollection.EndGeometryGroup(GeometryIndex);
	}

	class FGeometryCollectionSurfaceRenderCallbacks : public FRenderingFactory::ICallbackInterface
	{
		virtual UE::Dataflow::FRenderKey GetRenderKey() const override
		{
			return { "SurfaceRender", FGeometryCollection::StaticType() };
		}

		virtual bool CanRender(const UE::Dataflow::IDataflowConstructionViewMode& ViewMode) const override
		{
			return (ViewMode.GetName() == FDataflowConstruction3DViewMode::Name || ViewMode.GetName() == FDataflowConstructionUVViewMode::Name);
		}

		virtual void Render(GeometryCollection::Facades::FRenderingFacade& RenderCollection, const FGraphRenderingState& State) override
		{
			if (State.GetRenderOutputs().Num())
			{
				FManagedArrayCollection Default;
				FName PrimaryOutput = State.GetRenderOutputs()[0]; // "Collection"
				const FManagedArrayCollection& Collection = State.GetValue<FManagedArrayCollection>(PrimaryOutput, Default);

				const bool bFoundIndices = Collection.FindAttributeTyped<FIntVector>("Indices", FGeometryCollection::FacesGroup) != nullptr;
				const bool bFoundVertices = Collection.FindAttributeTyped<FVector3f>("Vertex", FGeometryCollection::VerticesGroup) != nullptr;
				const bool bFoundTransforms = Collection.FindAttributeTyped<FTransform3f>(FTransformCollection::TransformAttribute, FTransformCollection::TransformGroup) != nullptr;
				const bool bFoundBoneMap = Collection.FindAttributeTyped<int32>("BoneMap", FGeometryCollection::VerticesGroup) != nullptr;
				const bool bFoundParents = Collection.FindAttributeTyped<int32>(FTransformCollection::ParentAttribute, FTransformCollection::TransformGroup) != nullptr;
				bool bFoundRenderData = bFoundIndices && bFoundVertices && bFoundTransforms && bFoundBoneMap && bFoundParents
					&& Collection.NumElements(FTransformCollection::TransformGroup) > 0;

				const bool bFoundVertexStart = Collection.FindAttributeTyped<int32>("VertexStart", FGeometryCollection::GeometryGroup) != nullptr;
				const bool bFoundVertexCount = Collection.FindAttributeTyped<int32>("VertexCount", FGeometryCollection::GeometryGroup) != nullptr;
				const bool bFoundFaceStart = Collection.FindAttributeTyped<int32>("FaceStart", FGeometryCollection::GeometryGroup) != nullptr;
				const bool bFoundFaceCount = Collection.FindAttributeTyped<int32>("FaceCount", FGeometryCollection::GeometryGroup) != nullptr;
				bool bFoundGeometryAttributes = bFoundVertexStart && bFoundVertexCount && bFoundFaceStart && bFoundFaceCount
					&& Collection.NumElements(FGeometryCollection::GeometryGroup) > 0;

				if (State.GetViewMode().GetName() == Dataflow::FDataflowConstruction3DViewMode::Name)
				{
					if (bFoundRenderData && bFoundGeometryAttributes)
					{
						RenderMeshIndexedGeometryCollection(RenderCollection, State);
					}
					else if (bFoundRenderData)
					{
						RenderBasicGeometryCollection(RenderCollection, State);
					}
				}
				else if (State.GetViewMode().GetName() == Dataflow::FDataflowConstructionUVViewMode::Name)
				{
					RenderGeometryCollectionUV(RenderCollection, State);
				}
			}
		}
	};


	class FGeometryCollectionSurfaceWeightsRenderCallbacks : public FRenderingFactory::ICallbackInterface
	{
		virtual UE::Dataflow::FRenderKey GetRenderKey() const override
		{
			return { "SurfaceWeightsRender", FGeometryCollection::StaticType() };
		}

		virtual bool CanRender(const UE::Dataflow::IDataflowConstructionViewMode& ViewMode) const override
		{
			return (ViewMode.GetName() == FDataflowConstruction3DViewMode::Name);
		}

		virtual void Render(GeometryCollection::Facades::FRenderingFacade& RenderCollection, const FGraphRenderingState& State) override
		{
			if (State.GetRenderOutputs().Num() >= 2)
			{
				FManagedArrayCollection Default;
				FName PrimaryOutput = State.GetRenderOutputs()[0]; // "Collection"
				const FManagedArrayCollection& Collection = State.GetValue<FManagedArrayCollection>(PrimaryOutput, Default);

				const bool bFoundIndices = Collection.FindAttributeTyped<FIntVector>("Indices", FGeometryCollection::FacesGroup) != nullptr;
				const bool bFoundVertices = Collection.FindAttributeTyped<FVector3f>("Vertex", FGeometryCollection::VerticesGroup) != nullptr;
				const bool bFoundTransforms = Collection.FindAttributeTyped<FTransform3f>(FTransformCollection::TransformAttribute, FTransformCollection::TransformGroup) != nullptr;
				const bool bFoundBoneMap = Collection.FindAttributeTyped<int32>("BoneMap", FGeometryCollection::VerticesGroup) != nullptr;
				const bool bFoundParents = Collection.FindAttributeTyped<int32>(FTransformCollection::ParentAttribute, FTransformCollection::TransformGroup) != nullptr;
				bool bFoundRenderData = bFoundIndices && bFoundVertices && bFoundTransforms && bFoundBoneMap && bFoundParents
					&& Collection.NumElements(FTransformCollection::TransformGroup) > 0;

				const bool bFoundVertexStart = Collection.FindAttributeTyped<int32>("VertexStart", FGeometryCollection::GeometryGroup) != nullptr;
				const bool bFoundVertexCount = Collection.FindAttributeTyped<int32>("VertexCount", FGeometryCollection::GeometryGroup) != nullptr;
				const bool bFoundFaceStart = Collection.FindAttributeTyped<int32>("FaceStart", FGeometryCollection::GeometryGroup) != nullptr;
				const bool bFoundFaceCount = Collection.FindAttributeTyped<int32>("FaceCount", FGeometryCollection::GeometryGroup) != nullptr;
				bool bFoundGeometryAttributes = bFoundVertexStart && bFoundVertexCount && bFoundFaceStart && bFoundFaceCount
					&& Collection.NumElements(FGeometryCollection::GeometryGroup) > 0;

				FCollectionAttributeKey DefaultKey;
				FName SecondaryOutput = State.GetRenderOutputs()[1]; // "AttributeKey"
				const FCollectionAttributeKey& AttributeKey = State.GetValue<FCollectionAttributeKey>(SecondaryOutput, DefaultKey);

				const bool bFoundVertexColor = Collection.FindAttributeTyped<FLinearColor>("Color", FGeometryCollection::VerticesGroup) != nullptr;
				const bool bFoundFloatScalar = Collection.FindAttributeTyped<float>(FName(AttributeKey.Attribute), FName(AttributeKey.Group)) != nullptr;
				bool bFoundVertexScalarAndColors = bFoundVertexColor && bFoundFloatScalar && AttributeKey.Group.Equals(FGeometryCollection::VerticesGroup.ToString());

				TArray<FLinearColor>* Colors = nullptr;
				if (bFoundVertexScalarAndColors)
				{
					auto RangeValue = [](const TManagedArray<float>* FloatArray)
						{
							float Min = FLT_MAX;
							float Max = -FLT_MAX;
							for (int i = 0; i < FloatArray->Num(); i++) {
								Min = FMath::Min(Min, (*FloatArray)[i]);
								Max = FMath::Max(Max, (*FloatArray)[i]);
							}
							return TPair<float, float>(Min, Max);
						};

					const TManagedArray<float>* FloatArray = Collection.FindAttributeTyped<float>(FName(AttributeKey.Attribute), FName(AttributeKey.Group));
					if (FloatArray && FloatArray->Num())
					{
						Colors = new TArray<FLinearColor>();
						Colors->AddUninitialized(FloatArray->Num());

						TPair<float, float> Range = RangeValue(FloatArray);
						float Delta = FMath::Abs(Range.Get<1>() - Range.Get<0>());
						if (Delta > FLT_EPSILON)
						{
							for (int32 VertexIdx = 0; VertexIdx < FloatArray->Num(); ++VertexIdx)
							{
								(*Colors)[VertexIdx] = FLinearColor::White * ((*FloatArray)[VertexIdx] - Range.Get<0>()) / Delta;
							}
						}
						else
						{
							for (int32 VertexIdx = 0; VertexIdx < FloatArray->Num(); ++VertexIdx)
							{
								(*Colors)[VertexIdx] = FLinearColor::Black;
							}
						}
					}
				}

				if (bFoundRenderData && bFoundGeometryAttributes)
				{
					RenderMeshIndexedGeometryCollection(RenderCollection, State, Colors);
				}
				else if (bFoundRenderData)
				{
					RenderBasicGeometryCollection(RenderCollection, State, Colors);
				}

				if (Colors)
				{
					delete Colors;
				}
			}
		}
	};

	class FDynamicMesh3SurfaceRenderCallbacks : public FRenderingFactory::ICallbackInterface
	{
		virtual UE::Dataflow::FRenderKey GetRenderKey() const override
		{
			return { "SurfaceRender", FName("FDynamicMesh3") };
		}

		virtual bool CanRender(const UE::Dataflow::IDataflowConstructionViewMode& ViewMode) const override
		{
			return (ViewMode.GetName() == FDataflowConstruction3DViewMode::Name);
		}

		virtual void Render(GeometryCollection::Facades::FRenderingFacade& RenderCollection, const FGraphRenderingState& State) override
		{
			if (State.GetRenderOutputs().Num())
			{
				FName PrimaryOutput = State.GetRenderOutputs()[0]; // "Mesh"

				TObjectPtr<UDynamicMesh> Default;
				if (const TObjectPtr<UDynamicMesh> Mesh = State.GetValue<TObjectPtr<UDynamicMesh>>(PrimaryOutput, Default))
				{
					const UE::Geometry::FDynamicMesh3& DynamicMesh = Mesh->GetMeshRef();

					TArray<TObjectPtr<UMaterialInterface>> Materials;
					if (State.GetRenderOutputs().IsValidIndex(1))
					{
						TArray<TObjectPtr<UMaterialInterface>> MaterialsDefault;
						Materials = State.GetValue<TArray<TObjectPtr<UMaterialInterface>>>(State.GetRenderOutputs()[1], MaterialsDefault);
					}

					Private::RenderDynamicMesh(RenderCollection, State.GetGuid().ToString(), DynamicMesh, Materials);
				}
			}
		}
	};


	class FDataflowMeshSurfaceRenderCallbacks : public FRenderingFactory::ICallbackInterface
	{
		virtual UE::Dataflow::FRenderKey GetRenderKey() const override
		{
			return { "SurfaceRender", FName("UDataflowMesh") };
		}

		virtual bool CanRender(const UE::Dataflow::IDataflowConstructionViewMode& ViewMode) const override
		{
			return (ViewMode.GetName() == FDataflowConstruction3DViewMode::Name || ViewMode.GetName() == FDataflowConstructionUVViewMode::Name);
		}

		virtual void Render(GeometryCollection::Facades::FRenderingFacade& RenderCollection, const FGraphRenderingState& State) override
		{
			if (State.GetRenderOutputs().Num())
			{
				const TArray<FName>& RenderOutputs = State.GetRenderOutputs();
				if (RenderOutputs.Num() == 0)
				{
					return; // no outputs
				}
				FName PrimaryOutput = State.GetRenderOutputs()[0];

				TObjectPtr<UDataflowMesh> Default;
				if (const TObjectPtr<UDataflowMesh> Mesh = State.GetValue(PrimaryOutput, Default))
				{
					if (const UE::Geometry::FDynamicMesh3* const DynamicMesh = Mesh->GetDynamicMesh())
					{
						if (State.GetViewMode().GetName() == Dataflow::FDataflowConstruction3DViewMode::Name)
						{
							Private::RenderDynamicMesh(RenderCollection, State.GetGuid().ToString(), *DynamicMesh, Mesh->GetMaterials());
						}
						else if (State.GetViewMode().GetName() == Dataflow::FDataflowConstructionUVViewMode::Name)
						{
							if (DynamicMesh->HasAttributes())
							{
								const FName UVChannelOutput = (RenderOutputs.Num() > 1) ? RenderOutputs[1] : NAME_None; // "UVChannel"
								const int32 DefaultUVChannel = 0;
								const int32 UVChannel = (UVChannelOutput.IsNone()) ? DefaultUVChannel : State.GetValue<int32>(UVChannelOutput, DefaultUVChannel);

								if (DynamicMesh->Attributes()->NumUVLayers() > UVChannel)
								{
									Private::RenderDynamicMeshUV(RenderCollection, State.GetGuid().ToString(), *DynamicMesh, UVChannel);
								}
							}
						}
					}
				}
			}
		}
	};

	class FBoxSurfaceRenderCallbacks : public FRenderingFactory::ICallbackInterface
	{
		virtual UE::Dataflow::FRenderKey GetRenderKey() const override
		{
			return { "SurfaceRender", FName("FBox") };
		}

		virtual bool CanRender(const UE::Dataflow::IDataflowConstructionViewMode& ViewMode) const override
		{
			return (ViewMode.GetName() == FDataflowConstruction3DViewMode::Name);
		}

		virtual void Render(GeometryCollection::Facades::FRenderingFacade& RenderCollection, const FGraphRenderingState& State) override
		{
			if (State.GetRenderOutputs().Num())
			{
				FName PrimaryOutput = State.GetRenderOutputs()[0]; // "Box"

				FBox Default(ForceInit);
				const FBox& Box = State.GetValue<FBox>(PrimaryOutput, Default);

				int32 GeometryIndex = RenderCollection.StartGeometryGroup(State.GetGuid().ToString());
				RenderCollection.AddBox(FVector3f(Box.Min), FVector3f(Box.Max));
				RenderCollection.EndGeometryGroup(GeometryIndex);
			}
		}
	};

	class FBoxesSurfaceRenderCallbacks : public FRenderingFactory::ICallbackInterface
	{
		virtual UE::Dataflow::FRenderKey GetRenderKey() const override
		{
			return { "BoxesRender", FName("TArray<FBox>") };
		}

		virtual bool CanRender(const UE::Dataflow::IDataflowConstructionViewMode& ViewMode) const override
		{
			return (ViewMode.GetName() == FDataflowConstruction3DViewMode::Name);
		}

		virtual void Render(GeometryCollection::Facades::FRenderingFacade& RenderCollection, const FGraphRenderingState& State) override
		{
			if (State.GetRenderOutputs().Num())
			{
				FName PrimaryOutput = State.GetRenderOutputs()[0]; // "Box"

				TArray<FBox> Default;
				const TArray<FBox>& Boxes = State.GetValue<TArray<FBox>>(PrimaryOutput, Default);

				int32 GeometryIndex = RenderCollection.StartGeometryGroup(State.GetGuid().ToString());
				RenderCollection.AddBoxes(Boxes);
				RenderCollection.EndGeometryGroup(GeometryIndex);
			}
		}
	};

	class FSphereSurfaceRenderCallbacks : public FRenderingFactory::ICallbackInterface
	{
		virtual UE::Dataflow::FRenderKey GetRenderKey() const override
		{
			return { "SurfaceRender", FName("FSphere") };
		}

		virtual bool CanRender(const UE::Dataflow::IDataflowConstructionViewMode& ViewMode) const override
		{
			return (ViewMode.GetName() == FDataflowConstruction3DViewMode::Name);
		}

		virtual void Render(GeometryCollection::Facades::FRenderingFacade& RenderCollection, const FGraphRenderingState& State) override
		{
			if (State.GetRenderOutputs().Num())
			{
				FName PrimaryOutput = State.GetRenderOutputs()[0]; // "Sphere"

				FSphere Default(ForceInit);
				const FSphere& Sphere = State.GetValue<FSphere>(PrimaryOutput, Default);

				int32 GeometryIndex = RenderCollection.StartGeometryGroup(State.GetGuid().ToString());
				RenderCollection.AddSphere(FVector3f(Sphere.Center), float(Sphere.W), FLinearColor(FDataflowEditorModule::SurfaceColor));
				RenderCollection.EndGeometryGroup(GeometryIndex);
			}
		}
	};

	class FSpheresSurfaceRenderCallbacks : public FRenderingFactory::ICallbackInterface
	{
		virtual UE::Dataflow::FRenderKey GetRenderKey() const override
		{
			return { "SpheresRender", FName("TArray<FSphere>") };
		}

		virtual bool CanRender(const UE::Dataflow::IDataflowConstructionViewMode& ViewMode) const override
		{
			return (ViewMode.GetName() == FDataflowConstruction3DViewMode::Name);
		}

		virtual void Render(GeometryCollection::Facades::FRenderingFacade& RenderCollection, const FGraphRenderingState& State) override
		{
			if (State.GetRenderOutputs().Num())
			{
				FName PrimaryOutput = State.GetRenderOutputs()[0]; // "Spheres"

				TArray<FSphere> Default;
				const TArray<FSphere>& Spheres = State.GetValue<TArray<FSphere>>(PrimaryOutput, Default);

				int32 GeometryIndex = RenderCollection.StartGeometryGroup(State.GetGuid().ToString());
				RenderCollection.AddSpheres(Spheres, FLinearColor(FDataflowEditorModule::SurfaceColor));
				RenderCollection.EndGeometryGroup(GeometryIndex);
			}
		}
	};

	class FPointRenderCallbacks : public FRenderingFactory::ICallbackInterface
	{
		virtual UE::Dataflow::FRenderKey GetRenderKey() const override
		{
			return { "PointRender", FName("FVector") };
		}

		virtual bool CanRender(const UE::Dataflow::IDataflowConstructionViewMode& ViewMode) const override
		{
			return (ViewMode.GetName() == FDataflowConstruction3DViewMode::Name);
		}

		virtual void Render(GeometryCollection::Facades::FRenderingFacade& RenderCollection, const FGraphRenderingState& State) override
		{
			if (State.GetRenderOutputs().Num())
			{
				FName PrimaryOutput = State.GetRenderOutputs()[0]; // "Point"

				FVector Default;
				const FVector& Point = State.GetValue<FVector>(PrimaryOutput, Default);



				int32 GeometryIndex = RenderCollection.StartGeometryGroup(State.GetGuid().ToString());
				RenderCollection.AddPoint(FVector3f(Point));
				RenderCollection.EndGeometryGroup(GeometryIndex);
			}
		}
	};

	class FPointsRenderCallbacks : public FRenderingFactory::ICallbackInterface
	{
		virtual UE::Dataflow::FRenderKey GetRenderKey() const override
		{
			return { "PointsRender", FName("TArray<FVector>") };
		}

		virtual bool CanRender(const UE::Dataflow::IDataflowConstructionViewMode& ViewMode) const override
		{
			return (ViewMode.GetName() == FDataflowConstruction3DViewMode::Name);
		}

		virtual void Render(GeometryCollection::Facades::FRenderingFacade& RenderCollection, const FGraphRenderingState& State) override
		{
			if (State.GetRenderOutputs().Num())
			{
				FName PrimaryOutput = State.GetRenderOutputs()[0]; // "Points"

				TArray<FVector> Default;
				const TArray<FVector>& Points = State.GetValue<TArray<FVector>>(PrimaryOutput, Default);

				int32 NumPointsToRender = Points.Num();
				if (UE::Dataflow::Private::DataflowPointRenderLimit > -1 && NumPointsToRender > UE::Dataflow::Private::DataflowPointRenderLimit)
				{
					UE_LOGF(LogChaosDataflow, Warning, "Limited the number of points rendered from %d to %d; to see all points, adjust CVAR: p.Dataflow.PointRenderLimit",
						NumPointsToRender, UE::Dataflow::Private::DataflowPointRenderLimit);
					NumPointsToRender = UE::Dataflow::Private::DataflowPointRenderLimit;
				}

				TArray<FVector3f> PointsArr; PointsArr.AddUninitialized(NumPointsToRender);
				for (int32 Idx = 0; Idx < NumPointsToRender; ++Idx)
				{
					PointsArr[Idx] = (FVector3f)Points[Idx];
				}

				int32 GeometryIndex = RenderCollection.StartGeometryGroup(State.GetGuid().ToString());
				RenderCollection.AddPoints(MoveTemp(PointsArr));
				RenderCollection.EndGeometryGroup(GeometryIndex);
			}
		}
	};

	void RenderTetrahedronGeometryCollection(GeometryCollection::Facades::FRenderingFacade& RenderCollection, const UE::Dataflow::FGraphRenderingState& State, TArray<FLinearColor>* VertexColorOverride = nullptr)
	{
		auto ToD = [](FVector3f V) { return FVector3d(V.X, V.Y, V.Z); };
		auto ToF = [](FVector3d V) { return FVector3f(V.X, V.Y, V.Z); };

		FManagedArrayCollection Default;
		FName PrimaryOutput = State.GetRenderOutputs()[0]; // "Collection"
		const FManagedArrayCollection& Collection = State.GetValue<FManagedArrayCollection>(PrimaryOutput, Default);

		const TManagedArray<int32>& BoneIndex = Collection.GetAttribute<int32>("BoneMap", FGeometryCollection::VerticesGroup);
		const TManagedArray<int32>& Parents = Collection.GetAttribute<int32>(FTransformCollection::ParentAttribute, FTransformCollection::TransformGroup);
		const TManagedArray<FTransform3f>& Transforms = Collection.GetAttribute<FTransform3f>(FTransformCollection::TransformAttribute, FTransformCollection::TransformGroup);
		const TManagedArray<FString>& BoneNames = Collection.GetAttribute<FString>("BoneName", FGeometryCollection::TransformGroup);
		const TManagedArray<FVector3f>& Vertex = Collection.GetAttribute<FVector3f>("Vertex", FGeometryCollection::VerticesGroup);
		const TManagedArray<int32>& VertexStart = Collection.GetAttribute<int32>("VertexStart", FGeometryCollection::GeometryGroup);
		const TManagedArray<int32>& VertexCount = Collection.GetAttribute<int32>("VertexCount", FGeometryCollection::GeometryGroup);
		const TManagedArray<int32>& TetrahedronStart = Collection.GetAttribute<int32>("TetrahedronStart", FGeometryCollection::GeometryGroup);
		const TManagedArray<int32>& TetrahedronCount = Collection.GetAttribute<int32>("TetrahedronCount", FGeometryCollection::GeometryGroup);
		const TManagedArray<FIntVector4>& Tetrahedrons = Collection.GetAttribute<FIntVector4>("Tetrahedron", "Tetrahedral");
		const TManagedArray<int32>& GeometryToTransformIndex = Collection.GetAttribute<int32>("TransformIndex", FGeometryCollection::GeometryGroup);

		TArray<FTransform> RootSpaceTransforms;
		GeometryCollectionAlgo::GlobalMatrices(Transforms, Parents, RootSpaceTransforms);

		for (int GeometryIdx = 0; GeometryIdx < Collection.NumElements(FGeometryCollection::GeometryGroup); GeometryIdx++)
		{
			TArray<FVector3f> VerticesInCollectionSpace; VerticesInCollectionSpace.AddUninitialized(VertexCount[GeometryIdx]);
			TArray<FVector3f> SplitVertices;

			const int32 TransformIndex = GeometryToTransformIndex[GeometryIdx];

			// Transform vertices to Collection space
			const int32 GlobalVertexOffset = VertexStart[GeometryIdx];
			for (int32 LocalVertexIdx = 0; LocalVertexIdx < VertexCount[GeometryIdx]; ++LocalVertexIdx)
			{
				VerticesInCollectionSpace[LocalVertexIdx] = ToF(RootSpaceTransforms[TransformIndex].TransformPosition(ToD(Vertex[GlobalVertexOffset + LocalVertexIdx])));
			}

			TArray<FIntVector4> Tetras; Tetras.AddUninitialized(TetrahedronCount[GeometryIdx]);

			const int32 GlobalTetrahedronOffset = TetrahedronStart[GeometryIdx];
			for (int32 LocalTetrahedronIdx = 0; LocalTetrahedronIdx < TetrahedronCount[GeometryIdx]; ++LocalTetrahedronIdx)
			{
				const FIntVector4& Tetra = Tetrahedrons[GlobalTetrahedronOffset + LocalTetrahedronIdx];
				const int32 VtxStart = SplitVertices.Num();

				SplitVertices.Add(VerticesInCollectionSpace[Tetra[0] - GlobalVertexOffset]);
				SplitVertices.Add(VerticesInCollectionSpace[Tetra[1] - GlobalVertexOffset]);
				SplitVertices.Add(VerticesInCollectionSpace[Tetra[2] - GlobalVertexOffset]);
				SplitVertices.Add(VerticesInCollectionSpace[Tetra[3] - GlobalVertexOffset]);

				Tetras[LocalTetrahedronIdx] = { VtxStart, VtxStart + 1, VtxStart + 2, VtxStart + 3 };
			}

			if (SplitVertices.Num() && Tetras.Num())
			{
				FString GeometryName = State.GetGuid().ToString(); GeometryName.AppendChar('.').AppendInt(GeometryIdx);
				if (BoneIndex[VertexStart[GeometryIdx]] != INDEX_NONE)
				{
					GeometryName = BoneNames[BoneIndex[VertexStart[GeometryIdx]]];
				}
				int32 GeometryIndex = RenderCollection.StartGeometryGroup(GeometryName);
				RenderCollection.AddTetrahedrons(MoveTemp(SplitVertices), MoveTemp(Tetras));
				RenderCollection.EndGeometryGroup(GeometryIndex);
			}
		}
	}

	class FGeometryCollectionTetrahedronRenderCallbacks : public FRenderingFactory::ICallbackInterface
	{
		virtual UE::Dataflow::FRenderKey GetRenderKey() const override
		{
			return { "TetrahedronRender", FGeometryCollection::StaticType() };
		}

		virtual bool CanRender(const UE::Dataflow::IDataflowConstructionViewMode& ViewMode) const override
		{
			return (ViewMode.GetName() == FDataflowConstruction3DViewMode::Name);
		}

		virtual void Render(GeometryCollection::Facades::FRenderingFacade& RenderCollection, const FGraphRenderingState& State) override
		{
			if (State.GetRenderOutputs().Num())
			{
				FManagedArrayCollection Default;
				FName PrimaryOutput = State.GetRenderOutputs()[0]; // "Collection"
				const FManagedArrayCollection& Collection = State.GetValue<FManagedArrayCollection>(PrimaryOutput, Default);

				const bool bFoundIndices = Collection.FindAttributeTyped<FIntVector>("Indices", FGeometryCollection::FacesGroup) != nullptr;
				const bool bFoundVertices = Collection.FindAttributeTyped<FVector3f>("Vertex", FGeometryCollection::VerticesGroup) != nullptr;
				const bool bFoundTransforms = Collection.FindAttributeTyped<FTransform3f>(FTransformCollection::TransformAttribute, FTransformCollection::TransformGroup) != nullptr;
				const bool bFoundBoneMap = Collection.FindAttributeTyped<int32>("BoneMap", FGeometryCollection::VerticesGroup) != nullptr;
				const bool bFoundBoneNames = Collection.FindAttributeTyped<FString>("BoneName", FGeometryCollection::TransformGroup) != nullptr;
				const bool bFoundParents = Collection.FindAttributeTyped<int32>(FTransformCollection::ParentAttribute, FTransformCollection::TransformGroup) != nullptr;
				bool bFoundRenderData = bFoundIndices && bFoundVertices && bFoundTransforms && bFoundBoneMap && bFoundBoneNames && bFoundParents
					&& Collection.NumElements(FTransformCollection::TransformGroup) > 0;

				const bool bFoundVertexStart = Collection.FindAttributeTyped<int32>("VertexStart", FGeometryCollection::GeometryGroup) != nullptr;
				const bool bFoundVertexCount = Collection.FindAttributeTyped<int32>("VertexCount", FGeometryCollection::GeometryGroup) != nullptr;
				const bool bFoundTetrahedronStart = Collection.FindAttributeTyped<int32>("TetrahedronStart", FGeometryCollection::GeometryGroup) != nullptr;
				const bool bFoundTetrahedronCount = Collection.FindAttributeTyped<int32>("TetrahedronCount", FGeometryCollection::GeometryGroup) != nullptr;
				const bool bFoundTetrahedrons = Collection.FindAttributeTyped<FIntVector4>("Tetrahedron", "Tetrahedral") != nullptr;
				bool bFoundGeometryAttributes = bFoundVertexStart && bFoundVertexCount && bFoundTetrahedronStart && bFoundTetrahedronCount && bFoundTetrahedrons
					&& Collection.NumElements(FGeometryCollection::GeometryGroup) > 0;

				if (bFoundRenderData && bFoundGeometryAttributes)
				{
					RenderTetrahedronGeometryCollection(RenderCollection, State);
				}
			}
		}
	};

	class FFieldVolumeRenderCallbacks : public FRenderingFactory::ICallbackInterface
	{
		virtual UE::Dataflow::FRenderKey GetRenderKey() const override
		{
			return { "VolumeRender", FFieldCollection::StaticType() };
		}

		virtual bool CanRender(const UE::Dataflow::IDataflowConstructionViewMode& ViewMode) const override
		{
			return (ViewMode.GetName() == FDataflowConstruction3DViewMode::Name);
		}

		virtual void Render(GeometryCollection::Facades::FRenderingFacade& RenderCollection, const FGraphRenderingState& State) override
		{
			if (State.GetRenderOutputs().Num())
			{
				FName PrimaryOutput = State.GetRenderOutputs()[0]; // "VectorField"
				FFieldCollection Default;
				const FFieldCollection& Collection = State.GetValue<FFieldCollection>(PrimaryOutput, Default);
				TArray<TPair<FVector3f, FVector3f>> VectorField = Collection.GetVectorField();
				TArray<FLinearColor> VertexColors = Collection.GetVectorColor();
				const int32 NumVertices = 3 * VectorField.Num();
				const int32 NumTriangles = VectorField.Num();

				TArray<FVector3f> Vertices; Vertices.AddUninitialized(NumVertices);
				TArray<FIntVector> Tris; Tris.AddUninitialized(NumTriangles);
				TArray<FVector3f> VertexNormals; VertexNormals.AddUninitialized(NumVertices);

				for (int32 i = 0; i < VectorField.Num(); i++)
				{
							
					FVector3f Dir = VectorField[i].Value - VectorField[i].Key;
					FVector3f OrthogonalDir;
					if (Dir[1] < UE_SMALL_NUMBER && Dir[2] < UE_SMALL_NUMBER && Dir[0] > UE_SMALL_NUMBER)
					{
						OrthogonalDir = FVector3f(0, 0, 1);
					}
					else
					{
						FVector3f DirAdd = Dir;
						DirAdd.X += 1.f;
						OrthogonalDir = (Dir ^ DirAdd).GetSafeNormal();
					}
					Tris[i] = FIntVector(3*i, 3*i+1, 3*i+2);
					Vertices[3*i] = VectorField[i].Key;
					Vertices[3*i+1] = VectorField[i].Value;
					Vertices[3*i+2] = VectorField[i].Key + float(0.1) * Dir.Size() * OrthogonalDir;
					FVector3f TriangleNormal = (OrthogonalDir ^ Dir).GetSafeNormal();
					VertexNormals[3*i] = TriangleNormal;
					VertexNormals[3*i+1] = TriangleNormal;
					VertexNormals[3*i+2] = TriangleNormal;
				}

				int32 GeometryIndex = RenderCollection.StartGeometryGroup(State.GetGuid().ToString());
				RenderCollection.AddSurface(MoveTemp(Vertices), MoveTemp(Tris), MoveTemp(VertexNormals), MoveTemp(VertexColors));
				RenderCollection.EndGeometryGroup(GeometryIndex);
			}
		}
	};

	void RenderingCallbacks()
	{
		using namespace UE::Dataflow;

		FRenderingFactory::GetInstance()->RegisterCallbacks(MakeUnique<FGeometryCollectionSurfaceRenderCallbacks>());
		FRenderingFactory::GetInstance()->RegisterCallbacks(MakeUnique<FGeometryCollectionSurfaceWeightsRenderCallbacks>());
		FRenderingFactory::GetInstance()->RegisterCallbacks(MakeUnique<FDynamicMesh3SurfaceRenderCallbacks>());
		FRenderingFactory::GetInstance()->RegisterCallbacks(MakeUnique<FDataflowMeshSurfaceRenderCallbacks>());
		FRenderingFactory::GetInstance()->RegisterCallbacks(MakeUnique<FBoxSurfaceRenderCallbacks>());
		FRenderingFactory::GetInstance()->RegisterCallbacks(MakeUnique<FBoxesSurfaceRenderCallbacks>());
		FRenderingFactory::GetInstance()->RegisterCallbacks(MakeUnique<FSphereSurfaceRenderCallbacks>());
		FRenderingFactory::GetInstance()->RegisterCallbacks(MakeUnique<FSpheresSurfaceRenderCallbacks>());
		FRenderingFactory::GetInstance()->RegisterCallbacks(MakeUnique<FPointRenderCallbacks>());
		FRenderingFactory::GetInstance()->RegisterCallbacks(MakeUnique<FPointsRenderCallbacks>());
		FRenderingFactory::GetInstance()->RegisterCallbacks(MakeUnique<FGeometryCollectionTetrahedronRenderCallbacks>());
		FRenderingFactory::GetInstance()->RegisterCallbacks(MakeUnique<FFieldVolumeRenderCallbacks>());
	}
}
