// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowFacesOutlinerWidget.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Dataflow/DataflowSettings.h"
#include "GeometryCollection/GeometryCollection.h"
#include "Dataflow/DataflowCollectionSpreadSheetHelpers.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SComboBox.h"

#define LOCTEXT_NAMESPACE "FacesOutliner"

// --- FGeometryCollectionFacesTreeItem ---

void FGeometryCollectionFacesTreeItem::GenerateContextMenu(UToolMenu* Menu, SFacesOutliner& Outliner)
{
}

// --- FGeometryCollectionFacesTreeItemGeometry ---

TSharedRef<ITableRow> FGeometryCollectionFacesTreeItemGeometry::MakeTreeRowWidget(const TSharedRef<STableViewBase>& InOwnerTable, bool bNoExtraColumn)
{
	FString NumVertsString = "";
	FString NumFacesString = "";
	FString TransformIdxStr = "";
	FString GeometryIdxStr = "";
	FString BoneNameStr = "";

	if (Collection.IsValid() &&
		Collection->HasAttribute("TransformIndex", FGeometryCollection::GeometryGroup) &&
		Collection->HasAttribute("TransformToGeometryIndex", FGeometryCollection::TransformGroup) &&
		Collection->HasAttribute("VertexCount", FGeometryCollection::GeometryGroup) &&
		Collection->HasAttribute("FaceCount", FGeometryCollection::GeometryGroup) &&
		Collection->HasAttribute("BoneName", FGeometryCollection::TransformGroup))
	{
		const TManagedArray<int32>& TransformIndices = Collection->GetAttribute<int32>("TransformIndex", FGeometryCollection::GeometryGroup);
		const TManagedArray<int32>& TransformToGeometryIndices = Collection->GetAttribute<int32>("TransformToGeometryIndex", FGeometryCollection::TransformGroup);
		const TManagedArray<int32>& VertexCounts = Collection->GetAttribute<int32>("VertexCount", FGeometryCollection::GeometryGroup);
		const TManagedArray<int32>& FaceCounts = Collection->GetAttribute<int32>("FaceCount", FGeometryCollection::GeometryGroup);
		const TManagedArray<FString>& BoneNames = Collection->GetAttribute<FString>("BoneName", FGeometryCollection::TransformGroup);

		const int32 TransformIdx = TransformIndices[GeometryIdx];
		TransformIdxStr = FString::FormatAsNumber(TransformIdx);

		GeometryIdxStr = FString::FormatAsNumber(GeometryIdx);

		const int32 NumVerts = VertexCounts[GeometryIdx];
		NumVertsString = FString::FormatAsNumber(NumVerts);

		const int32 NumFaces = FaceCounts[GeometryIdx];
		NumFacesString = FString::FormatAsNumber(NumFaces);

		BoneNameStr = BoneNames[TransformIdx];
	}

	return SNew(STableRow<FGeometryCollectionFacesTreeItemPtr>, InOwnerTable)
		.Content()
		[
			SNew(STextBlock).Text(FText::FromString("Geometry Idx: " + GeometryIdxStr + "     BoneName: " + BoneNameStr + "     Transform Idx: " + TransformIdxStr + "     Faces: " + NumFacesString + "     Verts: " + NumVertsString))
		];
}

void FGeometryCollectionFacesTreeItemGeometry::GetChildren(FGeometryCollectionFacesTreeItemList& OutChildren)
{
	OutChildren = ChildItems;
}

FGeometryCollectionFacesTreeItemPtr FGeometryCollectionFacesTreeItemGeometry::GetItemFromFaceIndex(int32 InFaceIndex) const
{
	return ItemsByFaceIndex.FindRef(InFaceIndex);
}

void FGeometryCollectionFacesTreeItemGeometry::GetChildrenForFace(FGeometryCollectionFacesTreeItemFace& FaceItem, FGeometryCollectionFacesTreeItemList& OutChildren) const
{
	if (!Collection.IsValid())
	{
		return;
	}
}

bool FGeometryCollectionFacesTreeItemGeometry::HasChildrenForFace(const FGeometryCollectionFacesTreeItemFace& FaceItem) const
{
	if (!Collection.IsValid())
	{
		return false;
	}

	return false;
}

void FGeometryCollectionFacesTreeItemGeometry::ExpandAll()
{
	TreeView->SetItemExpansion(AsShared(), true);

	for (const auto& Elem : ItemsByFaceIndex)
	{
		TreeView->SetItemExpansion(Elem.Value, true);
	}
}

void FGeometryCollectionFacesTreeItemGeometry::RegenerateChildren()
{
	if (Collection.IsValid())
	{
		ItemsByFaceIndex.Empty();
		ChildItems.Empty();

		if (Collection.IsValid() &&
			Collection->HasAttribute("FaceStart", FGeometryCollection::GeometryGroup) &&
			Collection->HasAttribute("FaceCount", FGeometryCollection::GeometryGroup))
		{
			const TManagedArray<int32>& FaceStarts = Collection->GetAttribute<int32>("FaceStart", FGeometryCollection::GeometryGroup);
			const TManagedArray<int32>& FaceCounts = Collection->GetAttribute<int32>("FaceCount", FGeometryCollection::GeometryGroup);

			const int32 FaceStart = FaceStarts[GeometryIdx];
			const int32 FaceCount = FaceCounts[GeometryIdx];

			RootIndex = FGeometryCollection::Invalid;

			ItemsByFaceIndex.Reserve(FaceCount);
			ChildItems.Reserve(FaceCount);

			// Add a sub item to the outliner tree for each of the bones/chunks in this GeometryCollection
			for (int32 Index = 0; Index < FaceCount; ++Index)
			{
				const int32 FaceIndex = FaceStart + Index;

				TSharedRef<FGeometryCollectionFacesTreeItemFace> NewItem = MakeShared<FGeometryCollectionFacesTreeItemFace>(FaceIndex, this);

				ChildItems.Add(NewItem);
				ItemsByFaceIndex.Add(FaceIndex, NewItem);
			}
		}
	}
}

void FGeometryCollectionFacesTreeItemGeometry::RequestTreeRefresh()
{
	if (TreeView.IsValid())
	{
		TreeView->RequestTreeRefresh();
	}
}

bool FGeometryCollectionFacesTreeItemGeometry::FilterBoneIndex(int32 BoneIndex) const
{
	return true;
}

bool FGeometryCollectionFacesTreeItemGeometry::IsValid() const
{
	if (bInvalidated)
	{
		return false;
	}

	if (Collection)
	{
		return true;
	}

	return false;
}

// --- FGeometryCollectionFacesTreeItemGeometry ---


TSharedPtr<const FManagedArrayCollection> FGeometryCollectionFacesTreeItemFace::GetCollection() const
{
	ensure(ParentCollectionItem);
	return ParentCollectionItem->GetCollection();
}

void FGeometryCollectionFacesTreeItemFace::UpdateItemColorFromCollection()
{
	ItemColor = UE::Dataflow::CollectionSpreadSheetHelpers::UpdateItemColorFromCollection(GetCollection(), FGeometryCollection::FacesGroup, GetFaceIndex());
}

TSharedRef<ITableRow> FGeometryCollectionFacesTreeItemFace::MakeTreeRowWidget(const TSharedRef<STableViewBase>& InOwnerTable, bool bIsPinned)
{
	UpdateItemColorFromCollection();

	TSharedPtr<const FManagedArrayCollection> Collection = GetCollection();

	return SNew(SFacesOutlinerRow, InOwnerTable, SharedThis(this));
}

bool FGeometryCollectionFacesTreeItemFace::IsValidFace() const
{
	TSharedPtr<const FManagedArrayCollection> Collection = GetCollection();

	if (ParentCollectionItem && ParentCollectionItem->IsValid())
	{
		return FaceIndex >= 0 && FaceIndex < Collection->NumElements(FGeometryCollection::FacesGroup);
	}

	return false;
}

TSharedRef<SWidget> FGeometryCollectionFacesTreeItemFace::MakeIndexColumnWidget() const
{
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(STextBlock)
			.Text(FText::AsNumber(FaceIndex))
			.ColorAndOpacity(ItemColor)
		];
}

TSharedRef<SWidget> FGeometryCollectionFacesTreeItemFace::MakeEmptyColumnWidget() const
{
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.Padding(12.f, 0.f)
		[
			SNew(STextBlock)
			.ColorAndOpacity(ItemColor)
		];
}

void FGeometryCollectionFacesTreeItemFace::GetChildren(FGeometryCollectionFacesTreeItemList& OutChildren)
{
	ParentCollectionItem->GetChildrenForFace(*this, OutChildren);
}

bool FGeometryCollectionFacesTreeItemFace::HasChildren() const
{
	return ParentCollectionItem->HasChildrenForFace(*this);
}

// --- SFacesOutlinerRow ---

TSharedRef<SWidget> SFacesOutlinerRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	TSharedPtr<const FManagedArrayCollection> Collection = Item->GetCollection();
	FName AttrType = UE::Dataflow::CollectionSpreadSheetHelpers::GetArrayTypeString(Collection->GetAttributeType(ColumnName, FGeometryCollection::FacesGroup));

	// This can happen because sometimes slate retains old items until the next tick, and keeps calling callbacks on them until then
	if (!Item->IsValidFace())
	{
		return Item->MakeEmptyColumnWidget();
	}

	if (ColumnName == "Index")
	{
		const TSharedPtr<SWidget> NameWidget = Item->MakeIndexColumnWidget();
		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(SExpanderArrow, SharedThis(this))
					.ShouldDrawWires(true)
			]
			+ SHorizontalBox::Slot()
			[
				NameWidget.ToSharedRef()
			];
	}

	return UE::Dataflow::CollectionSpreadSheetHelpers::MakeColumnWidget(Collection,
		FGeometryCollection::FacesGroup,
		ColumnName,
		Item->GetFaceIndex(),
		Item->GetItemColor());
}

// --- SFacesOutliner ---

void SFacesOutliner::Construct(const FArguments& InArgs)
{
	bPerformingSelection = false;

	HeaderRowWidget =
		SNew(SHeaderRow)
		.Visibility(EVisibility::Visible);

	RegenerateHeader();

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.Padding(FMargin(0.0f, 3.f))
		[
			SNew(SOverlay)
			+ SOverlay::Slot()
			[
				SAssignNew(TreeView, STreeView<FGeometryCollectionFacesTreeItemPtr>)
				.TreeItemsSource(reinterpret_cast<FGeometryCollectionFacesTreeItemList*>(&RootNodes))
				.OnSelectionChanged(this, &SFacesOutliner::OnSelectionChanged)
				.OnGenerateRow(this, &SFacesOutliner::MakeTreeRowWidget)
				.OnGetChildren(this, &SFacesOutliner::OnGetChildren)
				.OnContextMenuOpening(this, &SFacesOutliner::OnOpenContextMenu)
				.AllowInvisibleItemSelection(true)
				.ShouldStackHierarchyHeaders(true)
				.OnGeneratePinnedRow(this, &SFacesOutliner::OnGeneratePinnedRowWidget, true)
				.HighlightParentNodesForSelection(true)
				.OnSetExpansionRecursive(this, &SFacesOutliner::ExpandRecursive)
				.HeaderRow(HeaderRowWidget)
				.ExternalScrollbar(InArgs._ExternalVerticalScrollBar)
			]
		]
	];
}

void SFacesOutliner::RegenerateHeader()
{
	constexpr int32 CustomColumnWidth = 120;
	constexpr int32 CWidthMult = 9;

	HeaderRowWidget->ClearColumns();

	if (!Collection.IsValid() || Collection->NumElements(FGeometryCollection::FacesGroup) == 0)
	{
		return;
	}

	TArray<UE::Dataflow::CollectionSpreadSheetHelpers::FAttrInfo> AttrInfo;

	// Add "Index" manually
	AttrInfo.Add({ "Index", "int32" });
	// Add all the other attrs
	for (FName Attr : Collection->AttributeNames(FGeometryCollection::FacesGroup))
	{
		AttrInfo.Add({ Attr, UE::Dataflow::CollectionSpreadSheetHelpers::GetArrayTypeString(Collection->GetAttributeType(Attr, FGeometryCollection::FacesGroup)).ToString()});
	}

	for (int32 Idx = 0; Idx < AttrInfo.Num(); ++Idx)
	{
		int32 ColumnWidth = CustomColumnWidth;
		const FString ColumnNameStr = AttrInfo[Idx].Name.ToString();

		if (ColumnNameStr == "Index")
		{
			ColumnWidth = CustomColumnWidth;

			HeaderRowWidget->AddColumn(
				SHeaderRow::Column(AttrInfo[Idx].Name)
				.DefaultLabel(FText::FromName(AttrInfo[Idx].Name))
				.ManualWidth(ColumnWidth)
				.HAlignCell(HAlign_Left)
				.HAlignHeader(HAlign_Center)
				.VAlignCell(VAlign_Center));
		}
		else
		{
			int32 ColumnNameStrLen = ColumnNameStr.Len() * CWidthMult;

			if (UE::Dataflow::CollectionSpreadSheetHelpers::AttrTypeWidthMap.Contains(AttrInfo[Idx].Type))
			{
				ColumnWidth = UE::Dataflow::CollectionSpreadSheetHelpers::AttrTypeWidthMap[AttrInfo[Idx].Type];
			}

			ColumnWidth = ColumnNameStrLen > ColumnWidth ? ColumnNameStrLen : ColumnWidth;

			HeaderRowWidget->AddColumn(
				SHeaderRow::Column(AttrInfo[Idx].Name)
				.DefaultLabel(FText::FromName(AttrInfo[Idx].Name))
				.ManualWidth(ColumnWidth)
				.HAlignCell(HAlign_Center)
				.HAlignHeader(HAlign_Center)
				.VAlignCell(VAlign_Center));
		}
	}
}

void SFacesOutliner::RegenerateItems()
{
	TreeView->RebuildList();
}

TSharedRef<ITableRow> SFacesOutliner::MakeTreeRowWidget(FGeometryCollectionFacesTreeItemPtr InItem, const TSharedRef<STableViewBase>& InOwnerTable)
{
	return InItem->MakeTreeRowWidget(InOwnerTable);
}

TSharedRef<ITableRow> SFacesOutliner::OnGeneratePinnedRowWidget(FGeometryCollectionFacesTreeItemPtr InItem, const TSharedRef<STableViewBase>& InOwnerTable, bool bPinned)
{
	return InItem->MakeTreeRowWidget(InOwnerTable, true);
}

void SFacesOutliner::OnGetChildren(FGeometryCollectionFacesTreeItemPtr InItem, TArray<FGeometryCollectionFacesTreeItemPtr>& OutChildren)
{
	InItem->GetChildren(OutChildren);
}

TSharedPtr<SWidget> SFacesOutliner::OnOpenContextMenu()
{
	return TSharedPtr<SWidget>();
}

void SFacesOutliner::UpdateGeometryCollection()
{
	TreeView->RequestTreeRefresh();
	//ExpandAll();
}

void SFacesOutliner::SetCollection(const TSharedPtr<const FManagedArrayCollection>& InCollection)
{
	if (Collection == InCollection)
	{
		return;
	}

	// Clear the cached Tree ItemSelection without affecting the SelectedBones as 
	// we want to refresh the tree selection using selected bones
	TGuardValue<bool> ExternalSelectionGuard(bPerformingSelection, true);
	TreeView->ClearSelection();

	// explicitly mark the root nodes as invalid before emptying, so we know we can safely ignore them in case slate still triggers callbacks for them (they will not be deleted until the tree view refresh, on tick)
	for (TSharedPtr<FGeometryCollectionFacesTreeItemGeometry>& RootNode : RootNodes)
	{
		if (RootNode)
		{
			RootNode->Invalidate();
		}
	}
	RootNodes.Empty();

	const int32 NumGeoms = InCollection->NumElements(FGeometryCollection::GeometryGroup);
	
	for (int32 Idx = 0; Idx < NumGeoms; ++Idx)
	{
		RootNodes.Add(MakeShared<FGeometryCollectionFacesTreeItemGeometry>(InCollection, Idx, TreeView));
	}

	TreeView->RequestTreeRefresh();
	//ExpandAll();

	Collection = InCollection;
}

void SFacesOutliner::ExpandAll()
{
	for (TSharedPtr<FGeometryCollectionFacesTreeItemGeometry> ItemPtr : RootNodes)
	{
		ItemPtr->ExpandAll();
	}
}

void SFacesOutliner::ExpandRecursive(TSharedPtr<FGeometryCollectionFacesTreeItem> ItemPtr, bool bInExpansionState) const
{
	TreeView->SetItemExpansion(ItemPtr, bInExpansionState);

	FGeometryCollectionFacesTreeItemList ItemChildren;
	ItemPtr->GetChildren(ItemChildren);
	for (auto& Child : ItemChildren)
	{
		ExpandRecursive(Child, bInExpansionState);
	}
}

int32 SFacesOutliner::GetBoneSelectionCount() const
{
	return TreeView->GetSelectedItems().Num();
}

void SFacesOutliner::OnSelectionChanged(FGeometryCollectionFacesTreeItemPtr Item, ESelectInfo::Type SelectInfo)
{
}

#undef LOCTEXT_NAMESPACE // "FacesOutliner"
