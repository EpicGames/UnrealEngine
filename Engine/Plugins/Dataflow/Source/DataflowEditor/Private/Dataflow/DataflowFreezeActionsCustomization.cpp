// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowFreezeActionsCustomization.h"
#include "Dataflow/DataflowEditorStyle.h"
#include "Dataflow/DataflowGraphEditor.h"
#include "Dataflow/DataflowNode.h"
#include "Dataflow/DataflowTerminalNode.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/Text/STextBlock.h"
#include "DetailWidgetRow.h"
#include "PropertyHandle.h"

#define LOCTEXT_NAMESPACE "DataflowNodeCustomization"

namespace UE::Dataflow
{
	namespace Private
	{
		FDataflowNode* GetDataflowNode(TSharedRef<IPropertyHandle> StructPropertyHandle)
		{
			if (const TSharedPtr<IPropertyHandle> ParentHandle = StructPropertyHandle->GetParentHandle())
			{
				if (const TSharedPtr<IPropertyHandleStruct> ParentHandleStruct = ParentHandle->AsStruct())
				{
					if (const TSharedPtr<FStructOnScope> StructOnScope = ParentHandleStruct->GetStructData())
					{
						if (const UStruct* const Struct = StructOnScope->GetStruct())
						{
							if (Struct->IsChildOf<FDataflowNode>())
							{
								if (StructOnScope->GetStructMemory())
								{
									return reinterpret_cast<FDataflowNode*>(StructOnScope->GetStructMemory());
								}
							}
						}
					}
				}
			}
			return nullptr;
		}
	}

	TSharedRef<IPropertyTypeCustomization> FFreezeActionsCustomization::MakeInstance()
	{
		return MakeShareable(new FFreezeActionsCustomization);
	}

	void FFreezeActionsCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
	{
		// Keep a weak pointer to the graph editor that is creating this customization
		TWeakPtr<const SDataflowGraphEditor> DataflowGraphEditor = SDataflowGraphEditor::GetSelectedGraphEditor();

		// Create WrapBox
		TSharedPtr<SWrapBox> WrapBox;
		HeaderRow
		[
			SAssignNew(WrapBox, SWrapBox)
			.PreferredSize(2000)  // Copied from FObjectDetails::AddCallInEditorMethods()
			.UseAllottedSize(true)
		];

		
		TWeakPtr<FDataflowNode> WeakDataflowNode;
		if (FDataflowNode* DataflowNode = Private::GetDataflowNode(StructPropertyHandle))
		{
			WeakDataflowNode = DataflowNode->AsWeak();
		}
		
		// Create button image + text
		WrapBox->AddSlot()
			.Padding(0.f, 0.f, 5.f, 3.f)
			[
				SNew(SButton)
					.Text(LOCTEXT("Freeze", "Freeze"))
					.ToolTipText_Lambda([WeakDataflowNode]() -> FText
						{
							const TSharedPtr<const FDataflowNode> DataflowNode = WeakDataflowNode.Pin();
							return DataflowNode && !DataflowNode->IsFrozen() ?
								LOCTEXT("FreezeToolTip", "Freeze all outputs, and disable evaluation for this node.") :
								LOCTEXT("UnfreezeToolTip", "UInfreeze all outputs, and re-enable evaluation for this node.");
						})
					.OnClicked_Lambda(
						[WeakDataflowNode, DataflowGraphEditor]() -> FReply
						{
							if (const TSharedPtr<FDataflowNode> DataflowNode = WeakDataflowNode.Pin())
							{
								// Retrieve context if any
								const TSharedPtr<const SDataflowGraphEditor> DataflowGraphEditorPtr = DataflowGraphEditor.Pin();
								const TSharedPtr<UE::Dataflow::FContext> Context =
									DataflowGraphEditorPtr ? DataflowGraphEditorPtr->GetDataflowContext() :
									TSharedPtr<UE::Dataflow::FContext>();
								if (DataflowNode->IsFrozen())
								{
									DataflowNode->Unfreeze(*Context);
								}
								else
								{
									DataflowNode->Freeze(*Context);
								}
							}
							return FReply::Handled();
						})
					.ContentPadding(FMargin(0.f, 2.f))  // Too much horizontal padding otherwise (default is 4, 2)
					.Visibility_Lambda([WeakDataflowNode]() -> EVisibility
						{
							const TSharedPtr<const FDataflowNode> DataflowNode = WeakDataflowNode.Pin();
							return DataflowNode && !DataflowNode->AsType<const FDataflowTerminalNode>() ? EVisibility::Visible : EVisibility::Collapsed;
						})
					[
						SNew(SHorizontalBox)
							+ SHorizontalBox::Slot()
							.AutoWidth()
							.Padding(0, 2)
							[
								SNew(SImage)
									.DesiredSizeOverride(FVector2D(16, 16))
									.Image(FDataflowEditorStyle::Get().GetBrush("Dataflow.FreezeNode"))
							]
							+ SHorizontalBox::Slot()
							.AutoWidth()
							.Padding(5.f, 0.f)
							+ SHorizontalBox::Slot()
							.AutoWidth()
							.VAlign(VAlign_Center)
							[
								SNew(STextBlock)
									.Text_Lambda([WeakDataflowNode]() -> FText
										{
											const TSharedPtr<const FDataflowNode> DataflowNode = WeakDataflowNode.Pin();
											return DataflowNode && !DataflowNode->IsFrozen() ?
												LOCTEXT("Freeze", "Freeze") :
												LOCTEXT("Unfreeze", "Unfreeze");
										})
							]
					]
			];
		// Add Refreeze button
		WrapBox->AddSlot()
			.Padding(0.f, 0.f, 5.f, 3.f)
			[
				SNew(SButton)
					.Text(LOCTEXT("Refreeze", "Refreeze"))
					.ToolTipText(LOCTEXT("RefreezeToolTip", "Unfreeze all outputs, redo this node's evaluation, and freeze all updated outputs again."))
					.OnClicked_Lambda(
						[WeakDataflowNode, DataflowGraphEditor]() -> FReply
						{
							if (const TSharedPtr<FDataflowNode> DataflowNode = WeakDataflowNode.Pin())
							{
								if (DataflowNode->IsFrozen())
								{
									// Retrieve context if any
									const TSharedPtr<const SDataflowGraphEditor> DataflowGraphEditorPtr = DataflowGraphEditor.Pin();
									const TSharedPtr<UE::Dataflow::FContext> Context =
										DataflowGraphEditorPtr ? DataflowGraphEditorPtr->GetDataflowContext() :
										TSharedPtr<UE::Dataflow::FContext>();
									// Refreeze
									DataflowNode->Unfreeze(*Context);
									Context->Evaluate(DataflowNode.Get(), nullptr);
									DataflowNode->Freeze(*Context);
								}
							}
							return FReply::Handled();
						})
					.ContentPadding(FMargin(0.f, 2.f))  // Too much horizontal padding otherwise (default is 4, 2)
					.IsEnabled_Lambda([WeakDataflowNode]() -> bool
						{
							const TSharedPtr<const FDataflowNode> DataflowNode = WeakDataflowNode.Pin();
							return DataflowNode && DataflowNode->IsFrozen();
						})
					.Visibility_Lambda([WeakDataflowNode]() -> EVisibility
						{
							const TSharedPtr<const FDataflowNode> DataflowNode = WeakDataflowNode.Pin();
							return DataflowNode && !DataflowNode->AsType<const FDataflowTerminalNode>() ? EVisibility::Visible : EVisibility::Collapsed;
						})
					[
						SNew(SHorizontalBox)
							+ SHorizontalBox::Slot()
							.AutoWidth()
							.Padding(0, 2)
							[
								SNew(SImage)
									.DesiredSizeOverride(FVector2D(16, 16))
									.Image(FAppStyle::GetBrush("Icons.Refresh"))
							]
							+ SHorizontalBox::Slot()
							.AutoWidth()
							.Padding(5.f, 0.f)
							+ SHorizontalBox::Slot()
							.AutoWidth()
							.VAlign(VAlign_Center)
							[
								SNew(STextBlock)
									.Text(LOCTEXT("Refreeze", "Refreeze"))
							]
					]
			];
	}
}  // End namespace UE::Dataflow

#undef LOCTEXT_NAMESPACE
