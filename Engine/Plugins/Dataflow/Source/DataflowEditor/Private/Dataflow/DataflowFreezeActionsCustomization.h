// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"

namespace UE::Dataflow
{
	/**
	 * Customization for the Dataflow node UI.
	 */
	class FFreezeActionsCustomization final : public IPropertyTypeCustomization
	{
	public:
		static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	private:
		//~ Begin IPropertyTypeCustomization interface
		virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& customizationutils) override;
		virtual void CustomizeChildren(TSharedRef<IPropertyHandle> /*StructPropertyHandle*/, IDetailChildrenBuilder& /*ChildBuilder*/, IPropertyTypeCustomizationUtils& /*CustomizationUtils*/) override {}
		//~ End IPropertyTypeCustomization interface
	};
}
