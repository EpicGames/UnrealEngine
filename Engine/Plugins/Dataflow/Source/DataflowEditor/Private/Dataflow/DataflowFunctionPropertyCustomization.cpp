// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowFunctionPropertyCustomization.h"
#include "Dataflow/DataflowFunctionProperty.h"
#include "Dataflow/DataflowGraphEditor.h"
#include "Dataflow/DataflowNodeParameters.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/Text/STextBlock.h"
#include "DetailWidgetRow.h"
#include "IPropertyUtilities.h"
#include "PropertyHandle.h"

#define LOCTEXT_NAMESPACE "FunctionPropertyCustomization"

namespace UE::Dataflow
{
	namespace Private
	{
		static const FDataflowFunctionProperty* GetFunctionProperty(const TSharedPtr<IPropertyHandle>& PropertyHandle)
		{
			const FDataflowFunctionProperty* FunctionProperty = nullptr;
			if (PropertyHandle)
			{
				if (const FStructProperty* const StructProperty = CastField<FStructProperty>(PropertyHandle->GetProperty()))
				{
					if (StructProperty->Struct && StructProperty->Struct->IsChildOf(FDataflowFunctionProperty::StaticStruct()))
					{
						void* Data;
						if (PropertyHandle->GetValueData(Data) == FPropertyAccess::Success)
						{
							FunctionProperty = reinterpret_cast<const FDataflowFunctionProperty*>(Data);
						}
					}
				}
			}
			return FunctionProperty;
		}
	}

	TSharedRef<IPropertyTypeCustomization> FFunctionPropertyCustomization::MakeInstance()
	{
		return MakeShareable(new FFunctionPropertyCustomization);
	}

	void FFunctionPropertyCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
	{
		StructProperty = StructPropertyHandle;

		// Keep a weak pointer to the graph editor that is creating this customization
		DataflowGraphEditor = SDataflowGraphEditor::GetSelectedGraphEditor();

		// Used to rebuild the details panel after a button runs, so EditConditionHides buttons re-evaluate
		const TSharedPtr<IPropertyUtilities> PropertyUtilities = CustomizationUtils.GetPropertyUtilities();

		// Find all functions under the same category
		const FName CategoryName = StructProperty->GetDefaultCategoryName();
		const TSharedPtr<IPropertyHandle> OwnerProperty = StructProperty->GetParentHandle();

		uint32 NumChildren;
		const FPropertyAccess::Result Result = OwnerProperty->GetNumChildren(NumChildren);
		check(Result == FPropertyAccess::Success);

		TSharedPtr<SWrapBox> WrapBox;
		bool bFirstFunction = true;

		for (uint32 Index = 0; Index < NumChildren; ++Index)
		{
			const TSharedPtr<IPropertyHandle> ChildProperty = OwnerProperty->GetChildHandle(Index);

			if (const FDataflowFunctionProperty* const FunctionProperty = Private::GetFunctionProperty(ChildProperty))
			{
				if (ChildProperty->GetDefaultCategoryName() == CategoryName)
				{
					if (bFirstFunction)
					{
						if (ChildProperty->GetProperty() != StructProperty->GetProperty())
						{
							return;  // The first function property draws all others, no need to do it again
						}
						bFirstFunction = false;

						// Create WrapBox
						HeaderRow
						.EditCondition(true, FOnBooleanValueChanged())  // Add a custom edit condition so that the first button doesn't affect the others
						[
							SAssignNew(WrapBox, SWrapBox)
							.PreferredSize(2000)  // Copied from FObjectDetails::AddCallInEditorMethods()
							.UseAllottedSize(true)
						];
					}

					auto OnClicked = [this, ChildProperty, FunctionProperty, PropertyUtilities]() -> FReply
						{
							// Retrieve context if any
							const TSharedPtr<const SDataflowGraphEditor> DataflowGraphEditorPtr = DataflowGraphEditor.Pin();
							const TSharedPtr<UE::Dataflow::FContext> Context = DataflowGraphEditorPtr ? DataflowGraphEditorPtr->GetDataflowContext() : TSharedPtr<UE::Dataflow::FContext>();

							// Execute function
							UE::Dataflow::FContextThreaded EmptyContext;
							FunctionProperty->Execute(Context.IsValid() ? *Context : EmptyContext);

							// Triggers node invalidation
							ChildProperty->NotifyFinishedChangingProperties();

							// Rebuild the details panel so EditConditionHides buttons gated on properties this function changed re-evaluate their visibility
							if (PropertyUtilities)
							{
								PropertyUtilities->RequestForceRefresh();
							}

							return FReply::Handled();
						};

					const FText Name = ChildProperty->HasMetaData("DisplayName") ?
						FText::FromString(ChildProperty->GetMetaData("DisplayName")) :
						ChildProperty->GetPropertyDisplayName();
					const FText ToolTip = ChildProperty->GetToolTipText();
					const FName ButtonImage = *ChildProperty->GetMetaData("ButtonImage");  // e.g. FAppStyle::GetBrush("Persona.ReimportAsset") will be Meta = (ButtonImage = "Persona.ReimportAsset")

					if (!ButtonImage.IsNone())
					{
						if (Name.IsEmptyOrWhitespace())
						{
							// Create button image
							WrapBox->AddSlot()
							.Padding(0.f, 0.f, 5.f, 3.f)
							[
								SNew(SButton)
								.ToolTipText(ToolTip)
								.OnClicked_Lambda(MoveTemp(OnClicked))
								.ContentPadding(FMargin(0.f, 4.f))  // Too much horizontal padding otherwise (default is 4, 2)
								.IsEnabled_Lambda([ChildProperty]() -> bool { return ChildProperty->IsEditable(); })
								.Visibility_Lambda([ChildProperty]() -> EVisibility
									{
										return !ChildProperty->HasMetaData("EditConditionHides") || ChildProperty->IsEditable() ? EVisibility::Visible : EVisibility::Collapsed;
									})
								[
									SNew(SImage)
										.DesiredSizeOverride(FVector2D(16, 16))
										.Image(FAppStyle::GetBrush(ButtonImage))
								]
							];
						}
						else
						{
							// Create button image + text
							WrapBox->AddSlot()
							.Padding(0.f, 0.f, 5.f, 3.f)
							[
								SNew(SButton)
								.Text(Name)
								.ToolTipText(ToolTip)
								.OnClicked_Lambda(MoveTemp(OnClicked))
								.ContentPadding(FMargin(0.f, 2.f))  // Too much horizontal padding otherwise (default is 4, 2)
								.IsEnabled_Lambda([ChildProperty]() -> bool { return ChildProperty->IsEditable(); })
								.Visibility_Lambda([ChildProperty]() -> EVisibility
									{
										return !ChildProperty->HasMetaData("EditConditionHides") || ChildProperty->IsEditable() ? EVisibility::Visible : EVisibility::Collapsed;
									})
								[
									SNew(SHorizontalBox)
									+ SHorizontalBox::Slot()
									.AutoWidth()
									.Padding(0, 2)
									[
										SNew(SImage)
										.DesiredSizeOverride(FVector2D(16, 16))
										.Image(FAppStyle::GetBrush(ButtonImage))
									]
									+ SHorizontalBox::Slot()
									.AutoWidth()
									.Padding(5.f, 0.f)
									+ SHorizontalBox::Slot()
									.AutoWidth()
									.VAlign(VAlign_Center)
									[
										SNew(STextBlock)
											.Text(Name)
									]
								]
							];
						}
					}
					else
					{
						// Create button text
						WrapBox->AddSlot()
						.Padding(0.f, 0.f, 5.f, 3.f)
						[
							SNew(SButton)
							.Text(Name)
							.ToolTipText(ToolTip)
							.OnClicked_Lambda(MoveTemp(OnClicked))
							.ContentPadding(FMargin(0.f, 2.f))  // Too much horizontal padding otherwise (default is 4, 2)
							.VAlign(VAlign_Center)  // No Slot, so need vertical align
							.IsEnabled_Lambda([ChildProperty]() -> bool { return ChildProperty->IsEditable(); })
							.Visibility_Lambda([ChildProperty]() -> EVisibility
								{
									return !ChildProperty->HasMetaData("EditConditionHides") || ChildProperty->IsEditable() ? EVisibility::Visible : EVisibility::Collapsed;
								})
						];
					}
				}
			}
		}
	}
}  // End namespace UE::Dataflow

#undef LOCTEXT_NAMESPACE
