// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"

class SDataflowGraphEditor;

namespace UE::Dataflow
{
	/**
	 * Customization for buttons in the UStruct UI of dataflow nodes.
	 */
	class FFunctionPropertyCustomization : public IPropertyTypeCustomization
	{
	public:
		static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	protected:
		virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
		virtual void CustomizeChildren(TSharedRef<IPropertyHandle> /*StructPropertyHandle*/, IDetailChildrenBuilder& /*ChildBuilder*/, IPropertyTypeCustomizationUtils& /*CustomizationUtils*/) override {}

	private:
		TSharedPtr<IPropertyHandle> StructProperty;
		TWeakPtr<const SDataflowGraphEditor> DataflowGraphEditor;
	};
}
