// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowGraphEditor.h"

#include "BoneDragDropOp.h"
#include "Dataflow/DataflowAssetEditUtils.h"
#include "Dataflow/DataflowEditor.h"
#include "Dataflow/DataflowEditorToolkit.h"
#include "Dataflow/DataflowEngine.h"
#include "Dataflow/DataflowEngineUtil.h"
#include "Dataflow/DataflowSEditorInterface.h"
#include "Dataflow/DataflowSNodeFactories.h"
#include "Dataflow/DataflowSCommentNode.h"
#include "Dataflow/DataflowNodeParameters.h"
#include "Dataflow/DataflowGraphSchemaAction.h"
#include "Dataflow/DataflowSubGraphNodes.h"
#include "Dataflow/DataflowSchema.h"
#include "Dataflow/DataflowSubGraph.h"
#include "Dataflow/DataflowVariableNodes.h"
#include "Dataflow/SDataflowGraphActionWidget.h"
#include "HAL/PlatformApplicationMisc.h"
#include "ScopedTransaction.h"
#include "IStructureDetailsView.h"

#include "SGraphEditorActionMenu.h"
#include "SGraphPanel.h"

#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Notifications/SProgressBar.h"

#define LOCTEXT_NAMESPACE "DataflowGraphEditor"

namespace DataflowGraphEditor::Private
{
	struct FDataflowZoomLevelEntry
	{
	public:
		FDataflowZoomLevelEntry(float InZoomAmount, const FText& InDisplayText, EGraphRenderingLOD::Type InLOD)
			: DisplayText(FText::Format(LOCTEXT("Zoom", "Zoom {0}"), InDisplayText))
			, ZoomAmount(InZoomAmount)
			, LOD(InLOD)
		{}

	public:
		FText DisplayText;
		float ZoomAmount;
		EGraphRenderingLOD::Type LOD;
	};

	struct FDataflowZoomLevelsContainer : public FZoomLevelsContainer
	{
		FDataflowZoomLevelsContainer()
		{
			ZoomLevels.Append({
					FDataflowZoomLevelEntry(0.025f, FText::FromString(TEXT("-14")), EGraphRenderingLOD::LowestDetail),
					FDataflowZoomLevelEntry(0.070f, FText::FromString(TEXT("-13")), EGraphRenderingLOD::LowestDetail),
					FDataflowZoomLevelEntry(0.100f, FText::FromString(TEXT("-12")), EGraphRenderingLOD::LowestDetail),
					FDataflowZoomLevelEntry(0.125f, FText::FromString(TEXT("-11")), EGraphRenderingLOD::LowestDetail),
					FDataflowZoomLevelEntry(0.150f, FText::FromString(TEXT("-10")), EGraphRenderingLOD::LowestDetail),
					FDataflowZoomLevelEntry(0.175f, FText::FromString(TEXT("-9")), EGraphRenderingLOD::LowestDetail),
					FDataflowZoomLevelEntry(0.200f, FText::FromString(TEXT("-8")), EGraphRenderingLOD::LowestDetail),
					FDataflowZoomLevelEntry(0.225f, FText::FromString(TEXT("-7")), EGraphRenderingLOD::LowDetail),
					FDataflowZoomLevelEntry(0.250f, FText::FromString(TEXT("-6")), EGraphRenderingLOD::LowDetail),
					FDataflowZoomLevelEntry(0.375f, FText::FromString(TEXT("-5")), EGraphRenderingLOD::MediumDetail),
					FDataflowZoomLevelEntry(0.500f, FText::FromString(TEXT("-4")), EGraphRenderingLOD::MediumDetail),
					FDataflowZoomLevelEntry(0.675f, FText::FromString(TEXT("-3")), EGraphRenderingLOD::MediumDetail),
					FDataflowZoomLevelEntry(0.750f, FText::FromString(TEXT("-2")), EGraphRenderingLOD::DefaultDetail),
					FDataflowZoomLevelEntry(0.875f, FText::FromString(TEXT("-1")), EGraphRenderingLOD::DefaultDetail),
					FDataflowZoomLevelEntry(1.000f, FText::FromString(TEXT("1:1")), EGraphRenderingLOD::DefaultDetail), // default #14
					FDataflowZoomLevelEntry(1.250f, FText::FromString(TEXT("+1")), EGraphRenderingLOD::DefaultDetail),
					FDataflowZoomLevelEntry(1.375f, FText::FromString(TEXT("+2")), EGraphRenderingLOD::DefaultDetail),
					FDataflowZoomLevelEntry(1.500f, FText::FromString(TEXT("+3")), EGraphRenderingLOD::FullyZoomedIn),
					FDataflowZoomLevelEntry(1.675f, FText::FromString(TEXT("+4")), EGraphRenderingLOD::FullyZoomedIn),
					FDataflowZoomLevelEntry(1.750f, FText::FromString(TEXT("+5")), EGraphRenderingLOD::FullyZoomedIn),
					FDataflowZoomLevelEntry(1.875f, FText::FromString(TEXT("+6")), EGraphRenderingLOD::FullyZoomedIn),
					FDataflowZoomLevelEntry(2.000f, FText::FromString(TEXT("+7")), EGraphRenderingLOD::FullyZoomedIn),
				});
		}

		float GetZoomAmount(int32 InZoomLevel) const override
		{
			checkSlow(ZoomLevels.IsValidIndex(InZoomLevel));
			return ZoomLevels[InZoomLevel].ZoomAmount;
		}

		int32 GetNearestZoomLevel(float InZoomAmount) const override
		{
			for (int32 ZoomLevelIndex = 0; ZoomLevelIndex < GetNumZoomLevels(); ++ZoomLevelIndex)
			{
				if (InZoomAmount <= GetZoomAmount(ZoomLevelIndex))
				{
					return ZoomLevelIndex;
				}
			}

			return GetDefaultZoomLevel();
		}

		FText GetZoomText(int32 InZoomLevel) const override
		{
			checkSlow(ZoomLevels.IsValidIndex(InZoomLevel));
			return ZoomLevels[InZoomLevel].DisplayText;
		}

		int32 GetNumZoomLevels() const override
		{
			return ZoomLevels.Num();
		}

		int32 GetDefaultZoomLevel() const override
		{
			return 14;
		}

		EGraphRenderingLOD::Type GetLOD(int32 InZoomLevel) const override
		{
			checkSlow(ZoomLevels.IsValidIndex(InZoomLevel));
			return ZoomLevels[InZoomLevel].LOD;
		}

		TArray<FDataflowZoomLevelEntry> ZoomLevels;
	};
}

TSharedPtr<FDataflowGraphEditorNodeFactory> SDataflowGraphEditor::NodeFactory;
TWeakPtr<SDataflowGraphEditor> SDataflowGraphEditor::SelectedGraphEditor;
TWeakPtr<SDataflowGraphEditor> SDataflowGraphEditor::LastActionMenuGraphEditor;

void SDataflowGraphEditor::Construct(const FArguments& InArgs, UObject* InAssetOwner)
{
	check(InArgs._GraphToEdit);
	AssetOwner = InAssetOwner; // nullptr is valid
	EdGraphWeakPtr = InArgs._GraphToEdit;
	DetailsView = InArgs._DetailsView;
	EvaluateGraphCallback = InArgs._EvaluateGraph;
	OnDragDropEventCallback = InArgs._OnDragDropEvent;
	IsEditableCallback = InArgs._IsEditable;
	DataflowEditor = InArgs._DataflowEditor;

	FGraphAppearanceInfo AppearanceInfo;
	AppearanceInfo.CornerText = FText::FromString("Dataflow");

	FGraphEditorCommands::Register();
	if (!GraphEditorCommands.IsValid())
	{
		GraphEditorCommands = MakeShareable(new FUICommandList);
		{
			GraphEditorCommands->MapAction(FGenericCommands::Get().SelectAll,
				FExecuteAction::CreateSP(this, &SDataflowGraphEditor::SelectAllNodes)
			);

			GraphEditorCommands->MapAction(
				FGenericCommands::Get().Delete,
				FExecuteAction::CreateSP(this, &SDataflowGraphEditor::DeleteNode),
				FCanExecuteAction::CreateSP(this, &SDataflowGraphEditor::IsEditable)
			);
			GraphEditorCommands->MapAction(
				FDataflowEditorCommands::Get().EvaluateNode,
				FExecuteAction::CreateSP(this, &SDataflowGraphEditor::EvaluateNode)
			);
			GraphEditorCommands->MapAction(
				FDataflowEditorCommands::Get().FreezeNodes,
				FExecuteAction::CreateSP(this, &SDataflowGraphEditor::FreezeNodes),
				FCanExecuteAction::CreateSP(this, &SDataflowGraphEditor::IsEditable)
			);
			GraphEditorCommands->MapAction(
				FDataflowEditorCommands::Get().UnfreezeNodes,
				FExecuteAction::CreateSP(this, &SDataflowGraphEditor::UnfreezeNodes),
				FCanExecuteAction::CreateSP(this, &SDataflowGraphEditor::IsEditable)
			);
			GraphEditorCommands->MapAction(
				FGraphEditorCommands::Get().CreateComment,
				FExecuteAction::CreateRaw(this, &SDataflowGraphEditor::CreateComment),
				FCanExecuteAction::CreateSP(this, &SDataflowGraphEditor::IsEditable)
			);
			GraphEditorCommands->MapAction(
				FGraphEditorCommands::Get().AlignNodesTop,
				FExecuteAction::CreateSP(this, &SDataflowGraphEditor::AlignTop),
				FCanExecuteAction::CreateSP(this, &SDataflowGraphEditor::IsEditable)
			);
			GraphEditorCommands->MapAction(
				FGraphEditorCommands::Get().AlignNodesMiddle,
				FExecuteAction::CreateSP(this, &SDataflowGraphEditor::AlignMiddle),
				FCanExecuteAction::CreateSP(this, &SDataflowGraphEditor::IsEditable)
			);
			GraphEditorCommands->MapAction(
				FGraphEditorCommands::Get().AlignNodesBottom,
				FExecuteAction::CreateSP(this, &SDataflowGraphEditor::AlignBottom),
				FCanExecuteAction::CreateSP(this, &SDataflowGraphEditor::IsEditable)
			);
			GraphEditorCommands->MapAction(
				FGraphEditorCommands::Get().AlignNodesLeft,
				FExecuteAction::CreateSP(this, &SDataflowGraphEditor::AlignLeft),
				FCanExecuteAction::CreateSP(this, &SDataflowGraphEditor::IsEditable)
			);
			GraphEditorCommands->MapAction(
				FGraphEditorCommands::Get().AlignNodesCenter,
				FExecuteAction::CreateSP(this, &SDataflowGraphEditor::AlignCenter),
				FCanExecuteAction::CreateSP(this, &SDataflowGraphEditor::IsEditable)
			);
			GraphEditorCommands->MapAction(
				FGraphEditorCommands::Get().AlignNodesRight,
				FExecuteAction::CreateSP(this, &SDataflowGraphEditor::AlignRight),
				FCanExecuteAction::CreateSP(this, &SDataflowGraphEditor::IsEditable)
			);
			GraphEditorCommands->MapAction(
				FGraphEditorCommands::Get().StraightenConnections,
				FExecuteAction::CreateSP(this, &SDataflowGraphEditor::StraightenConnections),
				FCanExecuteAction::CreateSP(this, &SDataflowGraphEditor::IsEditable)
			);
			GraphEditorCommands->MapAction(
				FGraphEditorCommands::Get().DistributeNodesHorizontally,
				FExecuteAction::CreateSP(this, &SDataflowGraphEditor::DistributeHorizontally),
				FCanExecuteAction::CreateSP(this, &SDataflowGraphEditor::IsEditable)
			);
			GraphEditorCommands->MapAction(
				FGraphEditorCommands::Get().DistributeNodesVertically,
				FExecuteAction::CreateSP(this, &SDataflowGraphEditor::DistributeVertically),
				FCanExecuteAction::CreateSP(this, &SDataflowGraphEditor::IsEditable)
			);
			GraphEditorCommands->MapAction(
				FDataflowEditorCommands::Get().ToggleEnabledState,
				FExecuteAction::CreateSP(this, &SDataflowGraphEditor::ToggleEnabledState),
				FCanExecuteAction::CreateSP(this, &SDataflowGraphEditor::IsEditable)
			);
			GraphEditorCommands->MapAction(
				FDataflowEditorCommands::Get().AddOptionPin,
				FExecuteAction::CreateSP(this, &SDataflowGraphEditor::OnAddOptionPin),
				FCanExecuteAction::CreateSP(this, &SDataflowGraphEditor::CanAddOptionPin)
			);
			GraphEditorCommands->MapAction(
				FDataflowEditorCommands::Get().RemoveOptionPin,
				FExecuteAction::CreateSP(this, &SDataflowGraphEditor::OnRemoveOptionPin),
				FCanExecuteAction::CreateSP(this, &SDataflowGraphEditor::CanRemoveOptionPin)
			);
			GraphEditorCommands->MapAction(
				FGenericCommands::Get().Duplicate,
				FExecuteAction::CreateSP(this, &SDataflowGraphEditor::DuplicateSelectedNodes),
				FCanExecuteAction::CreateSP(this, &SDataflowGraphEditor::IsEditable)
			);
			GraphEditorCommands->MapAction(
				FDataflowEditorCommands::Get().ZoomToFitGraph,
				FExecuteAction::CreateSP(this, &SDataflowGraphEditor::ZoomToFitGraph)
			);
			GraphEditorCommands->MapAction(
				FGraphEditorCommands::Get().ShowAllPins,
				FExecuteAction::CreateSP(this, &SDataflowGraphEditor::SetPinVisibility, SGraphEditor::Pin_Show),
				FCanExecuteAction::CreateSP(this, &SDataflowGraphEditor::IsEditable),
				FIsActionChecked::CreateSP(this, &SDataflowGraphEditor::GetPinVisibility, SGraphEditor::Pin_Show)
			);
			GraphEditorCommands->MapAction(
				FGraphEditorCommands::Get().HideNoConnectionPins,
				FExecuteAction::CreateSP(this, &SDataflowGraphEditor::SetPinVisibility, SGraphEditor::Pin_HideNoConnection),
				FCanExecuteAction::CreateSP(this, &SDataflowGraphEditor::IsEditable),
				FIsActionChecked::CreateSP(this, &SDataflowGraphEditor::GetPinVisibility, SGraphEditor::Pin_HideNoConnection)
			);
			GraphEditorCommands->MapAction(
				FGraphEditorCommands::Get().HideNoConnectionNoDefaultPins,
				FExecuteAction::CreateSP(this, &SDataflowGraphEditor::SetPinVisibility, SGraphEditor::Pin_HideNoConnectionNoDefault),
				FCanExecuteAction::CreateSP(this, &SDataflowGraphEditor::IsEditable),
				FIsActionChecked::CreateSP(this, &SDataflowGraphEditor::GetPinVisibility, SGraphEditor::Pin_HideNoConnectionNoDefault)
			);
			GraphEditorCommands->MapAction(
				FGenericCommands::Get().Copy,
				FExecuteAction::CreateSP(this, &SDataflowGraphEditor::CopySelectedNodes)
			);
			GraphEditorCommands->MapAction(
				FGenericCommands::Get().Cut,
				FExecuteAction::CreateSP(this, &SDataflowGraphEditor::CutSelectedNodes),
				FCanExecuteAction::CreateSP(this, &SDataflowGraphEditor::IsEditable)
			);
			GraphEditorCommands->MapAction(
				FGenericCommands::Get().Paste,
				FExecuteAction::CreateSP(this, &SDataflowGraphEditor::PasteSelectedNodes),
				FCanExecuteAction::CreateSP(this, &SDataflowGraphEditor::IsEditable)
			);
			GraphEditorCommands->MapAction(
				FGenericCommands::Get().Rename,
				FExecuteAction::CreateSP(this, &SDataflowGraphEditor::RenameNode),
				FCanExecuteAction::CreateSP(this, &SDataflowGraphEditor::CanRenameNode)
			);
			GraphEditorCommands->MapAction(
				FDataflowEditorCommands::Get().AddNewVariable,
				FExecuteAction::CreateSP(this, &SDataflowGraphEditor::AddNewVariable),
				FCanExecuteAction::CreateSP(this, &SDataflowGraphEditor::IsEditable)
			);
			GraphEditorCommands->MapAction(
					FDataflowEditorCommands::Get().AddNewSubGraph,
					FExecuteAction::CreateSP(this, &SDataflowGraphEditor::AddNewSubGraph),
				FCanExecuteAction::CreateSP(this, &SDataflowGraphEditor::IsEditable)
			);
			GraphEditorCommands->MapAction(FGraphEditorCommands::Get().StartWatchingPin,
				FExecuteAction::CreateSP(this, &SDataflowGraphEditor::OnStartWatchingPin),
				FCanExecuteAction::CreateSP(this, &SDataflowGraphEditor::CanStartWatchingPin)
			);

			GraphEditorCommands->MapAction(FGraphEditorCommands::Get().StopWatchingPin,
				FExecuteAction::CreateSP(this, &SDataflowGraphEditor::OnStopWatchingPin),
				FCanExecuteAction::CreateSP(this, &SDataflowGraphEditor::CanStopWatchingPin)
			);

			GraphEditorCommands->MapAction(FGraphEditorCommands::Get().PromoteToVariable,
				FExecuteAction::CreateSP(this, &SDataflowGraphEditor::OnPromoteToVariable),
				FCanExecuteAction::CreateSP(this, &SDataflowGraphEditor::CanPromoteToVariable)
			);

		}
	}

	SGraphEditor::FArguments Arguments;
	Arguments._AdditionalCommands = GraphEditorCommands;
	Arguments._Appearance = AppearanceInfo;
	Arguments._GraphToEdit = InArgs._GraphToEdit;
	Arguments._GraphEvents = InArgs._GraphEvents;
	Arguments.IsEditable_Lambda([this]()->bool { return IsEditable(); });
	Arguments.DisplayAsReadOnly_Lambda([this]()->bool { return !IsEditable(); });

	ensureMsgf(!Arguments._GraphEvents.OnSelectionChanged.IsBound(), TEXT("DataflowGraphEditor::OnSelectionChanged rebound during construction."));
	Arguments._GraphEvents.OnSelectionChanged = SGraphEditor::FOnSelectionChanged::CreateSP(this, &SDataflowGraphEditor::OnSelectedNodesChanged);

	Arguments._GraphEvents.OnCreateActionMenuAtLocation = SGraphEditor::FOnCreateActionMenuAtLocation::CreateSP(this, &SDataflowGraphEditor::OnCreateActionMenu);


	SGraphEditor::Construct(Arguments);

	SetNodeFactory( MakeShared<FDataflowGraphNodeFactory>(this) );
	GetGraphPanel()->SetZoomLevelsContainer<DataflowGraphEditor::Private::FDataflowZoomLevelsContainer>();

	InitGraphEditorMessageBar();
	InitEvaluationProgressBar();

	// Take the existing graph panel widget and add it to a new SOverlay so we can place the message bar over top of it
	if (ChildSlot.Num() > 0)
	{
		TSharedRef<SWidget> ChildWidget = ChildSlot.GetChildAt(0);

		this->ChildSlot
		[
			SNew(SOverlay)
				+ SOverlay::Slot()
				[
					ChildWidget
				]
				+ SOverlay::Slot()
				[
					MessageBar.ToSharedRef()
				]
				+ SOverlay::Slot()
				[
					EvaluationProgressBar.ToSharedRef()
				]
		];
	}
}

SDataflowGraphEditor::~SDataflowGraphEditor()
{
	if (IConsoleVariable* const ConsoleVar = IConsoleManager::Get().FindConsoleVariable(TEXT("p.Dataflow.EnableGraphEval")))
	{
		ConsoleVar->OnChangedDelegate().Remove(CVarChangedDelegateHandle);
	}
}

void SDataflowGraphEditor::SetLogMessageStorage(TSharedPtr<UE::Dataflow::Private::FLogMessageStorage> InLogMessages)
{
	LogMessages = InLogMessages;
}

bool SDataflowGraphEditor::IsEditable() const
{
	return IsEditableCallback ? IsEditableCallback() : false;
}

FActionMenuContent SDataflowGraphEditor::OnCreateActionMenu(UEdGraph* Graph, const FVector2f& Position, const TArray<UEdGraphPin*>& DraggedPins, bool bAutoExpandActionMenu, SGraphEditor::FActionMenuClosed OnClosed)
{
	LastActionMenuGraphEditor = StaticCastSharedRef<SDataflowGraphEditor>(AsShared()).ToWeakPtr();

	TSharedRef<SGraphEditorActionMenu> ActionMenu =
		SNew(SGraphEditorActionMenu)
		.GraphObj(EdGraphObj)
		.NewNodePosition(FVector2f{ (float)Position.X, (float)Position.Y })
		.DraggedFromPins(DraggedPins)
		.AutoExpandActionMenu(bAutoExpandActionMenu)
		.OnClosedCallback(OnClosed)
		.OnCreateWidgetForAction(SGraphActionMenu::FOnCreateWidgetForAction::CreateLambda(
			[](const FCreateWidgetForActionData* const Data)
			{
				return SNew(SDataflowGraphActionWidget, Data);
			}));

	TSharedRef<SWidget> ActionMenuWithOptions =
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.Padding(2)
		.AutoHeight()
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Right)
			.AutoWidth()
			.Padding(2.f)
			[
				SNew(SCheckBox)
				.OnCheckStateChanged(this, &SDataflowGraphEditor::OnActionMenuFilterByAssetTypeChanged, ActionMenu.ToWeakPtr())
				.IsChecked(this, &SDataflowGraphEditor::IsActionMenuFilterByAssetTypeChecked)
			]

			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			.AutoWidth()
			.Padding(2.f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("DataflowActionMenu_FilterByAssetType", "Filter by asset type"))
			]
		]
		+ SVerticalBox::Slot()
		.Padding(2)
		.AutoHeight()
		[
			ActionMenu
		];

	return FActionMenuContent(ActionMenuWithOptions, ActionMenu->GetFilterTextBox());
}

void SDataflowGraphEditor::OnActionMenuFilterByAssetTypeChanged(ECheckBoxState NewState, const TWeakPtr<SGraphEditorActionMenu> WeakActionMenu)
{
	bFilterActionMenyByAssetType = (NewState == ECheckBoxState::Checked);

	if (TSharedPtr<SGraphEditorActionMenu> ActionMenu = WeakActionMenu.Pin())
	{
		ActionMenu->RefreshAllActions();
	}
}

ECheckBoxState SDataflowGraphEditor::IsActionMenuFilterByAssetTypeChecked() const
{
	return bFilterActionMenyByAssetType ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

TSharedPtr<UE::Dataflow::FContext> SDataflowGraphEditor::GetDataflowContext() const
{
	if (DataflowEditor)
	{
		if (DataflowEditor->GetEditorContent())
		{
			return DataflowEditor->GetEditorContent()->GetDataflowContext();
		}
	}
	return TSharedPtr<UE::Dataflow::FContext>();
}

UDataflow* SDataflowGraphEditor::GetDataflowAsset() const
{
	if (TStrongObjectPtr<UEdGraph> EdGraph = EdGraphWeakPtr.Pin())
	{
		return UDataflow::GetDataflowAssetFromEdGraph(EdGraph.Get());
	}
	return nullptr;
}

void SDataflowGraphEditor::OnRenderToggleChanged(UDataflowEdNode* UpdatedEdNode) const
{
	// deselect all the existing pinned nodes unless CTRL is pressed
	if (UpdatedEdNode->ShouldWireframeRenderNode() && !IsControlDown())
	{
		if (UEdGraph* EdGraph = GetCurrentGraph())
		{
			for (UEdGraphNode* EdNode : EdGraph->Nodes)
			{
				if (UDataflowEdNode* DataflowEdNode = Cast<UDataflowEdNode>(EdNode))
				{
					if (UpdatedEdNode != DataflowEdNode)
					{
						DataflowEdNode->SetShouldWireframeRenderNode(false);
					}
				}
			}
		}
	}
	// todo : if nothing is selected should we render the terminal node ?

	// need to refresh the UI by refreshing the selection
	FGraphPanelSelectionSet SelectionSet = GetSelectedNodes();
	GetGraphPanel()->SelectionManager.SetSelectionSet(SelectionSet);
}

FText SDataflowGraphEditor::GetMessageText(const UDataflowEdNode& Node) const
{
	FText MessageText;
	if (LogMessages)
	{
		if (TSharedPtr<const FDataflowNode> DataflowNode = Node.GetDataflowNode())
		{
			using namespace UE::Dataflow;
			TArray<FContext::FLogMessage> Messages;
			LogMessages->GetMessageForNode(*DataflowNode, Messages);

			for (const FContext::FLogMessage& Message : Messages)
			{
				MessageText = FText::Format(LOCTEXT("NodeLogMessageFormat", "{0}\n[{1}] - {2}"), MessageText, FTokenizedMessage::GetSeverityText(Message.Severity), Message.Text);
			}
		}
	}
	return MessageText;
}

EMessageSeverity::Type SDataflowGraphEditor::GetMessageMaxSeverity(const UDataflowEdNode& Node) const
{
	EMessageSeverity::Type OutSeverity = EMessageSeverity::Info;
	if (LogMessages)
	{
		if (TSharedPtr<const FDataflowNode> DataflowNode = Node.GetDataflowNode())
		{
			OutSeverity = LogMessages->GetMaxSeverityForNode(*DataflowNode);
		}
	}
	return OutSeverity;
}

void SDataflowGraphEditor::EvaluateNode()
{
	UE_LOGF(LogChaosDataflow, VeryVerbose, "SDataflowGraphEditor::EvaluateNode(): Nodes [%ls]",
		*FString::JoinBy(GetSelectedNodes().Array(), TEXT(", "), [](const UObject* SelectedNode)
			{
				return Cast<UDataflowEdNode>(SelectedNode) && Cast<UDataflowEdNode>(SelectedNode)->GetDataflowNode() ? 
					Cast<UDataflowEdNode>(SelectedNode)->GetDataflowNode()->GetName().ToString() : FString();
			}));

	using namespace UE::Dataflow;

	for (UObject* Node : GetSelectedNodes())
	{
		if (UDataflowEdNode* const EdNode = Cast<UDataflowEdNode>(Node))
		{
			if (const TSharedPtr<FGraph> DataflowGraph = EdNode->GetDataflowGraph())
			{
				if (const TSharedPtr<FDataflowNode> DataflowNode = DataflowGraph->FindBaseNode(EdNode->GetDataflowNodeGuid()))
				{
					if (DataflowNode->IsActive())
					{
						DataflowNode->Invalidate();  // Force evaluation

						if (EvaluateGraphCallback)
						{
							EvaluateGraphCallback(DataflowNode.Get(), nullptr);  // Evaluation processes all outputs when passing a null Output
						}
						else
						{
							FContextThreaded DefaultContext;
							DefaultContext.Evaluate(DataflowNode.Get(), nullptr);
						}
					}
				}
			}
		}
	}
}

void SDataflowGraphEditor::FreezeNodes()
{
	if (UE::Dataflow::FContext* const DataflowContext = GetDataflowContext().Get())
	{
		FDataflowEditorCommands::FreezeNodes(*DataflowContext, GetSelectedNodes());
	}
}

void SDataflowGraphEditor::UnfreezeNodes()
{
	if (UE::Dataflow::FContext* const DataflowContext = GetDataflowContext().Get())
	{
		FDataflowEditorCommands::UnfreezeNodes(*DataflowContext, GetSelectedNodes());
	}
}

void SDataflowGraphEditor::DeleteNode()
{
	if (TStrongObjectPtr<UEdGraph> EdGraph = EdGraphWeakPtr.Pin())
	{
		if (DetailsView)
		{
			DetailsView->SetStructureData(nullptr);
		}

		const FGraphPanelSelectionSet& SelectedNodes = GetSelectedNodes();
		if (SelectedNodes.Num() > 0)
		{
			FDataflowEditorCommands::DeleteNodes(EdGraph.Get(), SelectedNodes);
			OnNodeDeletedMulticast.Broadcast(SelectedNodes);
		}
	}
}

void SDataflowGraphEditor::RenameNode()
{
	if (TStrongObjectPtr<UEdGraph> EdGraph = EdGraphWeakPtr.Pin())
	{
		const TSharedPtr<SDataflowGraphEditor>& DataflowGraphEditor = SharedThis(this);
		const FGraphPanelSelectionSet& SelectedNodes = GetSelectedNodes();

		if (SelectedNodes.Num() == 1)
		{
			if (CanRenameNode())
			{
				if (UDataflowEdNode* SelectedNode = Cast<UDataflowEdNode>(*SelectedNodes.CreateConstIterator()))
				{
					FDataflowEditorCommands::RenameNode(DataflowGraphEditor, SelectedNode);
				}
				else if (UEdGraphNode_Comment* SelectedCommentNode = Cast<UEdGraphNode_Comment>(*SelectedNodes.CreateConstIterator()))
				{
					FDataflowEditorCommands::RenameNode(DataflowGraphEditor, SelectedCommentNode);
				}
			}
		}
	}
}

bool SDataflowGraphEditor::CanRenameNode() const
{
	if (!IsEditable())
	{
		return false;
	}
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	if (SelectedNodes.Num() == 1)
	{
		if (UDataflowEdNode* SelectedNode = Cast<UDataflowEdNode>(*SelectedNodes.CreateConstIterator()))
		{
			return SelectedNode->bCanRenameNode;
		}
		else if (UEdGraphNode_Comment* SelectedCommentNode = Cast<UEdGraphNode_Comment>(*SelectedNodes.CreateConstIterator()))
		{
			return SelectedCommentNode->bCanRenameNode;
		}
	}

	return false;
}

void SDataflowGraphEditor::AddNewVariable() const
{
	if (UDataflow* DataflowAsset = GetDataflowAsset())
	{
		UE::Dataflow::FEditAssetUtils::AddNewVariable(DataflowAsset);
	}
}

void SDataflowGraphEditor::AddNewSubGraph() const
{
	if (UDataflow* DataflowAsset = GetDataflowAsset())
	{
		const FName NewSubGraphName = UE::Dataflow::FEditAssetUtils::AddNewSubGraph(DataflowAsset);
		if (UDataflowSubGraph* SubGraph = DataflowAsset->FindSubGraphByName(NewSubGraphName))
		{
			static const FName InputNodeName = "Input";
			static const FVector2D InputNodePos { -100, 0 };
			UE::Dataflow::FEditAssetUtils::AddNewNode(SubGraph, InputNodePos, InputNodeName, FDataflowSubGraphInputNode::StaticType(), nullptr);

			static const FName OutputNodeName = "Output";
			static const FVector2D OutputNodePos{ +100, 0 };
			UE::Dataflow::FEditAssetUtils::AddNewNode(SubGraph, OutputNodePos, OutputNodeName, FDataflowSubGraphOutputNode::StaticType(), nullptr);
		}
	}
}

void SDataflowGraphEditor::OnSelectedNodesChanged(const TSet<UObject*>& NewSelection)
{
	// Set the currently selected graph editor before running any callback
	ensureMsgf(!SelectedGraphEditor.IsValid(), TEXT("Two different editors cannot have their selection changed at once."));
	SelectedGraphEditor = StaticCastSharedRef<SDataflowGraphEditor>(AsShared()).ToWeakPtr();

	OnSelectionChangedMulticast.Broadcast(NewSelection);

	if (DetailsView)
	{
		if (UDataflow* DataflowAsset = GetDataflowAsset())
		{
			auto AsObjectPointers = [](const TSet<UObject*>& Set) {
				TSet<TObjectPtr<UObject> > Objs; for (UObject* Elem : Set) Objs.Add(Elem);
				return Objs;
				};

			FDataflowEditorCommands::OnSelectedNodesChanged(DetailsView, AssetOwner.Get(), DataflowAsset, AsObjectPointers(NewSelection));
		}
	}

	// Clear the current selected editor
	SelectedGraphEditor.Reset();
}

FReply SDataflowGraphEditor::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (InKeyEvent.GetKey() == EKeys::LeftControl)
	{
		LeftControlKeyDown = true;
	}
	if (InKeyEvent.GetKey() == EKeys::RightControl)
	{
		RightControlKeyDown = true;
	}
	if (InKeyEvent.GetKey() == EKeys::LeftAlt)
	{
		LeftAltKeyDown = true;
	}
	if (InKeyEvent.GetKey() == EKeys::RightAlt)
	{
		RightAltKeyDown = true;
	}
	if (InKeyEvent.GetKey() == EKeys::V)
	{
		VKeyDown = true;
	}
	return SGraphEditor::OnKeyDown(MyGeometry, InKeyEvent);
}

bool SDataflowGraphEditor::IsControlDown() const
{
	return LeftControlKeyDown || RightControlKeyDown;
}

bool SDataflowGraphEditor::IsAltDown() const
{
	return LeftAltKeyDown || RightAltKeyDown;
}




FReply SDataflowGraphEditor::OnKeyUp(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (InKeyEvent.GetKey() == EKeys::LeftControl)
	{
		LeftControlKeyDown = false;
	}
	if (InKeyEvent.GetKey() == EKeys::RightControl)
	{
		RightControlKeyDown = false;
	}
	if (InKeyEvent.GetKey() == EKeys::LeftAlt)
	{
		LeftAltKeyDown = false;
	}
	if (InKeyEvent.GetKey() == EKeys::RightAlt)
	{
		RightAltKeyDown = false;
	}
	if (InKeyEvent.GetKey() == EKeys::V)
	{
		VKeyDown = false;
	}
	if (InKeyEvent.GetKey() == EKeys::LeftControl)
	{
		return FReply::Unhandled();
	}
	return SGraphEditor::OnKeyUp(MyGeometry, InKeyEvent);
}


FReply SDataflowGraphEditor::OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	if (TSharedPtr<FBoneDragDropOp> SchemeDragDropOp = DragDropEvent.GetOperationAs<FBoneDragDropOp>())
	{
		return FReply::Handled();
	}
	return FReply::Unhandled();
}


FReply SDataflowGraphEditor::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	if (TSharedPtr<FBoneDragDropOp> SchemeDragDropOp = DragDropEvent.GetOperationAs<FBoneDragDropOp>())
	{
		if (OnDragDropEventCallback)
		{
			OnDragDropEventCallback(MyGeometry,DragDropEvent);
		}
	}
	return SGraphEditor::OnDrop(MyGeometry, DragDropEvent);
}

/*
void SDataflowGraphEditor::OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{

	TSharedPtr<FGraphSchemaActionDragDropAction> SchemeDragDropOp = DragDropEvent.GetOperationAs<FGraphSchemaActionDragDropAction>();
	{
	//	FDataprepSchemaActionContext ActionContext;
	//	ActionContext.DataprepActionPtr = StepData->DataprepActionPtr;
	//	ActionContext.DataprepActionStepPtr = StepData->DataprepActionStepPtr;
	//	ActionContext.StepIndex = StepData->StepIndex;
	//	DataprepDragDropOp->SetHoveredDataprepActionContext(ActionContext);
	}
	UE_LOGF(SDataflowGraphEditorLog, All, "SDataflowGraphEditor::OnDragEnter");

}
void SDataflowGraphEditor::OnDragLeave(const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FGraphSchemaActionDragDropAction> SchemeDragDropOp = DragDropEvent.GetOperationAs<FGraphSchemaActionDragDropAction>();
	{
		// DataflowDragDropOp->SetHoveredDataprepActionContext(TOptional<FDataprepSchemaActionContext>());
	}
	UE_LOGF(SDataflowGraphEditorLog, All, "SDataflowGraphEditor::OnDragLeave");
}
*/

void SDataflowGraphEditor::CreateComment()
{
	if (TStrongObjectPtr<UEdGraph> EdGraph = EdGraphWeakPtr.Pin())
	{
		const TSharedPtr<SGraphEditor>& InGraphEditor = SharedThis(GetGraphEditor());

		TSharedPtr<FAssetSchemaAction_Dataflow_CreateCommentNode_DataflowEdNode> CommentAction = FAssetSchemaAction_Dataflow_CreateCommentNode_DataflowEdNode::CreateAction(EdGraph.Get(), InGraphEditor);
		CommentAction->PerformAction(EdGraph.Get(), nullptr, GetGraphEditor()->GetPasteLocation2f(), false);
	}
}

void SDataflowGraphEditor::AlignTop()
{
	GetGraphEditor()->OnAlignTop();
}

void SDataflowGraphEditor::AlignMiddle()
{
	GetGraphEditor()->OnAlignMiddle();
}

void SDataflowGraphEditor::AlignBottom()
{
	GetGraphEditor()->OnAlignBottom();
}

void SDataflowGraphEditor::AlignLeft()
{
	GetGraphEditor()->OnAlignLeft();
}

void SDataflowGraphEditor::AlignCenter()
{
	GetGraphEditor()->OnAlignCenter();
}

void SDataflowGraphEditor::AlignRight()
{
	GetGraphEditor()->OnAlignRight();
}

void SDataflowGraphEditor::StraightenConnections()
{
	// somehow the StraightenConnections menu entry does not seems to accound for the editable state of the graph and remain accessible despite the readonly state 
	// to protect from any unvoluntary change to the graph let's make sure we ignore the command if iot still goes through 
	if (IsEditable())
	{
		GetGraphEditor()->OnStraightenConnections();
	}
}

void SDataflowGraphEditor::DistributeHorizontally()
{
	GetGraphEditor()->OnDistributeNodesH();
}

void SDataflowGraphEditor::DistributeVertically()
{
	GetGraphEditor()->OnDistributeNodesV();
}

void SDataflowGraphEditor::ToggleEnabledState()
{
	if (UDataflow* DataflowAsset = GetDataflowAsset())
	{
		FDataflowEditorCommands::ToggleEnabledState(DataflowAsset);
	}
}

void SDataflowGraphEditor::OnAddOptionPin()
{
	if (TStrongObjectPtr<UEdGraph> EdGraph = EdGraphWeakPtr.Pin())
	{
		if (UDataflow* DataflowAsset = UDataflow::GetDataflowAssetFromEdGraph(EdGraph.Get()))
		{
			FDataflowAssetEdit Edit = DataflowAsset->EditDataflow();
			if (UE::Dataflow::FGraph* const DataflowGraph = Edit.GetGraph())
			{
				const FGraphPanelSelectionSet& SelectedNodes = GetSelectedNodes();

				// Iterate over all nodes, and add the pin
				for (FGraphPanelSelectionSet::TConstIterator It(SelectedNodes); It; ++It)
				{
					UDataflowEdNode* const EdNode = CastChecked<UDataflowEdNode>(*It);

					if (const TSharedPtr<FDataflowNode> Node = DataflowGraph->FindBaseNode(EdNode->DataflowNodeGuid))
					{
						if (Node->CanAddPin())
						{
							const FScopedTransaction Transaction(LOCTEXT("AddOptionPin", "Add Option Pin"));
							DataflowAsset->Modify();
							EdGraph->Modify();
							EdNode->Modify();

							EdNode->AddOptionPin();

							const UDataflowSchema* const Schema = CastChecked<UDataflowSchema>(EdGraph->GetSchema());
							Schema->ReconstructNode(*EdNode);
						}
					}
				}
			}
		}
	}
}

bool SDataflowGraphEditor::CanAddOptionPin() const
{
	bool bCanAddOptionPin = false;

	if (TStrongObjectPtr<UEdGraph> EdGraph = EdGraphWeakPtr.Pin())
	{
		if (UDataflow* DataflowAsset = UDataflow::GetDataflowAssetFromEdGraph(EdGraph.Get()))
		{
			if (const UE::Dataflow::FGraph* const DataflowGraph = DataflowAsset->GetDataflow().Get())
			{
				const FGraphPanelSelectionSet& SelectedNodes = GetSelectedNodes();

				// Iterate over all nodes, and add the pin
				for (FGraphPanelSelectionSet::TConstIterator It(SelectedNodes); It; ++It)
				{
					if (const UDataflowEdNode* const EdNode = Cast<UDataflowEdNode>(*It))
					{
						if (const TSharedPtr<const FDataflowNode> Node = DataflowGraph->FindBaseNode(EdNode->DataflowNodeGuid))
						{
							bCanAddOptionPin = Node->CanAddPin();
						}
						else
						{
							bCanAddOptionPin = false;
						}

						if (!bCanAddOptionPin)
						{
							break;  // One bad node is good enough to return false
						}
					}
				}
			}
		}
	}

	return bCanAddOptionPin && IsEditable();
}

void SDataflowGraphEditor::OnRemoveOptionPin()
{
	if (TStrongObjectPtr<UEdGraph> EdGraph = EdGraphWeakPtr.Pin())
	{
		if (UDataflow* DataflowAsset = UDataflow::GetDataflowAssetFromEdGraph(EdGraph.Get()))
		{
			FDataflowAssetEdit Edit = DataflowAsset->EditDataflow();
			if (UE::Dataflow::FGraph* const DataflowGraph = Edit.GetGraph())
			{
				const FGraphPanelSelectionSet& SelectedNodes = GetSelectedNodes();

				// Iterate over all nodes, and remove a pin
				for (FGraphPanelSelectionSet::TConstIterator It(SelectedNodes); It; ++It)
				{
					UDataflowEdNode* const EdNode = CastChecked<UDataflowEdNode>(*It);

					if (const TSharedPtr<FDataflowNode> Node = DataflowGraph->FindBaseNode(EdNode->DataflowNodeGuid))
					{
						if (Node->CanRemovePin())
						{
							const FScopedTransaction Transaction(LOCTEXT("RemoveOptionPin", "Remove Option Pin"));
							DataflowAsset->Modify();
							EdGraph->Modify();
							EdNode->Modify();

							EdNode->RemoveOptionPin();

							const UDataflowSchema* const Schema = CastChecked<UDataflowSchema>(EdGraph->GetSchema());
							Schema->ReconstructNode(*EdNode);
						}
					}
				}
			}
		}
	}
}

bool SDataflowGraphEditor::CanRemoveOptionPin() const
{
	bool bCanRemoveOptionPin = false;

	if (TStrongObjectPtr<UEdGraph> EdGraph = EdGraphWeakPtr.Pin())
	{
		if (UDataflow* DataflowAsset = UDataflow::GetDataflowAssetFromEdGraph(EdGraph.Get()))
		{
			if (const UE::Dataflow::FGraph* const DataflowGraph = DataflowAsset->GetDataflow().Get())
			{
				const FGraphPanelSelectionSet& SelectedNodes = GetSelectedNodes();

				// Iterate over all nodes, and add the pin
				for (FGraphPanelSelectionSet::TConstIterator It(SelectedNodes); It; ++It)
				{
					if (const UDataflowEdNode* const EdNode = Cast<UDataflowEdNode>(*It))
					{
						if (const TSharedPtr<const FDataflowNode> Node = DataflowGraph->FindBaseNode(EdNode->DataflowNodeGuid))
						{
							bCanRemoveOptionPin = Node->CanRemovePin();
						}
						else
						{
							bCanRemoveOptionPin = false;
						}

						if (!bCanRemoveOptionPin)
						{
							break;  // One bad node is good enough to return false
						}
					}
				}
			}
		}
	}

	return bCanRemoveOptionPin && IsEditable();
}

void SDataflowGraphEditor::DuplicateSelectedNodes()
{
	if (TStrongObjectPtr<UEdGraph> EdGraph = EdGraphWeakPtr.Pin())
	{
		const TSharedPtr<SDataflowGraphEditor>& DataflowGraphEditor = SharedThis(this);
		const FGraphPanelSelectionSet& SelectedNodes = GetSelectedNodes();

		if (SelectedNodes.Num() > 0)
		{
			FDataflowEditorCommands::DuplicateNodes(EdGraph.Get(), DataflowGraphEditor, SelectedNodes);
		}
	}
}

void SDataflowGraphEditor::ZoomToFitGraph()
{
	constexpr bool bOnlySelection = true;	// This will focus on the selected nodes, if any. If no nodes are selected, it will focus the whole graph.
	ZoomToFit(bOnlySelection);
}

bool SDataflowGraphEditor::GetPinVisibility(SGraphEditor::EPinVisibility PinVisibility) const
{
	if (const SGraphPanel* GraphPanel = GetGraphPanel())
	{
		return GraphPanel->GetPinVisibility() == PinVisibility;
	}
	return false;
}

void SDataflowGraphEditor::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(EdGraphWeakPtr);
	Collector.AddReferencedObject(AssetOwner);
}

void SDataflowGraphEditor::CopySelectedNodes()
{
	if (TStrongObjectPtr<UEdGraph> EdGraph = EdGraphWeakPtr.Pin())
	{
		const TSharedPtr<SDataflowGraphEditor>& DataflowGraphEditor = SharedThis(this);
		const FGraphPanelSelectionSet& SelectedNodes = GetSelectedNodes();

		if (SelectedNodes.Num() > 0)
		{
			FDataflowEditorCommands::CopyNodes(EdGraph.Get(), DataflowGraphEditor, SelectedNodes);
		}
	}
}

void SDataflowGraphEditor::CutSelectedNodes()
{
	if (TStrongObjectPtr<UEdGraph> EdGraph = EdGraphWeakPtr.Pin())
	{
		const TSharedPtr<SDataflowGraphEditor>& DataflowGraphEditor = SharedThis(this);
		const FGraphPanelSelectionSet& SelectedNodes = GetSelectedNodes();

		if (SelectedNodes.Num() > 0)
		{
			FDataflowEditorCommands::CopyNodes(EdGraph.Get(), DataflowGraphEditor, SelectedNodes);

			FDataflowEditorCommands::DeleteNodes(EdGraph.Get(), SelectedNodes);
		}
	}
}

void SDataflowGraphEditor::PasteSelectedNodes()
{
	if (TStrongObjectPtr<UEdGraph> EdGraph = EdGraphWeakPtr.Pin())
	{
		const TSharedPtr<SDataflowGraphEditor>& DataflowGraphEditor = SharedThis(this);

		FDataflowEditorCommands::PasteNodes(EdGraph.Get(), DataflowGraphEditor);
	}
}

void SDataflowGraphEditor::OnStartWatchingPin()
{
	if (const UEdGraphPin* Pin = GetGraphPinForMenu())
	{
		if (UDataflowEdNode* EdNode = Cast<UDataflowEdNode>(Pin->GetOwningNode()))
		{
			EdNode->WatchPin(Pin, true);
		}
	}
}

bool SDataflowGraphEditor::CanStartWatchingPin() const
{
	return !CanStopWatchingPin();
}

void SDataflowGraphEditor::OnStopWatchingPin()
{
	if (const UEdGraphPin* Pin = GetGraphPinForMenu())
	{
		if (UDataflowEdNode* EdNode = Cast<UDataflowEdNode>(Pin->GetOwningNode()))
		{
			EdNode->WatchPin(Pin, false);
		}
	}
}

bool SDataflowGraphEditor::CanStopWatchingPin() const
{
	if (UEdGraphPin* Pin = const_cast<SDataflowGraphEditor*>(this)->GetGraphPinForMenu())
	{
		if (const UDataflowEdNode* EdNode = Cast<UDataflowEdNode>(Pin->GetOwningNode()))
		{
			return EdNode->IsPinWatched(Pin);
		}
	}
	return false;
}

void SDataflowGraphEditor::OnPromoteToVariable()
{
	if (UEdGraphPin* Pin = GetGraphPinForMenu())
	{
		if (UDataflowEdNode* EdNode = Cast<UDataflowEdNode>(Pin->GetOwningNode()))
		{
			if (UDataflow* DataflowAsset = GetDataflowAsset())
			{
				FDataflowInput* DataflowInput = nullptr;
				if (TSharedPtr<FDataflowNode> DataflowNode = EdNode->GetDataflowNode())
				{
					DataflowInput = DataflowNode->FindInput(Pin->PinName);
				}
				if (DataflowInput)
				{
					// create the variable from the input type
					using namespace UE::Dataflow::Type;
					const FPropertyBagPropertyDesc TemplateDesc = GetPropertyBagPropertyDescFromDataflowConnection(*DataflowInput);
					const FName NewVariableName = UE::Dataflow::FEditAssetUtils::AddNewVariable(DataflowAsset, Pin->PinName, TemplateDesc);

					// add the variable node
					if (UEdGraph* ParentGraph = EdNode->GetGraph())
					{
						if (TSharedPtr<FAssetSchemaAction_Dataflow_CreateNode_DataflowEdNode>  VariableNodeAction = FAssetSchemaAction_Dataflow_CreateNode_DataflowEdNode::CreateAction(ParentGraph, "FGetDataflowVariableNode", NewVariableName))
						{
							const FVector2f Location = EdNode->GetPosition() - FVector2f(200, 0);
							UDataflowEdNode* VariableEdNode = Cast<UDataflowEdNode>(VariableNodeAction->PerformAction(ParentGraph, nullptr, Location, false));
							if (VariableEdNode)
							{
								if (TSharedPtr<FGetDataflowVariableNode> VariableNode = StaticCastSharedPtr<FGetDataflowVariableNode>(VariableEdNode->GetDataflowNode()))
								{
									VariableNode->SetVariable(DataflowAsset, NewVariableName);
									VariableEdNode->UpdatePinsFromDataflowNode();
									ParentGraph->NotifyNodeChanged(VariableEdNode);
								}

								// now connect the nodes 
								// there's only pin on a variable node
								if (UEdGraphPin* VariableNodePin = VariableEdNode->GetPinAt(0))
								{
									ParentGraph->GetSchema()->TryCreateConnection(VariableNodePin, Pin);
								}

								ParentGraph->NotifyGraphChanged();
							}

						}
					}
				}
			}
		}
	}
}

bool SDataflowGraphEditor::CanPromoteToVariable() const
{
	if (UEdGraphPin* Pin = const_cast<SDataflowGraphEditor*>(this)->GetGraphPinForMenu())
	{
		return !(Pin->HasAnyConnections()) && IsEditable();
	}
	return false;
}

void SDataflowGraphEditor::InitGraphEditorMessageBar()
{
	SAssignNew(MessageBar, SHorizontalBox)
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Bottom)
		.FillWidth(1.0f)
		.Padding(FMargin(0.0f, 0.0f, 0.f, 15.f))
		[
			SNew(SBorder)
				.BorderImage(FCoreStyle::Get().GetBrush(TEXT("BlackBrush")))
				.BorderBackgroundColor(FLinearColor::Red)
				.Padding(FMargin(20.f, 5.0f, 20.f, 5.0f))
				[
					SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.Padding(FMargin(0.f, 0.f, 0.f, 0.f))
						.FillWidth(1.0f)
						[
							SNew(STextBlock)
								.Text(this, &SDataflowGraphEditor::GetGraphEditorOverlayText)
								.Font(FCoreStyle::GetDefaultFontStyle("Bold", 14))
						]
				]
		];

	auto UpdateGraphForEvalEnabled = [this](bool bGraphEvalEnabled)
	{
		if (EdGraphObj)
		{
			for (UEdGraphNode* const Node : EdGraphObj->Nodes)
			{
				Node->SetForceDisplayAsDisabled(!bGraphEvalEnabled);
			}
		}

		if (MessageBar)
		{
			MessageBar->SetVisibility(bGraphEvalEnabled ? EVisibility::Collapsed : EVisibility::SelfHitTestInvisible);
		}
		MessageBarText = bGraphEvalEnabled ? FText() : LOCTEXT("DataflowGraphEditorOverlayTextPaused", "GRAPH EVALUATION PAUSED");
	};

	IConsoleVariable* const ConsoleVar = IConsoleManager::Get().FindConsoleVariable(TEXT("p.Dataflow.EnableGraphEval"));
	const bool bDataflowEnableGraphEval = ConsoleVar ? ConsoleVar->GetBool() : true;
	UpdateGraphForEvalEnabled(bDataflowEnableGraphEval);

	if (ConsoleVar)
	{
		CVarChangedDelegateHandle = ConsoleVar->OnChangedDelegate().AddLambda([this, UpdateGraphForEvalEnabled](IConsoleVariable* Var)
		{
			const bool bEvalEnabled = Var->GetBool();
			UpdateGraphForEvalEnabled(bEvalEnabled);
		});
	}
}

void SDataflowGraphEditor::InitEvaluationProgressBar()
{
	const FSlateBrush* OverlayBrush = FAppStyle::Get().GetBrush("EditorViewport.OverlayBrush");

	auto GetBarVisibility = [this]() ->EVisibility
		{
			bool bVisible = false;
			if (TSharedPtr<UE::Dataflow::FContext> Context = GetDataflowContext())
			{
				bVisible = Context->IsAsyncEvaluating();
			}
			if (bVisible == false && DataflowEditor)
			{
				if (TObjectPtr<UDataflowBaseContent> EditorContent = DataflowEditor->GetEditorContent())
				{
					if (TSharedPtr<UE::Dataflow::IDataflowEvaluator> Evaluator = EditorContent->GetDataflowEvalutor())
					{
						bVisible = Evaluator->IsRunning();
					}
				}
			}
			return bVisible ? EVisibility::SelfHitTestInvisible : EVisibility::Collapsed;
		};

	auto GetProgressText = [this]() -> FText
		{
			FText ProgressText;
			if (TSharedPtr<UE::Dataflow::FContext> Context = GetDataflowContext())
			{
				int32 NumPendingTasks = 0;
				int32 NumRunningTasks = 0;
				int32 NumCompletedTasks = 0;
				Context->GetAsyncEvaluationStats(NumPendingTasks, NumRunningTasks, NumCompletedTasks);

				ProgressText = FText::Format(
					LOCTEXT("DataflowGraphEditor_ProgressText", "Pending: {0} | Running: {1} | Completed: {2}"),
					NumPendingTasks,
					NumRunningTasks,
					NumCompletedTasks
				);
				if (NumPendingTasks == 0)
				{
					if (TObjectPtr<UDataflowBaseContent> EditorContent = DataflowEditor->GetEditorContent())
					{
						if (TSharedPtr<UE::Dataflow::IDataflowEvaluator> Evaluator = EditorContent->GetDataflowEvalutor())
						{
							ProgressText = Evaluator->GetProgressMessage();
						}
					}
				}
			}
			return ProgressText;
		};

	auto GetProgressPercent = [this]() -> float
		{
			float Percent = 0.f;
			if (TSharedPtr<UE::Dataflow::FContext> Context = GetDataflowContext())
			{
				int32 NumPendingTasks = 0;
				int32 NumRunningTasks = 0;
				int32 NumCompletedTasks = 0;
				Context->GetAsyncEvaluationStats(NumPendingTasks, NumRunningTasks, NumCompletedTasks);

				const int32 TotalTasks = (NumPendingTasks + NumRunningTasks + NumCompletedTasks);
				if (TotalTasks > 0)
				{
					Percent = static_cast<float>(NumCompletedTasks) / static_cast<float>(TotalTasks);
				}

				if (NumPendingTasks == 0)
				{
					if (TObjectPtr<UDataflowBaseContent> EditorContent = DataflowEditor->GetEditorContent())
					{
						if (TSharedPtr<UE::Dataflow::IDataflowEvaluator> Evaluator = EditorContent->GetDataflowEvalutor())
						{
							Percent = Evaluator->GetProgress();
						}
					}
				}
			}
			return Percent;
		};

	auto OnCancel = [this]() -> FReply
		{
			if (TSharedPtr<UE::Dataflow::FContext> Context = GetDataflowContext())
			{
				Context->CancelAsyncEvaluation();
			}
			return FReply::Handled();
		};

	SAssignNew(EvaluationProgressBar, SHorizontalBox)
	.Visibility_Lambda(GetBarVisibility)

	+SHorizontalBox::Slot()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Bottom)
		.FillWidth(1.0)
		.Padding(FMargin(0.0f, 0.0f, 0.f, 15.f))
		[
			SNew(SBorder)
			.BorderImage(OverlayBrush)
			[
				SNew(SHorizontalBox)
				
				// Text + progress bar 
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(8)
				[
					SNew(SVerticalBox)

					// Progress text
					+ SVerticalBox::Slot()
					.VAlign(VAlign_Center)
					.Padding(4)
					[
						SNew(STextBlock)
						.Text_Lambda(GetProgressText)
					]

					// Progress bar 
					+ SVerticalBox::Slot()
					.VAlign(VAlign_Center)
					.Padding(4)
					[
						SNew(SProgressBar)
						.Percent_Lambda(GetProgressPercent)
					]
				]

				// Cancel button
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(8)
				[
					SNew(SButton)
					.Text(LOCTEXT("DataflowGraphEditor_CancelText", "Cancel"))
					.ToolTipText(LOCTEXT("DataflowGraphEditor_CancelTooltip", "Cancel current evaluation"))
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.OnClicked_Lambda(OnCancel)
				]
			]
		];
}

FText SDataflowGraphEditor::GetGraphEditorOverlayText() const
{
	return MessageBarText;
}

#undef LOCTEXT_NAMESPACE
