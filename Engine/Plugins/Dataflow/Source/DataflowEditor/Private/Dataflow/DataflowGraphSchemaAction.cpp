// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowGraphSchemaAction.h"

#include "Dataflow/DataflowAssetEditUtils.h"
#include "Dataflow/DataflowInstance.h"
#include "Dataflow/DataflowObject.h"
#include "Dataflow/DataflowSNode.h"
#include "Dataflow/DataflowSubGraph.h"
#include "Dataflow/DataflowAssetEditUtils.h"
#include "Dataflow/DataflowVariableNodes.h"
#include "Dataflow/DataflowSubGraphNodes.h"
#include "EdGraph/EdGraphSchema.h"
#include "PropertyBagDetails.h"
#include "StructUtils/PropertyBag.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowGraphSchemaAction)


#define LOCTEXT_NAMESPACE "DataflowGraphSchemaAction"

/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<FAssetSchemaAction_Dataflow_CreateNode_DataflowEdNode> FAssetSchemaAction_Dataflow_CreateNode_DataflowEdNode::CreateAction(const UEdGraph* ParentGraph, const FName& InNodeTypeName, const FName& InOverrideNodeName)
{
	if (const UDataflow* Dataflow = UDataflow::GetDataflowAssetFromEdGraph(ParentGraph))
	{
		if (UE::Dataflow::FNodeFactory* Factory = UE::Dataflow::FNodeFactory::GetInstance())
		{
			const UE::Dataflow::FFactoryParameters& Param = Factory->GetParameters(InNodeTypeName);
			if (Param.IsValid())
			{
				const bool bIsSimulationNode = Param.Tags.Contains(UDataflow::SimulationTag);
				const bool bIsSimulationGraph = (Dataflow->Type == EDataflowType::Simulation);

				if ((bIsSimulationGraph && bIsSimulationNode) || (!bIsSimulationGraph && !bIsSimulationNode))
				{
					const FText ToolTip = FText::FromString(Param.ToolTip.IsEmpty() ? FString("Add a Dataflow node.") : Param.ToolTip);
					const FName NodeName = !InOverrideNodeName.IsNone() ? InOverrideNodeName : Param.DisplayName;
					const FText MenuDesc = Param.bIsExperimental ?
						FText::FromString(NodeName.ToString() + TEXT(" (Experimental)")) :
						FText::FromName(NodeName);

					const FText Category = FText::FromString(Param.Category.ToString().IsEmpty() ? FString("Dataflow") : Param.Category.ToString());
					const FText Tags = FText::FromString(Param.Tags);

					FSlateIcon Icon;
					if (Param.DefaultNodeObject)
					{
						Icon = UDataflowEdNode::GetIconFromNode(*Param.DefaultNodeObject);
					}

					TSharedPtr<FAssetSchemaAction_Dataflow_CreateNode_DataflowEdNode> NewNodeAction(
						new FAssetSchemaAction_Dataflow_CreateNode_DataflowEdNode(NodeName, InNodeTypeName, Category, MenuDesc, ToolTip, Tags, Icon));
					return NewNodeAction;
				}
			}
		}
	}
	return TSharedPtr<FAssetSchemaAction_Dataflow_CreateNode_DataflowEdNode>(nullptr);
}

FAssetSchemaAction_Dataflow_CreateNode_DataflowEdNode::FAssetSchemaAction_Dataflow_CreateNode_DataflowEdNode(const FName& InName, const FName& InType, FText InNodeCategory, FText InMenuDesc, FText InToolTip, FText InKeywords, FName InIconName)
	: FEdGraphSchemaAction(InNodeCategory, InMenuDesc, InToolTip, 0, InKeywords)
	, NodeName(InName)
	, NodeTypeName(InType)
{
	Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), InIconName);
}

FAssetSchemaAction_Dataflow_CreateNode_DataflowEdNode::FAssetSchemaAction_Dataflow_CreateNode_DataflowEdNode(const FName& InName, const FName& InType, FText InNodeCategory, FText InMenuDesc, FText InToolTip, FText InKeywords, const FSlateIcon& InIcon)
	: FEdGraphSchemaAction(InNodeCategory, InMenuDesc, InToolTip, 0, InKeywords)
	, NodeName(InName)
	, NodeTypeName(InType)
	, Icon(InIcon)
{
}

UEdGraphNode* FAssetSchemaAction_Dataflow_CreateNode_DataflowEdNode::PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode)
{
	// by default use the type name 
	UEdGraphNode* NewNode = UE::Dataflow::FEditAssetUtils::AddNewNode(ParentGraph, FDeprecateSlateVector2D(Location), NodeName, NodeTypeName, FromPin);

	if (NewNode && bSelectNewNode)
	{
		ParentGraph->SelectNodeSet({ NewNode },  /*bFromUI*/true);
	}
	return NewNode;
}

FSlateBrush const* FAssetSchemaAction_Dataflow_CreateNode_DataflowEdNode::GetPaletteIcon() const
{
	return Icon.GetIcon();
}

/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FEdGraphSchemaAction_DataflowVariable::FEdGraphSchemaAction_DataflowVariable()
	: FEdGraphSchemaAction()
{}

FEdGraphSchemaAction_DataflowVariable::FEdGraphSchemaAction_DataflowVariable(UDataflow* InDataflowAsset, const FPropertyBagPropertyDesc& PropertyDesc)
	: FEdGraphSchemaAction(
		FText::FromString(CategoryFromFullName(PropertyDesc.Name)),
		FText::Format(LOCTEXT("DataflowVariableActionDescription", "Variable {0}"), FText::FromName(PropertyDesc.Name)),
		FText::Format(LOCTEXT("DataflowVariableActionTooltip", "Variable {0}"), FText::FromName(PropertyDesc.Name)),
		/*InGrouping*/0,
		/*Keywords*/FText::GetEmpty(),
		(int32)UE::Dataflow::ESchemaActionSectionID::VARIABLES)
	, FullVariableName(PropertyDesc.Name)
	, VariableName(NameFromFullName(PropertyDesc.Name))
	, VariableCategory(CategoryFromFullName(PropertyDesc.Name))
	, VariableType(UE::StructUtils::GetPropertyDescAsPin(PropertyDesc))
	, DataflowAssetWeakPtr(InDataflowAsset)
{
}

FString FEdGraphSchemaAction_DataflowVariable::CategoryFromFullName(FName FullName)
{
	const FString StrFullName(FullName.ToString());
	int32 DotIndex = 0;
	if (StrFullName.FindChar('|', DotIndex))
	{
		return StrFullName.Left(DotIndex);
	}
	return {};
}

FString FEdGraphSchemaAction_DataflowVariable::NameFromFullName(FName FullName)
{
	const FString StrFullName(FullName.ToString());
	int32 DotIndex = 0;
	if (StrFullName.FindChar('|', DotIndex))
	{
		return StrFullName.Right(StrFullName.Len() - DotIndex + 1);
	}
	return StrFullName;
}

bool FEdGraphSchemaAction_DataflowVariable::CanRenameItem(FText NewName) const
{
	if (TStrongObjectPtr<const UDataflow> DataflowAsset = DataflowAssetWeakPtr.Pin())
	{
		if (VariableName == NewName.ToString())
		{
			return true; // same value allowed
		}
		const FName NewVariableName(NewName.ToString());
		const FName SanitizedName = FInstancedPropertyBag::SanitizePropertyName(NewVariableName);
		if (SanitizedName != NewVariableName)
		{
			return false;
		}
		const bool bVariableAlreadyExists = (nullptr != DataflowAsset->Variables.FindPropertyDescByName(NewVariableName));
		return !bVariableAlreadyExists;
	}
	return false;
}

void FEdGraphSchemaAction_DataflowVariable::RenameItem(FText NewName)
{
	if (TStrongObjectPtr<UDataflow> DataflowAsset = DataflowAssetWeakPtr.Pin())
	{
		FName NewVariableName;
		if (VariableCategory.IsEmpty())
		{
			NewVariableName = FName(NewName.ToString());
		}
		else
		{
			NewVariableName = FName(FString::Format(TEXT("{0}|{1}"), { VariableCategory, NewName.ToString() }));
		}		
		UE::Dataflow::FEditAssetUtils::RenameVariable(DataflowAsset.Get(), FullVariableName, NewVariableName);
	}
}

void FEdGraphSchemaAction_DataflowVariable::CopyItemToClipboard()
{
	if (TStrongObjectPtr<UDataflow> DataflowAsset = DataflowAssetWeakPtr.Pin())
	{
		UE::Dataflow::FEditAssetUtils::CopyVariableToClipboard(DataflowAsset.Get(), GetFullVariableName());
	}
}

void FEdGraphSchemaAction_DataflowVariable::PasteItemFromClipboard()
{
	if (TStrongObjectPtr<UDataflow> DataflowAsset = DataflowAssetWeakPtr.Pin())
	{
		UE::Dataflow::FEditAssetUtils::PasteVariableFromClipboard(DataflowAsset.Get());
	}
}

void FEdGraphSchemaAction_DataflowVariable::SetVariableType(const FEdGraphPinType& PinType)
{
	if (TStrongObjectPtr<UDataflow> DataflowAsset = DataflowAssetWeakPtr.Pin())
	{
		UE::Dataflow::FEditAssetUtils::SetVariableType(DataflowAsset.Get(), FullVariableName, PinType);
	}
}

void FEdGraphSchemaAction_DataflowVariable::DeleteItem()
{
	if (TStrongObjectPtr<UDataflow> DataflowAsset = DataflowAssetWeakPtr.Pin())
	{
		UE::Dataflow::FEditAssetUtils::DeleteVariable(DataflowAsset.Get(), GetFullVariableName());
	}
}

void FEdGraphSchemaAction_DataflowVariable::DuplicateItem()
{
	if (TStrongObjectPtr<UDataflow> DataflowAsset = DataflowAssetWeakPtr.Pin())
	{
		UE::Dataflow::FEditAssetUtils::DuplicateVariable(DataflowAsset.Get(), GetFullVariableName());
	}
}

/** Execute this action, given the graph and schema, and possibly a pin that we were dragged from. Returns a node that was created by this action (if any). */
UEdGraphNode* FEdGraphSchemaAction_DataflowVariable::PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode)
{
	// TODO : move this code to the FEditableDataflow eventually 
	UEdGraphNode* NewEdNode = nullptr;
	if (TSharedPtr<FAssetSchemaAction_Dataflow_CreateNode_DataflowEdNode>  VariableNodeAction = FAssetSchemaAction_Dataflow_CreateNode_DataflowEdNode::CreateAction(ParentGraph, "FGetDataflowVariableNode", FullVariableName))
	{
		UDataflowEdNode* DataflowEdNode = Cast<UDataflowEdNode>(VariableNodeAction->PerformAction(ParentGraph, nullptr, Location, false));
		NewEdNode = DataflowEdNode;

		if (DataflowEdNode)
		{
			if (TSharedPtr<FGetDataflowVariableNode> VariableNode = StaticCastSharedPtr<FGetDataflowVariableNode>(DataflowEdNode->GetDataflowNode()))
			{
				if (TStrongObjectPtr<UDataflow> DataflowAsset = DataflowAssetWeakPtr.Pin())
				{
					VariableNode->SetVariable(DataflowAsset.Get(), FullVariableName);
					DataflowEdNode->UpdatePinsFromDataflowNode();
					ParentGraph->NotifyNodeChanged(DataflowEdNode);
				}
			}
		}
	}
	return NewEdNode;
}
/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
FGraphSchemaActionDragDropAction_DataflowVariable::FGraphSchemaActionDragDropAction_DataflowVariable()
	: FGraphSchemaActionDragDropAction()
{}

TSharedRef<FGraphSchemaActionDragDropAction_DataflowVariable> FGraphSchemaActionDragDropAction_DataflowVariable::New(TSharedPtr<FEdGraphSchemaAction_DataflowVariable>& InAction)
{
	TSharedRef<FGraphSchemaActionDragDropAction_DataflowVariable> Operation = MakeShareable(new FGraphSchemaActionDragDropAction_DataflowVariable);
	Operation->VariableAction = InAction;
	Operation->SourceAction = InAction;
	Operation->Construct();
	return Operation;
}

/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FEdGraphSchemaAction_DataflowSubGraph::FEdGraphSchemaAction_DataflowSubGraph()
	: FEdGraphSchemaAction()
{
}

FEdGraphSchemaAction_DataflowSubGraph::FEdGraphSchemaAction_DataflowSubGraph(UDataflow* InDataflowAsset, const FGuid& InSubGraphGuid)
	: FEdGraphSchemaAction(
		FText::GetEmpty(), // menu Category : defined in the ctor body
		FText::GetEmpty(), // menu description : defined in the ctor body
		FText::GetEmpty(), // menu tooltip : defined in the ctor body
		/*InGrouping*/0,
		/*Keywords*/FText::GetEmpty(),
		(int32)UE::Dataflow::ESchemaActionSectionID::SUBGRAPHS)
	, SubGraphGuid(InSubGraphGuid)
	, DataflowAssetWeakPtr(InDataflowAsset)
{
	if (TStrongObjectPtr<const UDataflow> DataflowAsset = DataflowAssetWeakPtr.Pin())
	{
		if (const UDataflowSubGraph* SubGraph = DataflowAsset->FindSubGraphByGuid(SubGraphGuid))
		{
			const FText NewMenuDescription = FText::Format(LOCTEXT("DataflowSubGraphActionDescription", "Function {0}"), FText::FromName(SubGraph->GetFName()));
			const FText NewToolTipDescription = FText::Format(LOCTEXT("DataflowSubGraphActionTooltip", "Function {0}"), FText::FromName(SubGraph->GetFName()));
			UpdateSearchData(NewMenuDescription, NewToolTipDescription, /*NewCategory*/FText::GetEmpty(), /*NewKeywords*/FText::GetEmpty());
		}
	}
}

const FName FEdGraphSchemaAction_DataflowSubGraph::GetSubGraphName() const
{
	if (TStrongObjectPtr<const UDataflow> DataflowAsset = DataflowAssetWeakPtr.Pin())
	{
		if (const UDataflowSubGraph* SubGraph = DataflowAsset->FindSubGraphByGuid(SubGraphGuid))
		{
			return SubGraph->GetFName();
		}
	}
	return {};
}

bool FEdGraphSchemaAction_DataflowSubGraph::IsForEachSubGraph() const
{
	if (TStrongObjectPtr<const UDataflow> DataflowAsset = DataflowAssetWeakPtr.Pin())
	{
		if (const UDataflowSubGraph* SubGraph = DataflowAsset->FindSubGraphByGuid(SubGraphGuid))
		{
			return SubGraph->IsForEachSubGraph();
		}
	}
	return false;
}

void FEdGraphSchemaAction_DataflowSubGraph::SetForEachSubGraph(bool bValue)
{
	if (TStrongObjectPtr<UDataflow> DataflowAsset = DataflowAssetWeakPtr.Pin())
	{
		if (UDataflowSubGraph* SubGraph = DataflowAsset->FindSubGraphByGuid(SubGraphGuid))
		{
			return SubGraph->SetForEachSubGraph(bValue);
		}
	}
}

UEdGraphNode* FEdGraphSchemaAction_DataflowSubGraph::PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode)
{
	UEdGraphNode* NewEdNode = nullptr;
	if (TSharedPtr<FAssetSchemaAction_Dataflow_CreateNode_DataflowEdNode>  SubGraphNodeAction = FAssetSchemaAction_Dataflow_CreateNode_DataflowEdNode::CreateAction(ParentGraph, FDataflowCallSubGraphNode::StaticType(), GetSubGraphName()))
	{
		UDataflowEdNode* DataflowEdNode = Cast<UDataflowEdNode>(SubGraphNodeAction->PerformAction(ParentGraph, nullptr, Location, false));
		NewEdNode = DataflowEdNode;

		if (DataflowEdNode)
		{
			if (TSharedPtr<FDataflowCallSubGraphNode> SubGraphCallNode = StaticCastSharedPtr<FDataflowCallSubGraphNode>(DataflowEdNode->GetDataflowNode()))
			{
				if (TStrongObjectPtr<UDataflow> DataflowAsset = DataflowAssetWeakPtr.Pin())
				{
					SubGraphCallNode->SetSubGraphGuid(SubGraphGuid);
					DataflowEdNode->UpdatePinsFromDataflowNode();
					ParentGraph->NotifyNodeChanged(DataflowEdNode);
				}
			}
		}
	}
	return NewEdNode;
}

bool FEdGraphSchemaAction_DataflowSubGraph::CanRenameItem(FText NewNameAsText) const
{
	if (TStrongObjectPtr<UDataflow> DataflowAsset = DataflowAssetWeakPtr.Pin())
	{
		const FName NewName(NewNameAsText.ToString());
		if (NewName == GetSubGraphName())
		{
			return true;
		}
		return UE::Dataflow::FEditAssetUtils::IsUniqueDataflowSubObjectName(DataflowAsset.Get(), NewName);
	}
	return false;
}

void FEdGraphSchemaAction_DataflowSubGraph::RenameItem(FText NewNameAsText)
{
	if (TStrongObjectPtr<UDataflow> DataflowAsset = DataflowAssetWeakPtr.Pin())
	{
		const FName NewSubGraphName(NewNameAsText.ToString());
		UE::Dataflow::FEditAssetUtils::RenameSubGraph(DataflowAsset.Get(), GetSubGraphName(), NewSubGraphName);

		// todo : shoudl close the existing tab if any 
	}
}

void FEdGraphSchemaAction_DataflowSubGraph::CopyItemToClipboard()
{
	// NOT YET IMPLEMENTED
}

void FEdGraphSchemaAction_DataflowSubGraph::PasteItemFromClipboard()
{
	// NOT YET IMPLEMENTED
}

void FEdGraphSchemaAction_DataflowSubGraph::DeleteItem()
{
	if (TStrongObjectPtr<UDataflow> DataflowAsset = DataflowAssetWeakPtr.Pin())
	{
		UE::Dataflow::FEditAssetUtils::DeleteSubGraph(DataflowAsset.Get(), SubGraphGuid);
	}
}

void FEdGraphSchemaAction_DataflowSubGraph::DuplicateItem()
{
	// NOT YET IMPLEMENTED
}

/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
FGraphSchemaActionDragDropAction_DataflowSubGraph::FGraphSchemaActionDragDropAction_DataflowSubGraph()
	: FGraphSchemaActionDragDropAction()
{}

TSharedRef<FGraphSchemaActionDragDropAction_DataflowSubGraph> FGraphSchemaActionDragDropAction_DataflowSubGraph::New(TSharedPtr<FEdGraphSchemaAction_DataflowSubGraph>& InAction)
{
	TSharedRef<FGraphSchemaActionDragDropAction_DataflowSubGraph> Operation = MakeShareable(new FGraphSchemaActionDragDropAction_DataflowSubGraph);
	Operation->SubGraphAction = InAction;
	Operation->SourceAction = InAction;
	Operation->Construct();
	return Operation;
}

#undef LOCTEXT_NAMESPACE
