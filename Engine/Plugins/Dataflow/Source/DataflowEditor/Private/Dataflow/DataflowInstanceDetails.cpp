// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowInstanceDetails.h"

#include "Dataflow/DataflowObject.h"
#include "Dataflow/DataflowInstance.h"

#include "IDetailChildrenBuilder.h"
#include "IPropertyUtilities.h"
#include "PropertyBagDetails.h"
#include "PropertyCustomizationHelpers.h"
#include "ScopedTransaction.h"

#include "DetailLayoutBuilder.h"


#define LOCTEXT_NAMESPACE "DataflowInstanceDetails"


namespace UE::Dataflow::Private
{
	// variables details ( using FInstancedPropertyBag )
	class FVariablesOverridesDetails : public FPropertyBagInstanceDataDetails
	{
	public:
		FVariablesOverridesDetails(const TSharedPtr<IPropertyHandle> InDataflowInstanceStructProperty, const TSharedPtr<IPropertyHandle> InVariableStructProperty, const TSharedPtr<IPropertyUtilities>& InPropUtils)
			: FPropertyBagInstanceDataDetails(InVariableStructProperty, InPropUtils, /*bInFixedLayout*/ true)
			, DataflowInstanceStructProperty(InDataflowInstanceStructProperty)
		{
		}

	protected:
		struct FVariablesOverridesProvider : public IPropertyBagOverrideProvider
		{
			FVariablesOverridesProvider(FDataflowVariableOverrides& InDataflowVariableOverride)
				: DataflowVariableOverride(InDataflowVariableOverride)
			{
			}

			virtual bool IsPropertyOverridden(const FGuid PropertyID) const override
			{
				return DataflowVariableOverride.IsVariableOverridden(PropertyID);
			}

			virtual void SetPropertyOverride(const FGuid PropertyID, const bool bIsOverridden) const override
			{
				DataflowVariableOverride.SetVariableOverridden(PropertyID, bIsOverridden);
			}

		private:
			FDataflowVariableOverrides& DataflowVariableOverride;
		};

		virtual bool HasPropertyOverrides() const override
		{
			return true;
		}

		virtual void PreChangeOverrides() override
		{
			check(DataflowInstanceStructProperty);
			DataflowInstanceStructProperty->NotifyPreChange();
		}

		virtual void PostChangeOverrides() override
		{
			check(DataflowInstanceStructProperty);
			DataflowInstanceStructProperty->NotifyPostChange(EPropertyChangeType::ValueSet);
			DataflowInstanceStructProperty->NotifyFinishedChangingProperties();
		}

		virtual void EnumeratePropertyBags(TSharedPtr<IPropertyHandle> PropertyBagHandle, const EnumeratePropertyBagFuncRef& Func) const override
		{
			check(DataflowInstanceStructProperty);
			DataflowInstanceStructProperty->EnumerateRawData([Func](void* RawData, const int32 /*DataIndex*/, const int32 /*NumDatas*/)
				{
					if (FDataflowVariableOverrides* DataflowVariableOverrides = static_cast<FDataflowVariableOverrides*>(RawData))
					{
						
						if (const FInstancedPropertyBag* DefaultVariables = DataflowVariableOverrides->GetDefaultVariablesFromAsset())
						{
							FInstancedPropertyBag& OVerridenVariables = DataflowVariableOverrides->GetOverridenVariables();
							FVariablesOverridesProvider OverridesProvider(*DataflowVariableOverrides);
							if (!Func(*DefaultVariables, OVerridenVariables, OverridesProvider))
							{
								return false;
							}
						}
					}
					return true;
				});
		}

	private:

		TSharedPtr<IPropertyHandle> DataflowInstanceStructProperty;
	};
}

///=============================================================================================================
/// 
/// FDataflowVariablesDetails
/// 
///=============================================================================================================
/// 
TSharedRef<IPropertyTypeCustomization> FDataflowVariableOverridesDetails::MakeInstance()
{
	return MakeShareable(new FDataflowVariableOverridesDetails);
}

void FDataflowVariableOverridesDetails::CustomizeHeader(TSharedRef<IPropertyHandle> InStructPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InStructCustomizationUtils)
{
	// default header row
	InHeaderRow
		.NameContent()
		[
			InStructPropertyHandle->CreatePropertyNameWidget()
		];
}

void FDataflowVariableOverridesDetails::CustomizeChildren(const TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& InChildrenBuilder, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	const TSharedPtr<IPropertyHandle> VariablesPropertyHandle = InStructPropertyHandle->GetChildHandle(FDataflowVariableOverrides::GetVariablePropertyName());
	check(VariablesPropertyHandle);

	uint32 NumChildren;
	if (InStructPropertyHandle->GetNumChildren(NumChildren) == FPropertyAccess::Success)
	{
		for (uint32 Index = 0; Index < NumChildren; Index++)
		{
			TSharedPtr<IPropertyHandle> ChildProperty = InStructPropertyHandle->GetChildHandle(Index);
			if (ChildProperty && ChildProperty->GetProperty() == VariablesPropertyHandle->GetProperty())
			{
				// customize the variables property
				const TSharedRef<UE::Dataflow::Private::FVariablesOverridesDetails> VariablesDetails = MakeShareable(new  UE::Dataflow::Private::FVariablesOverridesDetails(InStructPropertyHandle, VariablesPropertyHandle, InCustomizationUtils.GetPropertyUtilities()));
				InChildrenBuilder.AddCustomBuilder(VariablesDetails);
			}
			else
			{
				// add the child property untouched
				InChildrenBuilder.AddProperty(ChildProperty.ToSharedRef());
			}
		}
	}

}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FDataflowInstanceDetailCustomization::FDataflowInstanceDetailCustomization(bool bInOnlyShowVariableOverrides)
	: bOnlyShowVariableOverrides(bInOnlyShowVariableOverrides)
{}

void FDataflowInstanceDetailCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	if (bOnlyShowVariableOverrides)
	{
		DetailBuilder.HideProperty(FDataflowInstance::GetDataflowAssetPropertyName());
		DetailBuilder.HideProperty(FDataflowInstance::GetDataflowTerminalPropertyName());
	}
}


#undef LOCTEXT_NAMESPACE
