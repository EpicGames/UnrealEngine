// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowMembersWidget.h"

#include "Dataflow/DataflowAssetEditUtils.h"
#include "Dataflow/DataflowEditor.h"
#include "Dataflow/DataflowGraphEditor.h"
#include "Dataflow/DataflowGraphSchemaAction.h"
#include "Dataflow/DataflowInstance.h"
#include "Dataflow/DataflowInstanceDetails.h"
#include "Dataflow/DataflowEditorToolkit.h"
#include "Dataflow/DataflowObject.h"
#include "Dataflow/DataflowSubGraph.h"
#include "Dataflow/DataflowVariablePaletteItem.h"
#include "Dataflow/DataflowSubGraphPaletteItem.h"
#include "Dataflow/DataflowVariableNodes.h"
#include "Framework/Commands/UICommandList.h"
#include "GraphActionNode.h"
#include "IDetailsView.h"
#include "IStructureDetailsView.h"
#include "DetailsViewArgs.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "SGraphActionMenu.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "SDataflowMembersWidget"

/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
namespace UE::Dataflow::DataflowMembersWidget::Private
{
	bool bEnableSubGraphs = true;
	FAutoConsoleVariableRef CVarEnableSubGraphs(
		TEXT("p.Dataflow.Editor.EnableSubgraphs"),
		bEnableSubGraphs,
		TEXT("When true, enable Dataflow SubGraph access for the UI")
	);
}
/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

namespace UE::Dataflow::Private
{
	template<typename TActionType>
	TActionType* CastActionTo(FEdGraphSchemaAction& InAction)
	{
		TActionType* TypedAction = nullptr;
		if (InAction.GetTypeId() == TActionType::StaticGetTypeId())
		{
			TypedAction = static_cast<TActionType*>(&InAction);
		}
		return TypedAction;
	}

	template<typename TAction>
	struct FMemberSection: public SDataflowMembersWidget::ISection
	{
		virtual ~FMemberSection() override {}

		virtual const FText& GetTitle() const { return Title; }
		virtual const SDataflowMembersWidget::FButton* GetAddButton() const { return &AddButton; }

		virtual bool CanRequestRename() const override { return true; }

		// todo : when needed we should forward the methods to an action
		virtual bool CanCopy() const override { return true; }
		virtual bool CanPaste() const override { return true; }
		virtual bool CanDuplicate() const override { return true; }
		virtual bool CanDelete() const override { return true; }

		virtual void OnCopy(FEdGraphSchemaAction& InAction) const
		{
			if (TAction* TypedAction = AsTypedAction(InAction))
			{
				TypedAction->CopyItemToClipboard();
			}
		}
		virtual void OnPaste(FEdGraphSchemaAction& InAction) const override
		{
			if (TAction* TypedAction = AsTypedAction(InAction))
			{
				TypedAction->PasteItemFromClipboard();
			}
		}

		virtual void OnDuplicate(FEdGraphSchemaAction& InAction) const override
		{
			if (TAction* TypedAction = AsTypedAction(InAction))
			{
				TypedAction->DuplicateItem();
			}
		}

		virtual void OnDelete(FEdGraphSchemaAction& InAction) const override
		{
			if (TAction* TypedAction = AsTypedAction(InAction))
			{
				TypedAction->DeleteItem();
			}
		}

		static TAction* AsTypedAction(FEdGraphSchemaAction& InAction)
		{
			return CastActionTo<TAction>(InAction);
		}

		FText Title;
		SDataflowMembersWidget::FButton AddButton;
	};

	struct FSubGraphsSection : 
		public FMemberSection<FEdGraphSchemaAction_DataflowSubGraph>
	{
		using Super = FMemberSection<FEdGraphSchemaAction_DataflowSubGraph>;

		static TSharedPtr<FSubGraphsSection> Make()
		{
			TSharedPtr<FSubGraphsSection> SubGraphsSection = MakeShared<FSubGraphsSection>();

			SubGraphsSection->Title = LOCTEXT("SubGraphs", "SubGraphs");
			SubGraphsSection->AddButton =
			{
				.Tooltip = LOCTEXT("AddNewSubGraph", "Add New Sub-Graph"),
				.MetadataTag = TEXT("AddNewSubGraph"),
				.Command = FDataflowEditorCommands::Get().AddNewSubGraph,
			};
			return SubGraphsSection;
		}

		virtual ~FSubGraphsSection() override {}

		virtual TSharedRef<SWidget> CreateWidgetForAction(FCreateWidgetForActionData* const InCreateData, TSharedPtr<SDataflowGraphEditor> Editor) const override
		{
			return SNew(SDataflowSubGraphPaletteItem, InCreateData, Editor);
		}

		virtual void CollectActions(UDataflow* DataflowAsset, TArray<TSharedPtr<FEdGraphSchemaAction>>& OutActions) const override
		{
			if (DataflowAsset)
			{
				for (TObjectPtr<const UDataflowSubGraph> SubGraph : DataflowAsset->GetSubGraphs())
				{
					if (ensure(SubGraph))
					{
						TSharedPtr<FEdGraphSchemaAction_DataflowSubGraph> NewSubGraphAction =
							MakeShareable(new FEdGraphSchemaAction_DataflowSubGraph(DataflowAsset, SubGraph->GetSubGraphGuid()));
						OutActions.Add(NewSubGraphAction);
					}
				}
			}
		}

		virtual void OnDoubleClicked(FEdGraphSchemaAction& InAction, FDataflowEditorToolkit& InToolkit) const override
		{
			if (FEdGraphSchemaAction_DataflowSubGraph* TypedAction = AsTypedAction(InAction))
			{
				InToolkit.OpenSubGraphTab(TypedAction->GetSubGraphName());
			}
		}
	};

	struct FVariablesSection :
		public FMemberSection<FEdGraphSchemaAction_DataflowVariable>
	{
		static TSharedPtr<FVariablesSection> Make()
		{
			TSharedPtr<FVariablesSection> VariablesSection = MakeShared<FVariablesSection>();

			VariablesSection->Title = LOCTEXT("Variables", "Variables");
			VariablesSection->AddButton =
			{
				.Tooltip = LOCTEXT("AddNewVariable", "Add New Variable"),
				.MetadataTag = TEXT("AddNewVariable"),
				.Command = FDataflowEditorCommands::Get().AddNewVariable,
			};
			return VariablesSection;
		}

		virtual ~FVariablesSection() override {}

		virtual TSharedRef<SWidget> CreateWidgetForAction(FCreateWidgetForActionData* const InCreateData, TSharedPtr<SDataflowGraphEditor> Editor) const override
		{
			return SNew(SDataflowVariablePaletteItem, InCreateData, Editor);
		}

		virtual void CollectActions(UDataflow* DataflowAsset, TArray<TSharedPtr<FEdGraphSchemaAction>>& OutActions) const override
		{
			if (DataflowAsset)
			{
				if (const UPropertyBag* PropertyBag = DataflowAsset->Variables.GetPropertyBagStruct())
				{
					for (const FPropertyBagPropertyDesc& ConfigDesc : PropertyBag->GetPropertyDescs())
					{
						TSharedPtr<FEdGraphSchemaAction_DataflowVariable> NewVarAction =
							MakeShareable(new FEdGraphSchemaAction_DataflowVariable(DataflowAsset, ConfigDesc));
						OutActions.Add(NewVarAction);
					}
				}
			}
		}

		virtual void OnDoubleClicked(FEdGraphSchemaAction& InAction, FDataflowEditorToolkit& InToolkit) const override
		{
			if (FEdGraphSchemaAction_DataflowVariable* TypedAction = AsTypedAction(InAction))
			{
				LastFoundVariableNodeGuid = InToolkit.FocusOnNextVariableNode(TypedAction->GetFullVariableName(), LastFoundVariableNodeGuid);
			}
		}

		mutable FGuid LastFoundVariableNodeGuid;
	};
}

/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

void SDataflowMembersWidget::Construct(const FArguments& InArgs, TSharedPtr<FDataflowEditorToolkit> InEditorToolkit)
{
	EditorToolkitWeakPtr = InEditorToolkit;
	CacheAssets();

	InitializeCommands();
	InitializeSections();

	// search box that applies to the entire widget
	SAssignNew(FilterBox, SSearchBox)
		.OnTextChanged(this, &SDataflowMembersWidget::OnFilterTextChanged);

	// create the main action list piece of this widget
	SAssignNew(GraphActionMenu, SGraphActionMenu, false)
		.OnGetFilterText(this, &SDataflowMembersWidget::GetFilterText)
		.OnCreateWidgetForAction(this, &SDataflowMembersWidget::OnCreateWidgetForAction)
		.OnCollectAllActions(this, &SDataflowMembersWidget::CollectAllActions)
		.OnCollectStaticSections(this, &SDataflowMembersWidget::CollectStaticSections)
		.OnActionDragged(this, &SDataflowMembersWidget::OnActionDragged)
		.OnActionDoubleClicked(this, &SDataflowMembersWidget::OnActionDoubleClicked)
		.OnContextMenuOpening(this, &SDataflowMembersWidget::OnContextMenuOpening)
		.OnCanRenameSelectedAction(this, &SDataflowMembersWidget::CanRequestRenameOnActionNode)
		.OnGetSectionTitle(this, &SDataflowMembersWidget::OnGetSectionTitle)
		.OnGetSectionWidget(this, &SDataflowMembersWidget::OnGetSectionWidget)
		.OnActionMatchesName(this, &SDataflowMembersWidget::HandleActionMatchesName)
		.DefaultRowExpanderBaseIndentLevel(1)
		.AlphaSortItems(false)
		.UseSectionStyling(true);

	CreateVariableOverrideDetailView();

	FMenuBuilder ViewOptions(true, nullptr);

	// now piece together all the content for this widget
	ChildSlot
	[
		SNew(SVerticalBox)

		// top part 
		// ( seach bar ) + ( view option menu )
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SBorder)
			.Padding(4.0f)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				SNew(SVerticalBox)

				+ SVerticalBox::Slot()
				.AutoHeight()
				[ 
					SNew(SHorizontalBox)

					// search bar 
					+ SHorizontalBox::Slot()
					.FillWidth(1.0f)
					.VAlign(VAlign_Center)
					[
						FilterBox.ToSharedRef()
					]

					// view option menu 
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(2, 0, 0, 0)
					[
						SNew(SComboButton)
						.ContentPadding(0.0f)
						.ComboButtonStyle(&FAppStyle::Get().GetWidgetStyle<FComboButtonStyle>("SimpleComboButton"))
						.HasDownArrow(false)
						.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("ViewOptions")))
						.ButtonContent()
						[
							SNew(SImage)
							.ColorAndOpacity(FSlateColor::UseForeground())
							.Image(FAppStyle::Get().GetBrush("Icons.Settings"))
						]
						.MenuContent()
						[
							ViewOptions.MakeWidget()
						]
					]
				]
			]
		]

		// actions organized by section ( variables only for now )
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			GraphActionMenu.ToSharedRef()
		]

		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			OverridesDetailsView->GetWidget().ToSharedRef()
		]
	];

	// make sure th sections are expanded
	TMap<int32, bool> ExpandedSections;
	for (const TPair<int32, TSharedPtr<ISection>>& Section : SectionMap)
	{
		ExpandedSections.Add(Section.Key, true);
	}
	GraphActionMenu->SetSectionExpansion(ExpandedSections);

	FCoreUObjectDelegates::OnObjectPropertyChanged.AddRaw(this, &SDataflowMembersWidget::OnObjectPropertyChanged);
	FDataflowAssetDelegates::OnSubGraphsChanged.AddRaw(this, &SDataflowMembersWidget::OnSubGraphsChanged);
	FDataflowAssetDelegates::OnVariablesOverrideStateChanged.AddRaw(this, &SDataflowMembersWidget::OnVariablesOverrideStateChanged);
}

void SDataflowMembersWidget::CreateVariableOverrideDetailView()
{
	FDetailsViewArgs DetailsViewArgs;
	{
		DetailsViewArgs.bAllowSearch = false;
		DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
		DetailsViewArgs.bHideSelectionTip = true;
		DetailsViewArgs.DefaultsOnlyVisibility = EEditDefaultsOnlyNodeVisibility::Automatic;
		DetailsViewArgs.bShowOptions = false;
		DetailsViewArgs.bAllowMultipleTopLevelObjects = true;
		DetailsViewArgs.bShowKeyablePropertiesOption = false;
		DetailsViewArgs.bShowModifiedPropertiesOption = false;
		DetailsViewArgs.bAllowFavoriteSystem = false;
		DetailsViewArgs.bShowAnimatedPropertiesOption = false;
	}

	FStructureDetailsViewArgs StructureViewArgs;
	{
		StructureViewArgs.bShowObjects = true;
		StructureViewArgs.bShowAssets = true;
		StructureViewArgs.bShowClasses = true;
		StructureViewArgs.bShowInterfaces = true;
	}

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	OverridesDetailsView = PropertyEditorModule.CreateStructureDetailView(DetailsViewArgs, StructureViewArgs, nullptr);

	auto MakeDataflowInstanceDetailCustomizationLambda = []()
		{
			constexpr bool bOnlyShowVariableOverrides = true;
			return MakeShared<FDataflowInstanceDetailCustomization>(bOnlyShowVariableOverrides);
		};
	FOnGetDetailCustomizationInstance DataflowInstanceDetailsCustomizationInstance = FOnGetDetailCustomizationInstance::CreateLambda(MakeDataflowInstanceDetailCustomizationLambda);
	OverridesDetailsView->GetDetailsView()->RegisterInstancedCustomPropertyLayout(FDataflowInstance::StaticStruct(), DataflowInstanceDetailsCustomizationInstance);

	OverridesDetailsView->GetOnFinishedChangingPropertiesDelegate().AddRaw(this, &SDataflowMembersWidget::OverridesDetailsViewFinishedChangingProperties);

	RefreshVariableOverrideDetailView();
}

void SDataflowMembersWidget::RefreshVariableOverrideDetailView()
{
	// now assign the corresponding objects and structures
	if (IDataflowInstanceInterface* Interface = GetDataflowInstanceInterface())
	{
		OverridesDetailsView->GetDetailsView()->SetObject(EditedAssetWeakPtr.Get());
		TSharedPtr<FStructOnScope> StructOnScope = Interface->GetDataflowInstance().MakeStructOnScope();
		OverridesDetailsView->SetStructureData(StructOnScope);
	}
}

SDataflowMembersWidget::~SDataflowMembersWidget()
{
	FCoreUObjectDelegates::OnObjectPropertyChanged.RemoveAll(this);
	FDataflowAssetDelegates::OnSubGraphsChanged.RemoveAll(this);
	FDataflowAssetDelegates::OnVariablesOverrideStateChanged.RemoveAll(this); 
}

const TSharedPtr<SDataflowGraphEditor> SDataflowMembersWidget::GetGraphEditor() const
{
	if (EditorToolkitWeakPtr.IsValid())
	{
		return EditorToolkitWeakPtr.Pin()->GetDataflowGraphEditor();
	}
	return {};
}

bool SDataflowMembersWidget::IsDataflowAssetEditable() const
{
	if (TSharedPtr<SDataflowGraphEditor> GraphEditor = GetGraphEditor())
	{
		return GraphEditor->IsEditable();
	}
	return false;
}

void SDataflowMembersWidget::InitializeCommands()
{
	const TSharedPtr<SDataflowGraphEditor> GraphEditor = GetGraphEditor();

	// initialize command list ( merge from Graph editor )
	CommandList = MakeShareable(new FUICommandList);
	if (GraphEditor)
	{
		CommandList->Append(GraphEditor->GetCommands().ToSharedRef());
	}
	CommandList->MapAction(FGenericCommands::Get().Rename,
		FExecuteAction::CreateSP(this, &SDataflowMembersWidget::OnRequestRename),
		FCanExecuteAction::CreateSP(this, &SDataflowMembersWidget::CanRequestRename));

	CommandList->MapAction(FGenericCommands::Get().Copy,
		FExecuteAction::CreateSP(this, &SDataflowMembersWidget::OnCopy),
		FCanExecuteAction::CreateSP(this, &SDataflowMembersWidget::CanCopy));

	CommandList->MapAction(FGenericCommands::Get().Cut,
		FExecuteAction::CreateSP(this, &SDataflowMembersWidget::OnCut),
		FCanExecuteAction::CreateSP(this, &SDataflowMembersWidget::CanCut));

	CommandList->MapAction(FGenericCommands::Get().Paste,
		FExecuteAction::CreateSP(this, &SDataflowMembersWidget::OnPaste),
		FCanExecuteAction(), FIsActionChecked(),
		FIsActionButtonVisible::CreateSP(this, &SDataflowMembersWidget::CanPaste));

	CommandList->MapAction(FGenericCommands::Get().Duplicate,
		FExecuteAction::CreateSP(this, &SDataflowMembersWidget::OnDuplicate),
		FCanExecuteAction(), FIsActionChecked(),
		FIsActionButtonVisible::CreateSP(this, &SDataflowMembersWidget::CanDuplicate));

	CommandList->MapAction(FGenericCommands::Get().Delete,
		FExecuteAction::CreateSP(this, &SDataflowMembersWidget::OnDelete),
		FCanExecuteAction(), FIsActionChecked(),
		FIsActionButtonVisible::CreateSP(this, &SDataflowMembersWidget::CanDelete));

	CommandList->MapAction(FDataflowEditorCommands::Get().ConvertToBasicSubGraph,
		FExecuteAction::CreateSP(this, &SDataflowMembersWidget::SetForEachSubGraphOnSelection, false),
		FCanExecuteAction::CreateSP(this, &SDataflowMembersWidget::IsDataflowAssetEditable));

	CommandList->MapAction(FDataflowEditorCommands::Get().ConvertToForEachSubGraph,
		FExecuteAction::CreateSP(this, &SDataflowMembersWidget::SetForEachSubGraphOnSelection, true),
		FCanExecuteAction::CreateSP(this, &SDataflowMembersWidget::IsDataflowAssetEditable));
}

void SDataflowMembersWidget::InitializeSections()
{
	using namespace UE::Dataflow;
	SectionMap.Reset();
	if (UE::Dataflow::DataflowMembersWidget::Private::bEnableSubGraphs)
	{
		SectionMap.Emplace((int32)ESchemaActionSectionID::SUBGRAPHS, Private::FSubGraphsSection::Make());
	}
	SectionMap.Emplace((int32)ESchemaActionSectionID::VARIABLES, Private::FVariablesSection::Make());
}

const TSharedPtr<SDataflowMembersWidget::ISection> SDataflowMembersWidget::GetSectionById(int32 SectionId) const
{
	if (const TSharedPtr<ISection>* Section = SectionMap.Find(SectionId))
	{
		check(*Section); // should never be null - see InitializeSections()
		return (*Section);
	}
	return nullptr;
}

FReply SDataflowMembersWidget::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (CommandList.IsValid() && CommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

void SDataflowMembersWidget::CacheAssets()
{
	if (EditorToolkitWeakPtr.IsValid())
	{
		TObjectPtr<UDataflowBaseContent> Content = EditorToolkitWeakPtr.Pin()->GetEditorContent();
		if (Content)
		{
			DataflowAssetWeakPtr = Content->GetDataflowAsset();
		}
	}

	const TSharedPtr<SDataflowGraphEditor> GraphEditor = GetGraphEditor();
	if (!GraphEditor)
	{
		return;
	}

	TSharedPtr<UE::Dataflow::FContext> Context = GraphEditor->GetDataflowContext();
	if (!Context)
	{
		return;
	}

	const UE::Dataflow::FEngineContext* EngineContext = Context->AsType<UE::Dataflow::FEngineContext>();
	if (!EngineContext)
	{
		return;
	}

	EditedAssetWeakPtr = EngineContext->Owner;
}

IDataflowInstanceInterface* SDataflowMembersWidget::GetDataflowInstanceInterface() const
{
	if (EditedAssetWeakPtr.IsValid())
	{
		return UE::Dataflow::InstanceUtils::GetDataflowInterfaceFromObject(EditedAssetWeakPtr.Get());
	}
	return nullptr;
}

namespace DataflowMembersWidget::Private
{
	FName ExtractVariableNameFromPropertyChangeEvent(const FPropertyChangedEvent& InPropertyChangedEvent)
	{
		// check if the property changed is deep inside a stackl of property ( like a member of an eleent of an array )
		TMap<FString, int32> PropertyNameStack; // ordered from deeper to 
		InPropertyChangedEvent.GetArrayIndicesPerObject(0, PropertyNameStack);

		if (PropertyNameStack.Num() > 0)
		{
			TArray<FString> PropertyNames;
			PropertyNameStack.GetKeys(PropertyNames);
			// the 3 last ones should always be VariableOverrides / Variables / Value: 
			//   - VariableOverrides because that's the name of the FDataflowInstance member
			//	 - Variables because that the name of the property bag property in FDataflowVariableOverrides
			//	 - Value becasue this is the container of properties in FInstancedPropertyBag
			const bool bIsValid =
				(PropertyNames.Num() > 4) &&
				(PropertyNames[PropertyNames.Num() - 1] == FDataflowInstance::GetVariableOverridesPropertyName()) &&
				(PropertyNames[PropertyNames.Num() - 2] == FDataflowVariableOverrides::GetVariablePropertyName()) &&
				(PropertyNames[PropertyNames.Num() - 3] == "Value") // no programmaic access to the name 
				;
			if (bIsValid)
			{
				// variable name should be here :)
				return FName(PropertyNames[PropertyNames.Num() - 4]);
			}
		}

		return InPropertyChangedEvent.GetPropertyName();
	}
}

void SDataflowMembersWidget::OnVariablesOverrideStateChanged(const UDataflow* InDataflowAsset, FName InVariableName, bool bInNewOverrideState)
{
	if (TStrongObjectPtr<UDataflow> DataflowAsset = DataflowAssetWeakPtr.Pin())
	{
		if (DataflowAsset.Get() == InDataflowAsset)
		{
			InvalidateVariableNode(*DataflowAsset, InVariableName);
		}
	}
}

void SDataflowMembersWidget::OverridesDetailsViewFinishedChangingProperties(const FPropertyChangedEvent& InPropertyChangedEvent)
{
	if (TStrongObjectPtr<UDataflow> DataflowAsset = DataflowAssetWeakPtr.Pin())
	{
		if (IDataflowInstanceInterface* InstanceInterface = GetDataflowInstanceInterface())
		{
			const FDataflowInstance& DataflowInstance = InstanceInterface->GetDataflowInstance();

			TMap<FString, int32> PropertyNameStack; // ordered from deeper to 
			InPropertyChangedEvent.GetArrayIndicesPerObject(0, PropertyNameStack);

			const FName VariableName = DataflowMembersWidget::Private::ExtractVariableNameFromPropertyChangeEvent(InPropertyChangedEvent);
			InvalidateVariableNode(*DataflowAsset, VariableName);
		}
	}
}

void SDataflowMembersWidget::InvalidateVariableNode(const UDataflow& InDataflowAsset, FName InVariableName)
{
	if (InDataflowAsset.Variables.FindPropertyDescByName(InVariableName))
	{
		// invalidate all the get nodes all the nodes that match the variable name 
		for (const TSharedPtr<FDataflowNode>& Node : InDataflowAsset.GetDataflow()->GetNodes())
		{
			if (FGetDataflowVariableNode* VariableNode = Node->AsType<FGetDataflowVariableNode>())
			{
				if (VariableNode->GetVariableName() == InVariableName)
				{
					VariableNode->Invalidate();
				}
			}
		}
	}
}

void SDataflowMembersWidget::OnObjectPropertyChanged(UObject* InObject, FPropertyChangedEvent& InPropertyChangedEvent)
{
	static const FName DataflowVariablesPropertyName = GET_MEMBER_NAME_CHECKED(UDataflow, Variables);

	if (InObject)
	{
		if (InObject == DataflowAssetWeakPtr)
		{
			if (InPropertyChangedEvent.GetMemberPropertyName() == DataflowVariablesPropertyName
				|| InPropertyChangedEvent.GetPropertyName() == DataflowVariablesPropertyName)
			{
				Refresh();
			}
			RefreshVariableOverrideDetailView();
		}
		else if (InObject == EditedAssetWeakPtr)
		{
			RefreshVariableOverrideDetailView();
		}
	}
}

void SDataflowMembersWidget::OnSubGraphsChanged(const UDataflow* InDataflowAsset, const FGuid& InSubGraphGuid, UE::Dataflow::ESubGraphChangedReason InReason)
{
	// we only need to do something for our own asset
	if (InDataflowAsset != DataflowAssetWeakPtr)
	{
		return;
	}

	Refresh();
	
	// handle SubGraphs tabs
	if (TSharedPtr<FDataflowEditorToolkit> Toolkit = EditorToolkitWeakPtr.Pin())
	{
		if (InDataflowAsset)
		{
			if (const UDataflowSubGraph* SubGraph = InDataflowAsset->FindSubGraphByGuid(InSubGraphGuid))
			{
				switch (InReason)
				{
				case UE::Dataflow::ESubGraphChangedReason::Created:
					Toolkit->OpenSubGraphTab(SubGraph);
					break;

				case UE::Dataflow::ESubGraphChangedReason::Renamed:
					Toolkit->ReOpenSubGraphTab(SubGraph);
					break;

				case UE::Dataflow::ESubGraphChangedReason::Deleting:
					Toolkit->CloseSubGraphTab(SubGraph);
					break;

				case UE::Dataflow::ESubGraphChangedReason::Deleted:
				case UE::Dataflow::ESubGraphChangedReason::ChangedType:
					// nothing to do 
					break;
				}
			}
		}
	}
}

void SDataflowMembersWidget::Refresh()
{
	if (GraphActionMenu)
	{
		GraphActionMenu->RefreshAllActions(/*bPreserveExpansion=*/ true);
	}
}

void SDataflowMembersWidget::OnFilterTextChanged(const FText& InFilterText)
{
	GraphActionMenu->GenerateFilteredItems(false);
}

FText SDataflowMembersWidget::GetFilterText() const
{
	return FilterBox->GetText();
}

TSharedRef<SWidget> SDataflowMembersWidget::OnCreateWidgetForAction(FCreateWidgetForActionData* const InCreateData)
{
	if (InCreateData && InCreateData->Action)
	{
		if (const TSharedPtr<ISection> Section = GetSectionById(InCreateData->Action->SectionID))
		{
			TSharedRef<SWidget> Widget = Section->CreateWidgetForAction(InCreateData, GetGraphEditor());
			TAttribute<bool> SetEnabledAttribute(this, &SDataflowMembersWidget::IsDataflowAssetEditable);
			Widget->SetEnabled(SetEnabledAttribute);
			return Widget;
		}
	}
	return SNullWidget::NullWidget;
}

void SDataflowMembersWidget::CollectAllActions(FGraphActionListBuilderBase& OutAllActions)
{
	TArray<TSharedPtr<FEdGraphSchemaAction>> Actions;
	if (TStrongObjectPtr<UDataflow> DataflowAsset = DataflowAssetWeakPtr.Pin())
	{
		for (const TPair<int32, TSharedPtr<ISection>>& SectionEntry : SectionMap)
		{
			if (ensure(SectionEntry.Value))
			{
				SectionEntry.Value->CollectActions(DataflowAsset.Get(), Actions);
			}
		}
	}
	for (TSharedPtr<FEdGraphSchemaAction> Action : Actions)
	{
		OutAllActions.AddAction(Action);
	}
}

void SDataflowMembersWidget::CollectStaticSections(TArray<int32>& StaticSectionIDs)
{
	SectionMap.GetKeys(StaticSectionIDs);
}

FReply SDataflowMembersWidget::OnActionDragged(const TArray<TSharedPtr<FEdGraphSchemaAction>>& InActions, const FPointerEvent& MouseEvent)
{
	TSharedPtr<FEdGraphSchemaAction> InAction(InActions.Num() > 0 ? InActions[0] : nullptr);
	if (!InAction.IsValid())
	{
		return FReply::Unhandled();
	}

	if (InAction->GetTypeId() == FEdGraphSchemaAction_DataflowVariable::StaticGetTypeId())
	{
		TSharedPtr<FEdGraphSchemaAction_DataflowVariable> VariableAction = StaticCastSharedPtr<FEdGraphSchemaAction_DataflowVariable>(InAction);
		TSharedRef<FGraphSchemaActionDragDropAction_DataflowVariable> DragOperation = FGraphSchemaActionDragDropAction_DataflowVariable::New(VariableAction);
		return FReply::Handled().BeginDragDrop(DragOperation);
	}

	if (InAction->GetTypeId() == FEdGraphSchemaAction_DataflowSubGraph::StaticGetTypeId())
	{
		TSharedPtr<FEdGraphSchemaAction_DataflowSubGraph> SubGraphAction = StaticCastSharedPtr<FEdGraphSchemaAction_DataflowSubGraph>(InAction);
		TSharedRef<FGraphSchemaActionDragDropAction_DataflowSubGraph> DragOperation = FGraphSchemaActionDragDropAction_DataflowSubGraph::New(SubGraphAction);
		return FReply::Handled().BeginDragDrop(DragOperation);
	}
	return FReply::Unhandled();
}

void SDataflowMembersWidget::OnActionDoubleClicked(const TArray< TSharedPtr<FEdGraphSchemaAction> >& InActions)
{
	if (TSharedPtr<FDataflowEditorToolkit> Toolkit = EditorToolkitWeakPtr.Pin())
	{
		for (TSharedPtr<FEdGraphSchemaAction> Action : InActions)
		{
			if (Action)
			{
				if (const TSharedPtr<ISection> Section = GetSectionById(Action->GetSectionID()))
				{
					Section->OnDoubleClicked(*Action, *Toolkit);
				}
			}
		}
	}
}

TSharedPtr<SWidget> SDataflowMembersWidget::OnContextMenuOpening()
{
	const TAttribute<FText> EmptyTextAttribute;

	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, CommandList);

	if (IsAnyActionSelected())
	{
		const bool bSubGraphActions = IsOnlySubgraphActionsSelected();

		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Rename);
		if (bSubGraphActions)
		{
			MenuBuilder.AddMenuSeparator();

			if (IsSelectionForEachSubGraph())
			{
				const FSlateIcon FunctionIcon(FAppStyle::GetAppStyleSetName(), "GraphEditor.Function_16x");
				MenuBuilder.AddMenuEntry(FDataflowEditorCommands::Get().ConvertToBasicSubGraph, NAME_None, EmptyTextAttribute, EmptyTextAttribute, FunctionIcon);
			}
			else
			{
				const FSlateIcon LoopIcon(FAppStyle::GetAppStyleSetName(), "GraphEditor.Macro.Loop_16x");
				MenuBuilder.AddMenuEntry(FDataflowEditorCommands::Get().ConvertToForEachSubGraph, NAME_None, EmptyTextAttribute, EmptyTextAttribute, LoopIcon);
			}
		}
		if (!bSubGraphActions)
		{
			MenuBuilder.AddMenuSeparator();
			MenuBuilder.AddMenuEntry(FGenericCommands::Get().Cut);
			MenuBuilder.AddMenuEntry(FGenericCommands::Get().Copy);
			MenuBuilder.AddMenuEntry(FGenericCommands::Get().Paste);
			MenuBuilder.AddMenuEntry(FGenericCommands::Get().Duplicate);
		}
		MenuBuilder.AddMenuSeparator();
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Delete);
	}
	else
	{
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Paste);
	}

	return MenuBuilder.MakeWidget();
}

FText SDataflowMembersWidget::OnGetSectionTitle(int32 InSectionID)
{
	if (const TSharedPtr<ISection> Section = GetSectionById(InSectionID))
	{
		return Section->GetTitle();
	}
	return {};
}

TSharedRef<SWidget> SDataflowMembersWidget::OnGetSectionWidget(TSharedRef<SWidget> RowWidget, int32 InSectionID)
{
	TWeakPtr<SWidget> WeakRowWidget = RowWidget;
	return CreateAddToSectionButton(InSectionID, WeakRowWidget);
}

TSharedRef<SWidget> SDataflowMembersWidget::CreateAddToSectionButton(int32 InSectionID, TWeakPtr<SWidget> WeakRowWidget)
{
	if (const TSharedPtr<ISection> Section = GetSectionById(InSectionID))
	{
		if (const FButton* AddButton = Section->GetAddButton())
		{
			return
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.OnClicked(this, &SDataflowMembersWidget::OnAddButtonClickedOnSection, InSectionID)
				.IsEnabled(this, &SDataflowMembersWidget::CanAddNewElementToSection, InSectionID)
				.ContentPadding(FMargin(1, 0))
				.AddMetaData<FTagMetaData>(FTagMetaData(AddButton->MetadataTag))
				.ToolTipText(AddButton->Tooltip)
				[
					SNew(SImage)
						.Image(FAppStyle::Get().GetBrush("Icons.PlusCircle"))
						.ColorAndOpacity(FSlateColor::UseForeground())
				];
		}
	}
	return SNullWidget::NullWidget;
}

FReply SDataflowMembersWidget::OnAddButtonClickedOnSection(int32 InSectionID)
{
	if (const TSharedPtr<ISection> Section = GetSectionById(InSectionID))
	{
		if (const FButton* AddButton = Section->GetAddButton())
		{
			CommandList->ExecuteAction(AddButton->Command.ToSharedRef());
		}
	}
	return FReply::Handled();
}

bool SDataflowMembersWidget::CanAddNewElementToSection(int32 InSectionID) const
{
	return IsDataflowAssetEditable();
}

bool SDataflowMembersWidget::HandleActionMatchesName(FEdGraphSchemaAction* InAction, const FName& InName) const
{
	// todo
	return false;
}

void SDataflowMembersWidget::SelectItemByName(const FName& ItemName, ESelectInfo::Type SelectInfo, int32 SectionId/* = INDEX_NONE*/, bool bIsCategory/* = false*/)
{
	// Check if the graph action menu is being told to clear
	if (ItemName == NAME_None)
	{
		GraphActionMenu->SelectItemByName(NAME_None);
	}
	else
	{
		// Attempt to select the item in the main graph action menu
		const bool bSucceededAtSelecting = GraphActionMenu->SelectItemByName(ItemName, SelectInfo, SectionId, bIsCategory);
		if (!bSucceededAtSelecting)
		{ 
			// We failed to select the item, maybe because it was filtered out?
			// Reset the item filter and try again (we don't do this first because someone went to the effort of typing
			// a filter and probably wants to keep it unless it is getting in the way, as it just has)
			FilterBox->SetText(FText::GetEmpty());
			GraphActionMenu->SelectItemByName(ItemName, SelectInfo, SectionId, bIsCategory);
		}
	}
}

bool SDataflowMembersWidget::IsAnyActionSelected() const
{
	TArray<TSharedPtr<FEdGraphSchemaAction> > SelectedActions;
	GraphActionMenu->GetSelectedActions(SelectedActions);
	return (SelectedActions.Num() > 0);
}

bool SDataflowMembersWidget::IsOnlySubgraphActionsSelected() const
{
	TArray<TSharedPtr<FEdGraphSchemaAction> > SelectedActions;
	GraphActionMenu->GetSelectedActions(SelectedActions);
	if (SelectedActions.IsEmpty())
	{
		return false;
	}
	for (const TSharedPtr<FEdGraphSchemaAction>& Action : SelectedActions)
	{
		if (Action->GetTypeId() != FEdGraphSchemaAction_DataflowSubGraph::StaticGetTypeId())
		{
			return false;
		}
	}
	return true;
}

TSharedPtr<FEdGraphSchemaAction> SDataflowMembersWidget::GetFirstSelectedAction() const
{
	TArray<TSharedPtr<FEdGraphSchemaAction>> SelectedActions;
	GraphActionMenu->GetSelectedActions(SelectedActions);
	return (SelectedActions.Num()) ? SelectedActions[0] : nullptr;
}

void SDataflowMembersWidget::OnRequestRename()
{
	// simple forward to rename request of the action menu
	GraphActionMenu->OnRequestRenameOnActionNode();
}

bool SDataflowMembersWidget::CanRequestRename() const
{
	return IsAnyActionSelected() 
		&& GraphActionMenu->CanRequestRenameOnActionNode();
}

bool SDataflowMembersWidget::CanRequestRenameOnActionNode(TWeakPtr<FGraphActionNode> InSelectedNode) const
{
	if (IsDataflowAssetEditable())
	{
		if (TSharedPtr<FGraphActionNode> ActionNode = InSelectedNode.Pin())
		{
			if (TSharedPtr<FEdGraphSchemaAction> Action = ActionNode->Action)
			{
				if (const TSharedPtr<ISection> Section = GetSectionById(Action->GetSectionID()))
				{
					return Section->CanRequestRename();
				}
			}
		}
	}
	return false;
}

void SDataflowMembersWidget::OnCopy()
{
	if (const TSharedPtr<FEdGraphSchemaAction> Action = GetFirstSelectedAction())
	{
		if (const TSharedPtr<ISection> Section = GetSectionById(Action->GetSectionID()))
		{
			Section->OnCopy(*Action);
		}
	}
}

bool SDataflowMembersWidget::CanCopy() const
{
	if (const TSharedPtr<FEdGraphSchemaAction> Action = GetFirstSelectedAction())
	{
		if (const TSharedPtr<ISection> Section = GetSectionById(Action->GetSectionID()))
		{
			return Section->CanCopy();
		}
	}
	return false;
}

void SDataflowMembersWidget::OnCut()
{
	if (const TSharedPtr<FEdGraphSchemaAction> Action = GetFirstSelectedAction())
	{
		if (const TSharedPtr<ISection> Section = GetSectionById(Action->GetSectionID()))
		{
			Section->OnCopy(*Action);
			Section->OnDelete(*Action);
		}
	}
}

bool SDataflowMembersWidget::CanCut() const
{
	if (IsDataflowAssetEditable())
	{
		if (const TSharedPtr<FEdGraphSchemaAction> Action = GetFirstSelectedAction())
		{
			if (const TSharedPtr<ISection> Section = GetSectionById(Action->GetSectionID()))
			{
				return Section->CanCopy() && Section->CanDelete();
			}
		}
	}
	return false;
}

void SDataflowMembersWidget::OnPaste()
{
	if (const TSharedPtr<FEdGraphSchemaAction> Action = GetFirstSelectedAction())
	{
		if (const TSharedPtr<ISection> Section = GetSectionById(Action->GetSectionID()))
		{
			Section->OnPaste(*Action);
		}
	}
}

bool SDataflowMembersWidget::CanPaste() const
{
	if (!IsDataflowAssetEditable())
	{
		return false;
	}
	
	if (const TSharedPtr<FEdGraphSchemaAction> Action = GetFirstSelectedAction())
	{
		if (const TSharedPtr<ISection> Section = GetSectionById(Action->GetSectionID()))
		{
			return Section->CanPaste();
		}
	}

	// paste is allowed anywhere on the empty space of the widget 
	return true;
}

void SDataflowMembersWidget::OnDuplicate()
{
	if (const TSharedPtr<FEdGraphSchemaAction> Action = GetFirstSelectedAction())
	{
		if (const TSharedPtr<ISection> Section = GetSectionById(Action->GetSectionID()))
		{
			Section->OnDuplicate(*Action);
		}
	}
}

bool SDataflowMembersWidget::CanDuplicate() const
{
	if (IsDataflowAssetEditable())
	{
		if (const TSharedPtr<FEdGraphSchemaAction> Action = GetFirstSelectedAction())
		{
			if (const TSharedPtr<ISection> Section = GetSectionById(Action->GetSectionID()))
			{
				return Section->CanDuplicate();
			}
		}
	}
	return false;
}

void SDataflowMembersWidget::OnDelete()
{
	if (const TSharedPtr<FEdGraphSchemaAction> Action = GetFirstSelectedAction())
	{
		if (const TSharedPtr<ISection> Section = GetSectionById(Action->GetSectionID()))
		{
			Section->OnDelete(*Action);
		}
	}
}

bool SDataflowMembersWidget::CanDelete() const
{
	if (IsDataflowAssetEditable())
	{
		if (const TSharedPtr<FEdGraphSchemaAction> Action = GetFirstSelectedAction())
		{
			if (const TSharedPtr<ISection> Section = GetSectionById(Action->GetSectionID()))
			{
				return Section->CanDelete();
			}
		}
	}
	return false;
}


void SDataflowMembersWidget::SetForEachSubGraphOnSelection(bool bValue)
{
	TArray<TSharedPtr<FEdGraphSchemaAction> > SelectedActions;
	GraphActionMenu->GetSelectedActions(SelectedActions);

	for (TSharedPtr<FEdGraphSchemaAction>& Action : SelectedActions)
	{
		if (Action->GetTypeId() == FEdGraphSchemaAction_DataflowSubGraph::StaticGetTypeId())
		{
			if (FEdGraphSchemaAction_DataflowSubGraph* SubGraphAction = static_cast<FEdGraphSchemaAction_DataflowSubGraph*>(Action.Get()))
			{
				SubGraphAction->SetForEachSubGraph(bValue);
			}
		}
	}
}

bool SDataflowMembersWidget::IsSelectionForEachSubGraph() const
{
	TArray<TSharedPtr<FEdGraphSchemaAction> > SelectedActions;
	GraphActionMenu->GetSelectedActions(SelectedActions);

	if (SelectedActions.IsEmpty())
	{
		return false;
	}

	int32 NumForEachSubGraph = 0;
	for (const TSharedPtr<FEdGraphSchemaAction>& Action : SelectedActions)
	{
		if (Action->GetTypeId() == FEdGraphSchemaAction_DataflowSubGraph::StaticGetTypeId())
		{
			if (const FEdGraphSchemaAction_DataflowSubGraph* SubGraphAction = static_cast<FEdGraphSchemaAction_DataflowSubGraph*>(Action.Get()))
			{
				if (SubGraphAction->IsForEachSubGraph())
				{
					++NumForEachSubGraph;
				}
			}
		}
	}
	return (NumForEachSubGraph > (SelectedActions.Num() / 2));
}

#undef LOCTEXT_NAMESPACE