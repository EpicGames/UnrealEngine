// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class UDataflow;
class UDataflowEditorVariables;
class FDataflowEditorToolkit;
class FUICommandList;
class FUICommandInfo;
class SSearchBox;
class SGraphActionMenu;
class SDataflowGraphEditor;
class IDataflowInstanceInterface;
class IStructureDetailsView;

struct FCreateWidgetForActionData;
struct FGraphActionListBuilderBase;
struct FEdGraphSchemaAction;
struct FGraphActionNode;
struct FPropertyBagPropertyDesc;
struct FDataflowVariableClassFilter;

namespace UE::Dataflow
{
	enum class ESubGraphChangedReason : uint8;
}

/**
*
* Widget to interact with Variables, subgraphs in Dataflow editors
*
*/
class SDataflowMembersWidget: public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SDataflowMembersWidget) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<FDataflowEditorToolkit> InEditorToolkit);
	~SDataflowMembersWidget();

public:
	struct FButton
	{
		FText Tooltip;
		FName MetadataTag;
		TSharedPtr<FUICommandInfo> Command;
	};

	struct ISection
	{
		virtual ~ISection() = default;

		virtual const FText& GetTitle() const = 0;
		virtual const FButton* GetAddButton() const = 0;

		virtual bool CanRequestRename() const = 0;

		virtual bool CanCopy() const = 0;
		virtual bool CanPaste() const = 0;
		virtual bool CanDuplicate() const = 0;
		virtual bool CanDelete() const = 0;

		virtual void OnCopy(FEdGraphSchemaAction&) const = 0;
		virtual void OnPaste(FEdGraphSchemaAction&) const = 0;
		virtual void OnDuplicate(FEdGraphSchemaAction&) const = 0;
		virtual void OnDelete(FEdGraphSchemaAction&) const = 0;

		virtual void OnDoubleClicked(FEdGraphSchemaAction&, FDataflowEditorToolkit&) const = 0;

		virtual void CollectActions(UDataflow* DataflowAsset, TArray<TSharedPtr<FEdGraphSchemaAction>>& OutActions) const = 0;
		virtual TSharedRef<SWidget> CreateWidgetForAction(FCreateWidgetForActionData* const InCreateData, TSharedPtr<SDataflowGraphEditor> Editor) const = 0;
	};

private:
	/* SWidget interface */
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent);

	void OnObjectPropertyChanged(UObject* InObject, FPropertyChangedEvent& InPropertyChangedEvent);
	void OnVariablesOverrideStateChanged(const UDataflow* DataflowAsset, FName VariableName, bool bNewOverrideState);
	void OnSubGraphsChanged(const UDataflow* InDataflowAsset, const FGuid& InSubGraphGuid, UE::Dataflow::ESubGraphChangedReason InReason);

	void InvalidateVariableNode(const UDataflow& InDataflowAsset, FName VariableName);

	void OnFilterTextChanged(const FText& InFilterText);
	FText GetFilterText() const;
	TSharedRef<SWidget> OnCreateWidgetForAction(FCreateWidgetForActionData* const InCreateData);
	void CollectAllActions(FGraphActionListBuilderBase& OutAllActions);
	void CollectStaticSections(TArray<int32>& StaticSectionIDs);
	FReply OnActionDragged(const TArray<TSharedPtr<FEdGraphSchemaAction>>& InActions, const FPointerEvent& MouseEvent);
	void OnActionDoubleClicked(const TArray< TSharedPtr<FEdGraphSchemaAction> >& InActions);
	TSharedPtr<SWidget> OnContextMenuOpening();
	bool CanRequestRenameOnActionNode(TWeakPtr<FGraphActionNode> InSelectedNode) const;
	FText OnGetSectionTitle(int32 InSectionID);
	TSharedRef<SWidget> OnGetSectionWidget(TSharedRef<SWidget> RowWidget, int32 InSectionID);
	bool HandleActionMatchesName(FEdGraphSchemaAction* InAction, const FName& InName) const;

	TSharedRef<SWidget> CreateAddToSectionButton(int32 InSectionID, TWeakPtr<SWidget> WeakRowWidget);
	FReply OnAddButtonClickedOnSection(int32 InSectionID);
	bool CanAddNewElementToSection(int32 InSectionID) const;

	void SelectItemByName(const FName& ItemName, ESelectInfo::Type SelectInfo = ESelectInfo::Direct, int32 SectionId = INDEX_NONE, bool bIsCategory = false);
	bool IsAnyActionSelected() const;
	bool IsOnlySubgraphActionsSelected() const;
	TSharedPtr<FEdGraphSchemaAction> GetFirstSelectedAction() const;

	const TSharedPtr<ISection> GetSectionById(int32 SectionId) const;

	void CacheAssets();
	UDataflow* GetDataflowAsset() const;
	IDataflowInstanceInterface* GetDataflowInstanceInterface() const;

	// command functions
	void OnRequestRename();
	bool CanRequestRename() const;
	void OnCopy();
	bool CanCopy() const;
	void OnCut();
	bool CanCut() const;
	void OnPaste();
	bool CanPaste() const;
	void OnDuplicate();
	bool CanDuplicate() const;
	void OnDelete();
	bool CanDelete() const;
	bool IsSelectionForEachSubGraph() const;
	void SetForEachSubGraphOnSelection(bool bValue);
	
	bool IsDataflowAssetEditable() const;

	// UI refresh 
	void Refresh();

	void InitializeCommands();
	void InitializeSections();
	void CreateVariableOverrideDetailView();
	void RefreshVariableOverrideDetailView();
	void OverridesDetailsViewFinishedChangingProperties(const FPropertyChangedEvent& InPropertyChangedEvent);

	const TSharedPtr<SDataflowGraphEditor> GetGraphEditor() const;

private:
	TWeakPtr<FDataflowEditorToolkit> EditorToolkitWeakPtr;

	/** 
	* asset being edited by the graph - maybe a geometry collection with a dataflow asset bound to it 
	* Can be null if the dataflow asset graph is edited directly outside of the context of a specific asset
	*/
	TWeakObjectPtr<UObject> EditedAssetWeakPtr;

	/** Dataflow asset  */
	TWeakObjectPtr<UDataflow> DataflowAssetWeakPtr;

	/** List of UI Commands for this scope */
	TSharedPtr<FUICommandList> CommandList;

	/** The filter box that handles filtering for graph action menu. */
	TSharedPtr<SSearchBox> FilterBox;

	/** Graph Action Menu for displaying all our variables and sub-graphs */
	TSharedPtr<SGraphActionMenu> GraphActionMenu;

	TMap<int32, TSharedPtr<ISection>> SectionMap;

	TSharedPtr<IStructureDetailsView> OverridesDetailsView;
};
