// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowNodeDetailExtension.h"
#include "Misc/LazySingleton.h"
#include "ChaosLog.h"

#define LOCTEXT_NAMESPACE "DataflowNodeDetailExtension"

namespace UE::Dataflow
{
	FDataflowNodeDetailExtensionRegistry& FDataflowNodeDetailExtensionRegistry::GetInstance()
	{
		return TLazySingleton<FDataflowNodeDetailExtensionRegistry>::Get();
	}

	void FDataflowNodeDetailExtensionRegistry::TearDown()
	{
		return TLazySingleton<FDataflowNodeDetailExtensionRegistry>::TearDown();
	}

	void FDataflowNodeDetailExtensionRegistry::RegisterExtension(TUniquePtr<IDataflowNodeDetailExtension>&& Extension)
	{
		const FName NewExtensionName = Extension->GetName();
		if (ExtensionMap.Contains(NewExtensionName))
		{
			UE_LOGF(LogChaos, Warning, "Dataflow node detail extension registration conflicts with existing extension: %ls", *NewExtensionName.ToString());
		}
		else
		{
			ExtensionMap.Add(NewExtensionName, MoveTemp(Extension));
		}
	}

	void FDataflowNodeDetailExtensionRegistry::DeregisterExtension(const FName& ExtensionName)
	{
		if (!ExtensionMap.Contains(ExtensionName))
		{
			UE_LOGF(LogChaos, Warning, "Dataflow node detail extension deregistration -- extension not registered : %ls", *ExtensionName.ToString());
		}
		else
		{
			ExtensionMap.Remove(ExtensionName);
		}
	}

	bool FDataflowNodeDetailExtensionRegistry::IsPropertyExtendable(const UClass* InObjectClass, const IPropertyHandle& PropertyHandle) const
	{
		for (TMap<FName, TUniquePtr<IDataflowNodeDetailExtension>>::TConstIterator Iter = ExtensionMap.CreateConstIterator(); Iter; ++Iter)
		{
			if (Iter.Value() && Iter.Value()->IsPropertyExtendable(InObjectClass, PropertyHandle))
			{
				return true;
			}
		}
		return false;
	}

	void FDataflowNodeDetailExtensionRegistry::ExtendWidgetRow(FDetailWidgetRow& InWidgetRow, const IDetailLayoutBuilder& InDetailBuilder, const UClass* InObjectClass, TSharedPtr<IPropertyHandle> PropertyHandle)
	{
		if (!PropertyHandle)
		{
			return;
		}

		for (TMap<FName, TUniquePtr<IDataflowNodeDetailExtension>>::TIterator Iter = ExtensionMap.CreateIterator(); Iter; ++Iter)
		{
			if (Iter.Value() && Iter.Value()->IsPropertyExtendable(InObjectClass, *PropertyHandle))
			{
				Iter.Value()->ExtendWidgetRow(InWidgetRow, InDetailBuilder, InObjectClass, PropertyHandle);
			}
		}
	}

	bool FDataflowNodeDetailExtensionHandler::IsPropertyExtendable(const UClass* InObjectClass, const IPropertyHandle& PropertyHandle) const
	{
		return FDataflowNodeDetailExtensionRegistry::GetInstance().IsPropertyExtendable(InObjectClass, PropertyHandle);
	}

	void FDataflowNodeDetailExtensionHandler::ExtendWidgetRow(FDetailWidgetRow& InWidgetRow, const IDetailLayoutBuilder& InDetailBuilder, const UClass* InObjectClass, TSharedPtr<IPropertyHandle> PropertyHandle)
	{
		FDataflowNodeDetailExtensionRegistry::GetInstance().ExtendWidgetRow(InWidgetRow, InDetailBuilder, InObjectClass, PropertyHandle);
	}
}

#undef LOCTEXT_NAMESPACE 

