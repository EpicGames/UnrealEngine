// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataflowOutlinerFactory.h"
#include "Dataflow/DataflowOutlinerFactory.h"
#include "Dataflow/DataflowContent.h"
#include "Dataflow/DataflowEditorPreviewSceneBase.h"
#include "Dataflow/DataflowElement.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementHiearchyColumns.h"
#include "Elements/Columns/TypedElementLabelColumns.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Columns/TypedElementTypeInfoColumns.h"
#include "Elements/Columns/TypedElementVisibilityColumns.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowOutlinerFactory)

namespace UE::Dataflow::Private
{
	/** Command to set visibility onto the actors */
	struct FSetActorVisibilityCommand
	{
		void operator()()
		{
			if (TStrongObjectPtr<AActor> PinnedActor = SceneActor.Pin())
			{
				SceneActor->SetIsTemporarilyHiddenInEditor(!bIsVisible);
			}
			
		}
		TWeakObjectPtr<AActor> SceneActor;
		bool bIsVisible;
	};

	/** Command to set visibility onto the components */
	struct FSetComponentVisibilityCommand
	{
		void operator()()
		{
			if (TStrongObjectPtr<USceneComponent> PinnedComponent = SceneComponent.Pin())
			{
				PinnedComponent->SetVisibility(bIsVisible);
			}
		}
		TWeakObjectPtr<USceneComponent> SceneComponent;
		bool bIsVisible;
	};

	/** Command to set visibility onto the elements */
	struct FSetElementVisibilityCommand
	{
		void operator()()
		{
			if (SceneElement)
			{
				SceneElement->bIsVisible = bIsVisible;
			}
		}
		FDataflowBaseElement* SceneElement;
		bool bIsVisible;
	};
}

void UDataflowObjectFactory::RegisterTables(UE::Editor::DataStorage::ICoreProvider& DataStorage, UE::Editor::DataStorage::ICompatibilityProvider& DataStorageCompatibility)
{
	Super::RegisterTables(DataStorage, DataStorageCompatibility);
}

void UDataflowObjectFactory::RegisterHierarchyQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	using namespace UE::Editor::DataStorage::Queries;
	using namespace UE::Editor::DataStorage;
	
	DataStorage.RegisterQuery(
	Select(
		TEXT("Sync dataflow object hierarchy to column"),
		FProcessor(EQueryTickPhase::PrePhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage))
			.SetExecutionMode(EExecutionMode::GameThread),
		[&DataStorage](IQueryContext& Context, RowHandle ChildHandle,
			const FTypedElementUObjectColumn& RawObject, FTableRowParentColumn& ParentColumn)
		{
			UObject* ParentObject = nullptr;
			if (const USceneComponent* SceneComponent = Cast<USceneComponent>(RawObject.Object))
			{
				if (USceneComponent* ParentComponent = SceneComponent->GetAttachParent())
				{
					ParentObject = ParentComponent;
				}
				else if(AActor* ParentActor = SceneComponent->GetOwner())
				{
					ParentObject = ParentActor;
				}	
			}
			else if (const AActor* SceneActor = Cast<AActor>(RawObject.Object))
			{
				if(AActor* ParentActor = SceneActor->GetOwner())
				{
					ParentObject = ParentActor;
				}
			}
			if(ParentObject)
			{
				FMapKeyView IdKey = FMapKeyView(ParentObject);
				RowHandle ParentHandle = Context.LookupMappedRow(ICompatibilityProvider::ObjectMappingDomain, IdKey);

				ParentColumn.Parent = ParentHandle;
			}
		})
	.Where(TColumn<FDataflowSceneObjectTag>() && TColumn<FTypedElementSyncFromWorldTag>())
	.Compile());

	DataStorage.RegisterQuery(
		Select(
			TEXT("Sync dataflow struct hierarchy to column"),
			FProcessor(EQueryTickPhase::PrePhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage))
				.SetExecutionMode(EExecutionMode::GameThread),
			[&DataStorage](IQueryContext& Context, RowHandle ChildHandle,
				const FTypedElementExternalObjectColumn& RawObject, const FTypedElementScriptStructTypeInfoColumn& TypeInfo,  FTableRowParentColumn& ParentColumn)
			{
				if (RawObject.Object && TypeInfo.TypeInfo->IsChildOf(FDataflowBaseElement::StaticStruct()))
				{
					FDataflowBaseElement* SceneObject = static_cast<FDataflowBaseElement*>(RawObject.Object);

					if(SceneObject->ParentElement)
					{
						FMapKeyView IdKey = FMapKeyView(SceneObject->ParentElement);
						RowHandle ParentHandle = Context.LookupMappedRow(ICompatibilityProvider::ObjectMappingDomain, IdKey);

						ParentColumn.Parent = ParentHandle;
					}
				}
			})
		.Where(TColumn<FDataflowSceneStructTag>() && TColumn<FTypedElementSyncFromWorldTag>())
		.Compile());
}

void UDataflowObjectFactory::RegisterLabelQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	using namespace UE::Editor::DataStorage::Queries;
	using namespace UE::Editor::DataStorage;

	DataStorage.RegisterQuery(
	Select(
		TEXT("Sync dataflow object label to column"),
		FProcessor(EQueryTickPhase::PostPhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage))
			.SetExecutionMode(EExecutionMode::GameThread),
		[](IQueryContext& Context, RowHandle RowHandle, 
			const FTypedElementUObjectColumn& RawObject, FTypedElementLabelColumn& ObjectLabel, FTypedElementLabelHashColumn& LabelHash)
		{
			if (const UObject* SceneObject = RawObject.Object.Get())
			{
				const FString ObjectLabelName = SceneObject->GetName();
				const uint64 ObjectLabelHash = CityHash64(reinterpret_cast<const char*>(*ObjectLabelName), ObjectLabelName.Len() * sizeof(**ObjectLabelName));
				if (LabelHash.LabelHash != ObjectLabelHash)
				{
					LabelHash.LabelHash = ObjectLabelHash;
					ObjectLabel.Label = ObjectLabelName;
				}
			}
		})
	.Where(TColumn<FDataflowSceneObjectTag>() && TColumn<FTypedElementSyncFromWorldTag>())
	.Compile());

	DataStorage.RegisterQuery(
		Select(
			TEXT("Sync dataflow struct label to column"),
			FProcessor(EQueryTickPhase::PostPhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage))
				.SetExecutionMode(EExecutionMode::GameThread),
			[](IQueryContext& Context, RowHandle RowHandle, 
				const FTypedElementExternalObjectColumn& RawObject, const FTypedElementScriptStructTypeInfoColumn& TypeInfo, FTypedElementLabelColumn& ObjectLabel, FTypedElementLabelHashColumn& LabelHash)
			{
				if (RawObject.Object && TypeInfo.TypeInfo->IsChildOf(FDataflowBaseElement::StaticStruct()))
				{
					FDataflowBaseElement* SceneObject = static_cast<FDataflowBaseElement*>(RawObject.Object);
					
					const FString ObjectLabelName = SceneObject->ElementName;
					const uint64 ObjectLabelHash = CityHash64(reinterpret_cast<const char*>(*ObjectLabelName), ObjectLabelName.Len() * sizeof(**ObjectLabelName));
					if (LabelHash.LabelHash != ObjectLabelHash)
					{
						LabelHash.LabelHash = ObjectLabelHash;
						ObjectLabel.Label = ObjectLabelName;
					}
				}
			})
		.Where(TColumn<FDataflowSceneStructTag>() && TColumn<FTypedElementSyncFromWorldTag>())
		.Compile());
}

void UDataflowObjectFactory::RegisterVisibilityQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	using namespace UE::Editor::DataStorage::Queries;
	using namespace UE::Editor::DataStorage;

	DataStorage.RegisterQuery(
		Select(
			TEXT("Sync dataflow object visibility to Column"),
			FProcessor(EQueryTickPhase::PrePhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage)).SetExecutionMode(EExecutionMode::GameThread),
			[](IQueryContext& Context, RowHandle RowHandle, const FTypedElementUObjectColumn& ObjectColumn, FVisibleInEditorColumn& VisibilityColumn)
			{
				if (const USceneComponent* SceneComponent = Cast<USceneComponent>(ObjectColumn.Object))
				{
					VisibilityColumn.bIsVisibleInEditor = SceneComponent->IsVisible();
				}
				else if (const AActor* SceneActor = Cast<AActor>(ObjectColumn.Object))
				{
					VisibilityColumn.bIsVisibleInEditor = !SceneActor->IsHiddenEd();
				}
			})
		.Where(TColumn<FDataflowSceneObjectTag>() && TColumn<FTypedElementSyncFromWorldTag>())
		.Compile()
	);

	DataStorage.RegisterQuery(
		Select(
			TEXT("Sync dataflow object visibility to Column"),
			FProcessor(EQueryTickPhase::PrePhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage)).SetExecutionMode(EExecutionMode::GameThread),
			[](IQueryContext& Context, RowHandle RowHandle, const FTypedElementExternalObjectColumn& RawObject, const FTypedElementScriptStructTypeInfoColumn& TypeInfo,  FVisibleInEditorColumn& VisibilityColumn)
			{
				if (RawObject.Object && TypeInfo.TypeInfo->IsChildOf(FDataflowBaseElement::StaticStruct()))
				{
					FDataflowBaseElement* SceneObject = static_cast<FDataflowBaseElement*>(RawObject.Object);
					VisibilityColumn.bIsVisibleInEditor = SceneObject->bIsVisible;
				}
			})
		.Where(TColumn<FDataflowSceneStructTag>() && TColumn<FTypedElementSyncFromWorldTag>())
		.Compile()
	);

	DataStorage.RegisterQuery(
		Select(
			TEXT("Sync visibility Column to dataflow object"),
			FProcessor(EQueryTickPhase::PrePhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncDataStorageToExternal)).SetExecutionMode(EExecutionMode::GameThread),
			[&](IQueryContext& Context, RowHandle ObjectHandle, FTypedElementUObjectColumn& ObjectColumn, const FVisibleInEditorColumn& VisibilityColumn)
			{
				if (USceneComponent* SceneComponent = Cast<USceneComponent>(ObjectColumn.Object))
				{
					Context.PushCommand(UE::Dataflow::Private::FSetComponentVisibilityCommand
					{
						.SceneComponent = SceneComponent,
						.bIsVisible = VisibilityColumn.bIsVisibleInEditor
					});
				}
				else if (AActor* SceneActor = Cast<AActor>(ObjectColumn.Object))
				{
					Context.PushCommand(UE::Dataflow::Private::FSetActorVisibilityCommand
					{
						.SceneActor = SceneActor,
						.bIsVisible = VisibilityColumn.bIsVisibleInEditor
					});
				}
			})
		.Where(TColumn<FDataflowSceneObjectTag>() && TColumn<FVisibilityChangedTag>())
		.Compile()
	);

	DataStorage.RegisterQuery(
		Select(
			TEXT("Sync visibility Column to dataflow object"),
			FProcessor(EQueryTickPhase::PrePhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncDataStorageToExternal)).SetExecutionMode(EExecutionMode::GameThread),
			[&](IQueryContext& Context, RowHandle ObjectHandle, const FTypedElementExternalObjectColumn& RawObject, const FTypedElementScriptStructTypeInfoColumn& TypeInfo,  const FVisibleInEditorColumn& VisibilityColumn)
			{
				if (RawObject.Object && TypeInfo.TypeInfo->IsChildOf(FDataflowBaseElement::StaticStruct()))
				{
					FDataflowBaseElement* SceneObject = static_cast<FDataflowBaseElement*>(RawObject.Object);
					Context.PushCommand(UE::Dataflow::Private::FSetElementVisibilityCommand
					{
						.SceneElement = SceneObject,
						.bIsVisible = VisibilityColumn.bIsVisibleInEditor
					});
				}
			})
		.Where(TColumn<FDataflowSceneStructTag>() && TColumn<FVisibilityChangedTag>())
		.Compile()
	);
}

void UDataflowObjectFactory::RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	Super::RegisterQueries(DataStorage);

	RegisterLabelQueries(DataStorage);
	RegisterHierarchyQueries(DataStorage);
	RegisterVisibilityQueries(DataStorage);
}
