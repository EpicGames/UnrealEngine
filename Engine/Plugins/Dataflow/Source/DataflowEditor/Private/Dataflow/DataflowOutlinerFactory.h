// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Elements/Interfaces/TypedElementDataStorageFactory.h"

#include "DataflowOutlinerFactory.generated.h"

/** Dataflow outliner factory used to register TEDs queries */
UCLASS()
class UDataflowObjectFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	virtual ~UDataflowObjectFactory() override = default;

	//~ Begin UEditorDataStorageFactory interface
	virtual void RegisterTables(UE::Editor::DataStorage::ICoreProvider& DataStorage, UE::Editor::DataStorage::ICompatibilityProvider&) override;
	virtual void RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;
	//~ End UEditorDataStorageFactory interface

protected:
	/** Register all the visibility queries */
	void RegisterVisibilityQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage);

	/** Register all the label queries */
	void RegisterLabelQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage);

	/** Register all the hierarchy queries */
	void RegisterHierarchyQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage);
};
