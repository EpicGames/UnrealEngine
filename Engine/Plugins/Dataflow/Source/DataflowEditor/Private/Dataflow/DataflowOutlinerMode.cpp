// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowOutlinerMode.h"
#include "Dataflow/DataflowConstructionScene.h"
#include "Dataflow/DataflowSimulationScene.h"
#include "AssetEditorModeManager.h"
#include "EditorViewportCommands.h"
#include "TedsOutlinerItem.h"
#include "Selection.h"
#include "Components/PrimitiveComponent.h"
#include "Framework/Application/SlateApplication.h"
#include "Dataflow/DataflowContent.h"
#include "Dataflow/DataflowElement.h"
#include "DataStorage/Features.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementVisibilityColumns.h"
#include "Elements/Columns/TypedElementTypeInfoColumns.h"

struct FTypedElementScriptStructTypeInfoColumn;

namespace UE::Dataflow::Private
{
	template<typename ObjectType>
	static ObjectType* GetOutlinerItemObject(const TWeakPtr<ISceneOutlinerTreeItem>& WeakTreeItem)
	{
		using namespace UE::Editor::Outliner;
		using namespace UE::Editor::DataStorage;

		if (const TSharedPtr<ISceneOutlinerTreeItem> TreeItem = WeakTreeItem.Pin())
		{
			if (FTedsOutlinerTreeItem* TEDSItem = TreeItem->CastTo<FTedsOutlinerTreeItem>())
			{
				ICoreProvider* Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
				if (TEDSItem->IsValid() && Storage)
				{
					const FTypedElementUObjectColumn* RawObjectColumn = Storage->GetColumn<FTypedElementUObjectColumn>(TEDSItem->GetRowHandle());
					return RawObjectColumn ? Cast<ObjectType>(RawObjectColumn->Object) : nullptr;
				}
			}
		}
	
		return nullptr;
	}

	template<typename ObjectType>
	static ObjectType* GetOutlinerItemStruct(const TWeakPtr<ISceneOutlinerTreeItem>& WeakTreeItem)
	{
		using namespace UE::Editor::Outliner;
		using namespace UE::Editor::DataStorage;
		
		if (const TSharedPtr<ISceneOutlinerTreeItem> TreeItem = WeakTreeItem.Pin())
		{
			if (FTedsOutlinerTreeItem* TEDSItem = TreeItem->CastTo<FTedsOutlinerTreeItem>())
			{
				ICoreProvider* Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
				if (TEDSItem->IsValid() && Storage)
				{
					const FTypedElementExternalObjectColumn* RawObjectColumn = Storage->GetColumn<FTypedElementExternalObjectColumn>(TEDSItem->GetRowHandle());
					const FTypedElementScriptStructTypeInfoColumn* TypeInfoColumn = Storage->GetColumn<FTypedElementScriptStructTypeInfoColumn>(TEDSItem->GetRowHandle());
					if (RawObjectColumn && TypeInfoColumn)
					{
						if (RawObjectColumn->Object && TypeInfoColumn->TypeInfo == ObjectType::StaticStruct())
						{
							return static_cast<ObjectType*>(RawObjectColumn->Object);
						}
					}
				}
			}
		}
	
		return nullptr;
	}
	
	static void SetOutlinerItemVisibility(const TWeakPtr<ISceneOutlinerTreeItem>& WeakTreeItem, const bool bIsVisible)
	{
		using namespace UE::Editor::Outliner;
		using namespace UE::Editor::DataStorage;

		if (const TSharedPtr<ISceneOutlinerTreeItem> TreeItem = WeakTreeItem.Pin())
		{
			if (FTedsOutlinerTreeItem* TEDSItem = TreeItem->CastTo<FTedsOutlinerTreeItem>())
			{
				ICoreProvider* Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
				if (TEDSItem->IsValid() && Storage)
				{
					RowHandle Row = TEDSItem->GetRowHandle();
					if(FVisibleInEditorColumn* VisibilityColumn = Storage->GetColumn<FVisibleInEditorColumn>(Row))
					{
						VisibilityColumn->bIsVisibleInEditor = bIsVisible;
						Storage->AddColumn<FVisibilityChangedTag>(Row);
					}
				}
			}
		}
	}

	/** Functor which can be used to get dataflow actors from a selection */
	template<typename ObjectType>
	struct FDataflowObjectSelector
	{
		FDataflowObjectSelector()
		{}

		bool operator()(const TWeakPtr<ISceneOutlinerTreeItem>& TreeItem, ObjectType*& SceneObject) const
		{
			if (ObjectType* ItemObject = GetOutlinerItemObject<ObjectType>(TreeItem))
			{
				SceneObject = ItemObject;
				return true;
			}
			return false;
		}
	};
	
	/** Functor which can be used to get dataflow actors from a selection */
	template<typename ObjectType>
	struct FDataflowStructSelector
	{
		FDataflowStructSelector()
		{}

		bool operator()(const TWeakPtr<ISceneOutlinerTreeItem>& TreeItem, ObjectType*& SceneObject) const
		{
			if (ObjectType* ItemObject = GetOutlinerItemStruct<ObjectType>(TreeItem))
			{
				SceneObject = ItemObject;
				return true;
			}
			return false;
		}
	};
	
	static void UpdateSceneSelection(const FSceneOutlinerItemSelection& Selection, FDataflowPreviewSceneBase* PreviewScene, USelection* SceneSelection)
	{
		// Get the selected components in TEDS
		TArray<UPrimitiveComponent*> SelectedComponents = Selection.GetData<UPrimitiveComponent*>(UE::Dataflow::Private::FDataflowObjectSelector<UPrimitiveComponent>());

		// Unselect all previous components
		TArray<UPrimitiveComponent*> PreviousSelection;
		SceneSelection->GetSelectedObjects<UPrimitiveComponent>(PreviousSelection);

		SceneSelection->Modify();
		SceneSelection->BeginBatchSelectOperation();
		SceneSelection->DeselectAll();

		// Transfer components selection from TEDS
		for(UPrimitiveComponent* SelectedComponent : SelectedComponents)
		{
			if(SelectedComponent->GetWorld() == PreviewScene->GetWorld())
			{
				SceneSelection->Select(SelectedComponent);
				SelectedComponent->PushSelectionToProxy();
			}
		}
		SceneSelection->EndBatchSelectOperation();

		// Update the previous proxies 
		for(UPrimitiveComponent* PreviousComponent : PreviousSelection)
		{
			if(PreviousComponent->GetWorld() == PreviewScene->GetWorld())
			{
				PreviousComponent->PushSelectionToProxy();
			}
		}

		// Get the selected elements in TEDS
		TArray<FDataflowBaseElement*> SelectedElements = Selection.GetData<FDataflowBaseElement*>(UE::Dataflow::Private::FDataflowStructSelector<FDataflowBaseElement>());

		// Unselect all previous elements
		for(IDataflowDebugDrawInterface::FDataflowElementsType::ElementType& SelectedElement : PreviewScene->ModifySceneElements())
		{
			if(SelectedElement.IsValid())
			{
				SelectedElement->bIsSelected = false;
			}
		}

		// Transfer elements selection from TEDS
		for(FDataflowBaseElement* SelectedElement : SelectedElements)
		{
			if(SelectedElement)
			{
				SelectedElement->bIsSelected = true;
			}
		}
	}
}

FDataflowOutlinerMode::FDataflowOutlinerMode(const UE::Editor::Outliner::FTedsOutlinerParams& InModeParams,
		TWeakPtr<FDataflowConstructionScene> InConstructionScene, TWeakPtr<FDataflowSimulationScene> InSimulationScene)
	: FTedsOutlinerMode(InModeParams)
	, ConstructionScene(InConstructionScene)
	, SimulationScene(InSimulationScene)
{
	if (SceneOutliner)
	{
		TAttribute<bool> ConditionalEnabledAttribute(true);
		ConditionalEnabledAttribute.BindRaw(this, &FDataflowOutlinerMode::CanPopulate);

		SceneOutliner->SetEnabled(ConditionalEnabledAttribute);
	}
}

FDataflowOutlinerMode::~FDataflowOutlinerMode()
{
	if (SceneOutliner)
	{
		TAttribute<bool> EmptyConditionalEnabledAttribute;
		SceneOutliner->SetEnabled(EmptyConditionalEnabledAttribute);
	}
}

void FDataflowOutlinerMode::OnItemSelectionChanged(FSceneOutlinerTreeItemPtr Item, ESelectInfo::Type SelectionType, const FSceneOutlinerItemSelection& Selection)
{
	const TSharedPtr<FDataflowConstructionScene> ConstructionScenePtr = ConstructionScene.Pin();
	const TSharedPtr<FDataflowSimulationScene> SimulationScenePtr = SimulationScene.Pin();
	if ((SelectionType == ESelectInfo::Direct) || !ConstructionScenePtr || !SimulationScenePtr)
	{
		return;
	}
	if (const TSharedPtr<FAssetEditorModeManager> ConstructionManager = ConstructionScenePtr->GetDataflowModeManager())
	{
		UE::Dataflow::Private::UpdateSceneSelection(Selection, ConstructionScenePtr.Get(), ConstructionManager->GetSelectedComponents());
	}
	if (const TSharedPtr<FAssetEditorModeManager> SimulationManager = SimulationScenePtr->GetDataflowModeManager())
	{
		UE::Dataflow::Private::UpdateSceneSelection(Selection, SimulationScenePtr.Get(), SimulationManager->GetSelectedComponents());
	}
}

void FDataflowOutlinerMode::OnItemAdded(FSceneOutlinerTreeItemPtr Item)
{
	const UObject* AddedObject = UE::Dataflow::Private::GetOutlinerItemObject<UObject>(Item);
	const FName ObjectName = AddedObject ? AddedObject->GetFName(): NAME_None;

	// auto expand the construction view object since this is the one we use the most 
	bool bShouldExpand = false;
	if (const TSharedPtr<FDataflowConstructionScene> ConstructionScenePtr = ConstructionScene.Pin())
	{
		if (ConstructionScenePtr->GetRootActor() == AddedObject)
		{
			bShouldExpand = true;
		}
	}
	SceneOutliner->SetItemExpansion(Item, bShouldExpand);
	Item->Flags.bIsExpanded = bShouldExpand;
}

void FDataflowOutlinerMode::OnItemDoubleClick(FSceneOutlinerTreeItemPtr SelectedItem)
{
	const TSharedPtr<FDataflowConstructionScene> ConstructionScenePtr = ConstructionScene.Pin();
	const TSharedPtr<FDataflowSimulationScene> SimulationScenePtr = SimulationScene.Pin();
	if (!ConstructionScenePtr || !SimulationScenePtr)
	{
		return;
	}

	if(const UPrimitiveComponent* SelectedComponent = UE::Dataflow::Private::GetOutlinerItemObject<UPrimitiveComponent>(SelectedItem))
	{
		if(SelectedComponent->GetWorld() == ConstructionScenePtr->GetWorld())
		{
			ConstructionScenePtr->OnFocusRequest().Broadcast(SelectedComponent->Bounds.GetBox());
		}
		else if(SelectedComponent->GetWorld() == SimulationScenePtr->GetWorld())
		{
			SimulationScenePtr->OnFocusRequest().Broadcast(SelectedComponent->Bounds.GetBox());
		}
	}
	else if(FDataflowBaseElement* SelectedElement = UE::Dataflow::Private::GetOutlinerItemStruct<FDataflowBaseElement>(SelectedItem))
	{
		if(SelectedElement->bIsConstruction)
		{
			ConstructionScenePtr->OnFocusRequest().Broadcast(SelectedElement->BoundingBox);
		}
		else
		{
			SimulationScenePtr->OnFocusRequest().Broadcast(SelectedElement->BoundingBox);
		}
	}
}

FReply FDataflowOutlinerMode::OnKeyDown(const FKeyEvent& InKeyEvent)
{
	const TSharedPtr<FDataflowConstructionScene> ConstructionScenePtr = ConstructionScene.Pin();
	const TSharedPtr<FDataflowSimulationScene> SimulationScenePtr = SimulationScene.Pin();
	if (SceneOutliner && ConstructionScenePtr && SimulationScenePtr)
	{
		const FModifierKeysState ModifierKeys = FSlateApplication::Get().GetModifierKeys();
		const FInputChord CheckChord( InKeyEvent.GetKey(), EModifierKey::FromBools(ModifierKeys.IsControlDown(), ModifierKeys.IsAltDown(), ModifierKeys.IsShiftDown(), ModifierKeys.IsCommandDown()) );

		const FSceneOutlinerItemSelection& Selection = SceneOutliner->GetSelection();
	
		// Use the keyboard shortcut bound to 'Focus Viewport To Selection'
		if (FEditorViewportCommands::Get().FocusViewportToSelection->HasActiveChord(CheckChord))
		{
			FBox ConstructionBox, SimulationBox;
			for(TWeakPtr<ISceneOutlinerTreeItem>& SelectedItem: Selection.SelectedItems)
			{
				if(const UPrimitiveComponent* SelectedComponent = UE::Dataflow::Private::GetOutlinerItemObject<UPrimitiveComponent>(SelectedItem))
				{
					if(SelectedComponent->GetWorld() == ConstructionScenePtr->GetWorld())
					{
						ConstructionBox += SelectedComponent->Bounds.GetBox();
					}
					else if(SelectedComponent->GetWorld() == SimulationScenePtr->GetWorld())
					{
						SimulationBox += SelectedComponent->Bounds.GetBox();
					}
				}
				else if(FDataflowBaseElement* SelectedElement = UE::Dataflow::Private::GetOutlinerItemStruct<FDataflowBaseElement>(SelectedItem))
				{
					if(SelectedElement->bIsConstruction)
					{
						ConstructionBox += SelectedElement->BoundingBox;
					}
					else
					{
						SimulationBox += SelectedElement->BoundingBox;
					}
				}
			}
			if(ConstructionBox.IsValid)
			{
				ConstructionScenePtr->OnFocusRequest().Broadcast(ConstructionBox);
			}
			if(SimulationBox.IsValid)
			{
				SimulationScenePtr->OnFocusRequest().Broadcast(SimulationBox);
			}
		}
		else if (InKeyEvent.GetKey() == EKeys::H)
		{
			const bool bIsVisible = (InKeyEvent.IsShiftDown() || InKeyEvent.IsControlDown()) ? true : false;
			for(TWeakPtr<ISceneOutlinerTreeItem>& SelectedItem : Selection.SelectedItems)
			{
				if(UPrimitiveComponent* SelectedComponent = UE::Dataflow::Private::GetOutlinerItemObject<UPrimitiveComponent>(SelectedItem))
				{
					SelectedComponent->SetVisibility(bIsVisible);
				}
				else if(FDataflowBaseElement* SelectedElement = UE::Dataflow::Private::GetOutlinerItemStruct<FDataflowBaseElement>(SelectedItem))
				{
					SelectedElement->bIsVisible = bIsVisible;
				}
				UE::Dataflow::Private::SetOutlinerItemVisibility(SelectedItem, bIsVisible);
			}
		}
	}
	return UE::Editor::Outliner::FTedsOutlinerMode::OnKeyDown(InKeyEvent);
}
