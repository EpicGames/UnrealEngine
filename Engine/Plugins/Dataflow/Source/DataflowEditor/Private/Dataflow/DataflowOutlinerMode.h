// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "TedsOutlinerMode.h"

class FDataflowConstructionScene;
class FDataflowSimulationScene;

/**
 * Dataflow outliner mode used to represent the dataflow scenes (construction/simulation)
 * It has a more limited view compared to the normal outliner, hiding features we don't support,
 */
class FDataflowOutlinerMode : public UE::Editor::Outliner::FTedsOutlinerMode
{
public:

	FDataflowOutlinerMode(const UE::Editor::Outliner::FTedsOutlinerParams& InModeParams,
		TWeakPtr<FDataflowConstructionScene> InConstructionScene, TWeakPtr<FDataflowSimulationScene> InSimulationScene);

	virtual ~FDataflowOutlinerMode() override;

	//~ Begin FTedsOutlinerMode interface
	virtual void OnItemSelectionChanged(FSceneOutlinerTreeItemPtr Item, ESelectInfo::Type SelectionType, const FSceneOutlinerItemSelection& Selection) override;
	virtual void OnItemDoubleClick(FSceneOutlinerTreeItemPtr Item) override;
	virtual void OnItemAdded(FSceneOutlinerTreeItemPtr Item) override;
	virtual bool CanInteract(const ISceneOutlinerTreeItem& Item) const override {return true;}
	virtual bool CanPopulate() const override {return true;}
	virtual bool CanSupportDragAndDrop() const override { return false; }
	virtual ESelectionMode::Type GetSelectionMode() const override { return ESelectionMode::Multi; }
	virtual FReply OnKeyDown(const FKeyEvent& InKeyEvent) override;
	//~ End FTedsOutlinerMode interface

private :
	/** Construction dataflow scene */
	TWeakPtr<FDataflowConstructionScene> ConstructionScene;

	/** Simulation dataflow scene */
	TWeakPtr<FDataflowSimulationScene> SimulationScene;
};
