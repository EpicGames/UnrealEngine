// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowOutlinerView.h"

#include "Compatibility/SceneOutlinerTedsBridge.h"
#include "Dataflow/DataflowEditorPreviewSceneBase.h"
#include "Dataflow/DataflowOutlinerMode.h"
#include "Dataflow/DataflowContent.h"
#include "DataStorage/Features.h"
#include "DataStorage/Queries/Description.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementTypeInfoColumns.h"
#include "Elements/Columns/TypedElementVisibilityColumns.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "TedsOutlinerImpl.h"
#include "TedsOutlinerItem.h"
#include "TedsOutlinerFilter.h"

#define LOCTEXT_NAMESPACE "DataflowOutlinerView"

FDataflowOutlinerView::FDataflowOutlinerView(TWeakPtr<FDataflowConstructionScene> InConstructionScene, TWeakPtr<FDataflowSimulationScene> InSimulationScene, TObjectPtr<UDataflowBaseContent> InContent)
	: FDataflowNodeView(InContent)
	, OutlinerWidget(nullptr)
	, ConstructionScene(InConstructionScene)
	, SimulationScene(InSimulationScene)
{
	check(InContent);
}

FDataflowOutlinerView::~FDataflowOutlinerView()
{}

TSharedPtr<ISceneOutliner> FDataflowOutlinerView::CreateWidget()
{
	using namespace UE::Editor::DataStorage::Queries;
	using namespace UE::Editor::Outliner;

	ICoreProvider* Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
	
	const UE::Editor::DataStorage::FQueryDescription RowQueryDescription = 
		Select()
		.Where(TColumn<FDataflowSceneObjectTag>(GetEditorContent()->GetDataflowOwner().GetFName()) || TColumn<FDataflowSceneStructTag>(GetEditorContent()->GetDataflowOwner().GetFName()) )
		.Compile();
		
	const TMap<TWeakObjectPtr<const UScriptStruct>, FTedsOutlinerColumnParams> ColumnParams
	{
		{FVisibleInEditorColumn::StaticStruct(), FTedsOutlinerColumnParams(FTedsOutlinerColumnParams::EColumnPriorityGroup::Left)}
	};

	const UE::Editor::Outliner::FTedsOutlinerColumnDescription ColumnDescription({
		FTypedElementClassTypeInfoColumn::StaticStruct(),
		FVisibleInEditorColumn::StaticStruct()
	}, ColumnParams);

	FSceneOutlinerInitializationOptions InitOptions;
	InitOptions.bShowHeaderRow = true;
	InitOptions.FilterBarOptions.bHasFilterBar = true;
	InitOptions.bShowTransient = false;
	InitOptions.OutlinerIdentifier = "DataflowOutliner";

	// Make sure we hide component that have the Hidden flag set ( ie: UCLASS(Hidden) )
	InitOptions.Filters->AddFilterPredicate<UE::Editor::Outliner::FTedsOutlinerTreeItem>(
		UE::Editor::Outliner::FTedsOutlinerTreeItem::FFilterPredicate::CreateLambda(
			[](const UE::Editor::DataStorage::RowHandle RowHandle)
			{
				using namespace UE::Editor::DataStorage;
				if (const ICoreProvider* Storage = GetDataStorageFeature<ICoreProvider>(StorageFeatureName))
				{
					if (const FTypedElementUObjectColumn* Col = Storage->GetColumn<FTypedElementUObjectColumn>(RowHandle))
					{
						if (const UObject* Obj = Col->Object.Get())
						{
							return !Obj->GetClass()->HasAnyClassFlags(CLASS_Hidden);
						}
					}
				}
				return true;
			}),
		FSceneOutlinerFilter::EDefaultBehaviour::Pass
	);

	
	UE::Editor::Outliner::FTedsOutlinerParams Params(nullptr);
	Params.QueryDescription = RowQueryDescription;
	Params.bShowRowHandleColumn = false;
	Params.ColumnDescription = ColumnDescription;
	
	// Add outliner filter queries
	Params.Filters.Add(MakeShared<FTedsOutlinerFilter>("Dataflow Construction",
		LOCTEXT("FilterDataflowConstructionDisplayName", "Dataflow Construction"),
		Storage->RegisterQuery(
			Select()
			.Where(TColumn<FDataflowConstructionObjectTag>())
			.Compile())));
	Params.Filters.Add(MakeShared<FTedsOutlinerFilter>("Dataflow Simulation",
		LOCTEXT("FilterDataflowSimulationDisplayName", "Dataflow Simulation"),
		Storage->RegisterQuery(
			Select()
			.Where(TColumn<FDataflowSimulationObjectTag>())
			.Compile())));
	Params.Filters.Add(MakeShared<FTedsOutlinerFilter>("Dataflow Elements",
		LOCTEXT("FilterDataflowElementsDisplayName", "Dataflow Elements"),
		Storage->RegisterQuery(
			Select()
			.Where(TColumn<FDataflowSceneStructTag>(GetEditorContent()->GetDataflowOwner().GetFName()))
			.Compile())));
	Params.Filters.Add(MakeShared<FTedsOutlinerFilter>("Dataflow Components",
		LOCTEXT("FilterDataflowComponentsDisplayName", "Dataflow Components"),
		Storage->RegisterQuery(
			Select()
			.Where(TColumn<FDataflowSceneObjectTag>(GetEditorContent()->GetDataflowOwner().GetFName()))
			.Compile())));

	InitOptions.ModeFactory = FCreateSceneOutlinerMode::CreateLambda([this, &Params](SSceneOutliner* Outliner)
	{
		Params.SceneOutliner = Outliner;
		return new FDataflowOutlinerMode(Params, ConstructionScene, SimulationScene);
	});

	OutlinerWidget = SNew(SSceneOutliner, InitOptions);

	return OutlinerWidget;
}

void FDataflowOutlinerView::SetSupportedOutputTypes()
{
	GetSupportedOutputTypes().Empty();
	GetSupportedOutputTypes().Add("FManagedArrayCollection");
}

void FDataflowOutlinerView::RefreshView()
{
	UpdateViewData();
}

void FDataflowOutlinerView::UpdateViewData()
{
	if(OutlinerWidget)
	{
		OutlinerWidget->CollapseAll();
		OutlinerWidget->FullRefresh();
	}
}

void FDataflowOutlinerView::ConstructionViewSelectionChanged(const TArray<UPrimitiveComponent*>& SelectedComponents, const TArray<FDataflowBaseElement*>& SelectedElements)
{
	OutlinerWidget->ClearSelection();
	
	using namespace UE::Editor::DataStorage;
	TArray<RowHandle> SelectedRowHandles;
	if (const ICompatibilityProvider* Compatibility = GetDataStorageFeature<ICompatibilityProvider>(CompatibilityFeatureName))
	{
		// Transfer components selection to outliner
		for(UPrimitiveComponent* SelectedComponent : SelectedComponents)
		{
			if (FSceneOutlinerTreeItemPtr SelectedTreeItem = OutlinerWidget->GetTreeItem(
				Compatibility->FindRowWithCompatibleObject(SelectedComponent), true))
			{
				OutlinerWidget->AddToSelection(SelectedTreeItem);
				OutlinerWidget->ScrollItemIntoView(SelectedTreeItem);
			}
		}
		// Transfer elements selection to outliner
		for(FDataflowBaseElement* SelectedElement : SelectedElements)
		{
			if (FSceneOutlinerTreeItemPtr SelectedTreeItem = OutlinerWidget->GetTreeItem(
				Compatibility->FindRowWithCompatibleObject(SelectedElement), true))
			{
				OutlinerWidget->AddToSelection(SelectedTreeItem);
				OutlinerWidget->ScrollItemIntoView(SelectedTreeItem);
			}
		}
	}
}

void FDataflowOutlinerView::SimulationViewSelectionChanged(const TArray<UPrimitiveComponent*>& SelectedComponents, const TArray<FDataflowBaseElement*>& SelectedElements)
{
	ConstructionViewSelectionChanged(SelectedComponents, SelectedElements);
}

void FDataflowOutlinerView::AddReferencedObjects(FReferenceCollector& Collector)
{
	FDataflowNodeView::AddReferencedObjects(Collector);
}

#undef LOCTEXT_NAMESPACE