// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Dataflow/DataflowView.h"

class SSceneOutliner;
class ISceneOutliner;
class FDataflowConstructionScene;
class FDataflowSimulationScene;
struct FDataflowBaseElement;

/** Class to handle the dataflow outliner widget */
class FDataflowOutlinerView : public FDataflowNodeView
{
public:
	FDataflowOutlinerView(TWeakPtr<FDataflowConstructionScene> InConstructionScene, TWeakPtr<FDataflowSimulationScene> InSimulationScene, TObjectPtr<UDataflowBaseContent> InContent = nullptr);
	~FDataflowOutlinerView();

	/** Create the outliner widget */
	TSharedPtr<ISceneOutliner> CreateWidget();

	/** Set the supported output types */
	virtual void SetSupportedOutputTypes() override;

	/** Update the view if necessary */
	virtual void UpdateViewData() override;

	/** Refresh the view if necessary */
	virtual void RefreshView() override;

	/** Update the view based on changes in the construction view */
	virtual void ConstructionViewSelectionChanged(const TArray<UPrimitiveComponent*>& SelectedComponents, const TArray<FDataflowBaseElement*>& SelectedElements) override;
	
	/** Update the view based on changes in the construction view */
	virtual void SimulationViewSelectionChanged(const TArray<UPrimitiveComponent*>& SelectedComponents, const TArray<FDataflowBaseElement*>& SelectedElements) override;

	/** Add GC managed objects*/
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;

private:
	/** Outliner widget stored on the view */
	TSharedPtr<SSceneOutliner> OutlinerWidget;

	/** Construction scene the outliner could refer to */
	TWeakPtr<FDataflowConstructionScene> ConstructionScene;

	/** Simulation scene the outliner could refer to */
	TWeakPtr<FDataflowSimulationScene> SimulationScene;
};

