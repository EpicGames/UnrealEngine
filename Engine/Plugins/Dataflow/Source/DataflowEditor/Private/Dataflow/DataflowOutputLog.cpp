// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowOutputLog.h"
#include "IMessageLogListing.h"
#include "MessageLogModule.h"
#include "Dataflow/DataflowPath.h"

DEFINE_LOG_CATEGORY_STATIC(LogDataflowContextOutput, Warning, All);

FDataflowOutputLog::FDataflowOutputLog(TObjectPtr<UDataflowBaseContent> InContent)
	: FDataflowNodeView(InContent)
{
	CreateMessageLog();
	CreateMessageLogWidget();
}

FDataflowOutputLog::~FDataflowOutputLog()
{
	if (MessageLogListing)
	{
		MessageLogListing->OnMessageTokenClicked().RemoveAll(this);
	}
}

void FDataflowOutputLog::CreateMessageLog()
{
	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");

	const FString NameString = FString::Printf(TEXT("LogDataflow%lld"), UE::Dataflow::FTimestamp::Current());
	const FName MessageLogName = FName(*NameString);

	if (MessageLogModule.IsRegisteredLogListing(MessageLogName))
	{
		MessageLogListing = MessageLogModule.GetLogListing(MessageLogName);
	}
	else
	{
		FMessageLogInitializationOptions LogOptions;
		LogOptions.bShowPages = false;
		LogOptions.bShowFilters = true;
		LogOptions.bAllowClear = true;
		LogOptions.MaxPageCount = 1;
		LogOptions.bShowInLogWindow = false;

		MessageLogListing = MessageLogModule.CreateLogListing(MessageLogName, LogOptions);
	}

	if (MessageLogListing.IsValid())
	{
		MessageLogListing->OnMessageTokenClicked().AddRaw(this, &FDataflowOutputLog::OnMessageTokenClicked);
		MessageLogListing->ClearMessages();
	}
}

TSharedRef<IMessageLogListing> FDataflowOutputLog::GetMessageLog() const
{
	return MessageLogListing.ToSharedRef();
}

void FDataflowOutputLog::ClearMessageLog()
{
	if (MessageLogListing.IsValid())
	{
		MessageLogListing->ClearMessages();
	}
}

namespace UE::Dataflow::Private
{
	TSharedRef<FTokenizedMessage> BuildTokenizedMessage(const EMessageSeverity::Type InSeverity, const FText& InMessage, const FDataflowPath& InPath)
	{
		// Token0 - Severity
		// Token1 - Time code
		// Token2 - Path ()
		// Token3 - Message

		const FDateTime TimeNow = FDateTime::Now();
		const FString TimeString = FString::Printf(TEXT("[%02d:%02d:%02d:%02d]"), TimeNow.GetHour(), TimeNow.GetMinute(), TimeNow.GetSecond(), TimeNow.GetMillisecond());

		TSharedRef<FTokenizedMessage> TokenizedMessage = FTokenizedMessage::Create(InSeverity);
		TSharedRef<IMessageToken> TokenTimecode = FTextToken::Create(FText::FromString(TimeString));
		TokenizedMessage->AddToken(TokenTimecode);
		TSharedRef<IMessageToken> TokenPath = FTextToken::Create(FText::FromString("[" + InPath.ToString() + "]"));
		TokenizedMessage->AddToken(TokenPath);
		TokenizedMessage->SetMessageLink(TokenPath);
		TSharedRef<IMessageToken> TokenMessage = FTextToken::Create(InMessage);
		TokenizedMessage->AddToken(TokenMessage);

		return TokenizedMessage;
	}

	FString GetTokenString(const TSharedRef<IMessageToken>& InMessageToken)
	{
		// Remove [] from InToken
		FString TokenString = InMessageToken->ToText().ToString();
		TokenString = TokenString.TrimChar(TEXT('['));
		TokenString = TokenString.TrimChar(TEXT(']'));

		return TokenString;
	}
}

void FDataflowOutputLog::AddMessage(const EMessageSeverity::Type InSeverity, const FString& InMessage, const FDataflowPath& InPath)
{
	const FText TextMessage = FText::FromString(InMessage);
	AddMessage(InSeverity, TextMessage, InPath);
}

void FDataflowOutputLog::AddMessage(const EMessageSeverity::Type InSeverity, const FText& InMessage, const FDataflowPath& InPath)
{
	if (MessageLogListing.IsValid())
	{
		TSharedRef<FTokenizedMessage> TokenizedMessage = UE::Dataflow::Private::BuildTokenizedMessage(InSeverity, InMessage, InPath);
		MessageLogListing->AddMessage(TokenizedMessage, false);
	}
	UE_LOGF(LogDataflowContextOutput, Verbose, "[%ls][%ls]", *InPath.ToString(), *InMessage.ToString());
}

void FDataflowOutputLog::CreateMessageLogWidget()
{
	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");

	OutputLogWidget = MessageLogModule.CreateLogListingWidget(MessageLogListing.ToSharedRef());
}

void FDataflowOutputLog::OnMessageTokenClicked(const TSharedRef<IMessageToken>& InMessageToken)
{
	FString TokenString = UE::Dataflow::Private::GetTokenString(InMessageToken);

	if (OnOutputLogMessageTokenClickedDelegate.IsBound())
	{
		OnOutputLogMessageTokenClickedDelegate.Broadcast(TokenString);
	}
}

