// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Dataflow/DataflowView.h"

namespace UE::Dataflow
{
	class SOutputLogWidget;
}

struct FDataflowPath;
class IMessageLogListing;

/**
*
* Class to handle the OutputLog widget
*
*/
class FDataflowOutputLog final : public FDataflowNodeView
{
public:
	explicit FDataflowOutputLog(TObjectPtr<UDataflowBaseContent> InContent = nullptr);
	virtual ~FDataflowOutputLog();

	TSharedPtr<SWidget> GetOutputLogWidget() { return OutputLogWidget; }

	TSharedRef<IMessageLogListing> GetMessageLog() const;
	void ClearMessageLog();
	
	UE_DEPRECATED(5.8, "AddMessage with a FString parameter is now deprecated, use the FText version")
	void AddMessage(const EMessageSeverity::Type InSeverity, const FString& InMessage, const FDataflowPath& InPath);
	
	void AddMessage(const EMessageSeverity::Type InSeverity, const FText& InMessage, const FDataflowPath& InPath);

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnOutputLogMessageTokenClicked, const FString);
	FOnOutputLogMessageTokenClicked& GetOnOutputLogMessageTokenClickedDelegate() { return OnOutputLogMessageTokenClickedDelegate; }

private:
	virtual void SetSupportedOutputTypes() override {};
	virtual void UpdateViewData() override {};
	virtual void ConstructionViewSelectionChanged(const TArray<UPrimitiveComponent*>& SelectedComponents, const TArray<FDataflowBaseElement*>& SelectedElements) override {};
	virtual void SimulationViewSelectionChanged(const TArray<UPrimitiveComponent*>& SelectedComponents, const TArray<FDataflowBaseElement*>& SelectedElements) override {};

	void OnMessageTokenClicked(const TSharedRef<IMessageToken>& InMessageToken);

	void CreateMessageLog();
	void CreateMessageLogWidget();

	TSharedPtr<SWidget> OutputLogWidget;
	TSharedPtr<IMessageLogListing> MessageLogListing;

	FOnOutputLogMessageTokenClicked OnOutputLogMessageTokenClickedDelegate;
};

