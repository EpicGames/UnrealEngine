// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowPreviewProfileController.h"
#include "AssetViewerSettings.h"

FDataflowPreviewProfileController::FDataflowPreviewProfileController(TSharedPtr<IProfileIndexStorage> ProfileIndexStorage) :
	ProfileIndexStorage(ProfileIndexStorage)
{
	AssetViewerSettings = UAssetViewerSettings::Get();
	if (AssetViewerSettings)
	{
		AssetViewerSettingsProfileAddRemoveHandle = AssetViewerSettings->OnAssetViewerProfileAddRemoved().AddLambda([this]()
		{
			UpdateAssetViewerProfiles();
			OnPreviewProfileListChanged().Broadcast();
		});

		AssetViewerSettingsChangedHandle = AssetViewerSettings->OnAssetViewerSettingsChanged().AddLambda([this](const FName& InPropertyName)
		{
			FString CurrProfileName = AssetViewerProfileNames[CurrentProfileIndex];
			UpdateAssetViewerProfiles();
			if (CurrProfileName != AssetViewerProfileNames[CurrentProfileIndex])
			{
				OnPreviewProfileChanged().Broadcast();
			}
		});
			
		UpdateAssetViewerProfiles();
	}
}

FDataflowPreviewProfileController::~FDataflowPreviewProfileController()
{
	if (IsValid(AssetViewerSettings))
	{
		AssetViewerSettings->OnAssetViewerProfileAddRemoved().Remove(AssetViewerSettingsProfileAddRemoveHandle);
		AssetViewerSettings->OnAssetViewerSettingsChanged().Remove(AssetViewerSettingsChangedHandle);
	}
}

void FDataflowPreviewProfileController::UpdateAssetViewerProfiles()
{
	AssetViewerProfileNames.Empty();

	if (AssetViewerSettings && ProfileIndexStorage)
	{
		// Rebuild the profile list.
		for (const FPreviewSceneProfile& Profile : AssetViewerSettings->Profiles)
		{
			AssetViewerProfileNames.Add(Profile.ProfileName);
		}

		CurrentProfileIndex = ProfileIndexStorage->RetrieveProfileIndex();

		EnsureProfilesStateCoherence();
	}
}

TArray<FString> FDataflowPreviewProfileController::GetPreviewProfiles(int32& OutCurrentProfileIndex) const
{
	if (AssetViewerSettings)
	{
		EnsureProfilesStateCoherence();
		OutCurrentProfileIndex = CurrentProfileIndex;
	}
	return AssetViewerProfileNames;
}

bool FDataflowPreviewProfileController::SetActiveProfile(const FString& ProfileName)
{
	if (ProfileIndexStorage && IsValid(AssetViewerSettings))
	{
		EnsureProfilesStateCoherence();

		int32 Index = AssetViewerProfileNames.IndexOfByKey(ProfileName);
		if (Index != INDEX_NONE && Index != ProfileIndexStorage->RetrieveProfileIndex())
		{
			// Store the settings.
			ProfileIndexStorage->StoreProfileIndex(Index);
			CurrentProfileIndex = Index;

			// Notify the observer about the change.
			AssetViewerSettings->OnAssetViewerSettingsChanged().Broadcast(GET_MEMBER_NAME_CHECKED(FPreviewSceneProfile, ProfileName));
			return true;
		}
	}
	return false;
}

FString FDataflowPreviewProfileController::GetActiveProfile() const
{
	if (AssetViewerSettings)
	{
		EnsureProfilesStateCoherence();
		return AssetViewerProfileNames[CurrentProfileIndex];
	}
	return FString();
}

bool FDataflowPreviewProfileController::HasAnyUserProfiles() const
{
	if (AssetViewerSettings)
	{
		for (const FPreviewSceneProfile& Profile : AssetViewerSettings->Profiles)
		{
			if (!Profile.bIsEngineDefaultProfile)
			{
				return true;
			}
		}
	}
	
	return false;
}

void FDataflowPreviewProfileController::EnsureProfilesStateCoherence() const
{
	ensureMsgf(AssetViewerProfileNames.Num() == AssetViewerSettings->Profiles.Num(), TEXT("List of profiles is out of sync with the list of corresponding profile names."));
	ensureMsgf(AssetViewerProfileNames.Num() > 0, TEXT("The list of profiles is expected to always have at least one default profile"));
}
