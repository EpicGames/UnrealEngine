// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPreviewProfileController.h"

class UAssetViewerSettings;

// This class is almost identical to FPreviewProfileController except that it doesn't use UEditorPerProjectUserSettings 
// to get the current scene profile index. Instead it is supplied with an IProfileIndexStorage object which stores and 
// loads the scene profile index. This allows a separate profile controller to be created for each AdvancedPreviewScene.

/** Preview Profile Controller that interfaces with a user-supplied index storage object */
class FDataflowPreviewProfileController : public IPreviewProfileController
{
public:

	class IProfileIndexStorage
	{
	public:
		virtual void StoreProfileIndex(int32) = 0;
		virtual int32 RetrieveProfileIndex() = 0;

	protected:
		virtual ~IProfileIndexStorage() = default;
	};

	FDataflowPreviewProfileController(TSharedPtr<IProfileIndexStorage> ProfileIndexStorage);
	virtual ~FDataflowPreviewProfileController();

	/** Returns the list of available preview profiles names. */
	virtual TArray<FString> GetPreviewProfiles(int32& OutCurrentProfileIndex) const override;

	/** Set the specified preview profiles as the active one. */
	virtual bool SetActiveProfile(const FString& ProfileName) override;

	/** Returns the preview profiles currently active. */
	virtual FString GetActiveProfile() const override;

	/** Returns true if user has added one or more of their own profiles */
	virtual bool HasAnyUserProfiles() const override;

	/** Invoked after the list of available profiles has changed. */
	virtual FOnPreviewProfileListChanged& OnPreviewProfileListChanged() override { return OnPreviewProfileListChangedDelegate; }

	/** Invoked after the active preview profile changed. */
	virtual FOnPreviewProfileChanged& OnPreviewProfileChanged() override { return OnPreviewSettingChangedDelegate; }

private:
	void UpdateAssetViewerProfiles();
	void EnsureProfilesStateCoherence() const;

private:

	TSharedPtr<IProfileIndexStorage> ProfileIndexStorage;

	/** Provides the list of available preview profiles along with the profile change delegates (to update the profile combo box selection) */
	UAssetViewerSettings* AssetViewerSettings = nullptr;

	/** The list of available profiles. */
	TArray<FString> AssetViewerProfileNames;

	/** Holds the current profile index in the list. This is kept consistent with the cache list of names. */
	int32 CurrentProfileIndex = 0;

	FOnPreviewProfileListChanged OnPreviewProfileListChangedDelegate;
	FOnPreviewProfileChanged OnPreviewSettingChangedDelegate;

	FDelegateHandle AssetViewerSettingsProfileAddRemoveHandle;
	FDelegateHandle AssetViewerSettingsChangedHandle;
};
