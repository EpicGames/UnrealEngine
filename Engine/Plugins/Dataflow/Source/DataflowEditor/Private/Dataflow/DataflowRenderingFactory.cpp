// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowRenderingFactory.h"

#include "Dataflow/DataflowNode.h"
#include "Dataflow/DataflowInputOutput.h"
#include "Misc/MessageDialog.h"
#include "GeometryCollection/Facades/CollectionRenderingFacade.h"

namespace UE::Dataflow
{
	FRenderingFactory* FRenderingFactory::Instance = nullptr;

	void FRenderingFactory::RenderNodeOutput(GeometryCollection::Facades::FRenderingFacade& RenderingFacade, const FGraphRenderingState& State)
	{
		if (CallbackMap.Contains(State.GetRenderKey()))
		{
			CallbackMap[State.GetRenderKey()]->Render(RenderingFacade, State);
		}
		else
		{
			UE_LOGF(LogChaos, Warning,
				"Warning : Dataflow missing output renderer <%ls,%ls> for node %ls", 
				*State.GetRenderKey().Get<0>(),
				*State.GetRenderKey().Get<1>().ToString(),
				*State.GetNodeName().ToString());
		}
	}

	bool FRenderingFactory::CanRenderNodeOutput(const FGraphRenderingState& State) const
	{
		if (CallbackMap.Contains(State.GetRenderKey()))
		{
			return CallbackMap[State.GetRenderKey()]->CanRenderFromState(State);
		}

		return false;
	}

	FRenderingFactory* FRenderingFactory::GetInstance()
	{
		if (!Instance)
		{
			Instance = new FRenderingFactory();
		}
		return Instance;
	}

	void FRenderingFactory::RegisterCallbacks(TUniquePtr<ICallbackInterface> InCallbacks)
	{
		const FRenderKey Key = InCallbacks->GetRenderKey();
		if (CallbackMap.Contains(Key))
		{
			UE_LOGF(LogChaos, Warning,
				"Warning : Dataflow output rendering callback registration conflicts with existing rendering callback (<%ls,%ls>)",
				*Key.Get<0>(),
				*Key.Get<1>().ToString());
		}
		else
		{
			CallbackMap.Add(Key, MoveTemp(InCallbacks));
		}
	}

	void FRenderingFactory::DeregisterCallbacks(const FRenderKey& Key)
	{
		if (!CallbackMap.Contains(Key))
		{
			UE_LOGF(LogChaos, Warning,
				"Warning : Dataflow output rendering callback deregistration. Rendering callback not registered: (<%ls,%ls>)",
				*Key.Get<0>(),
				*Key.Get<1>().ToString());
		}
		else
		{
			CallbackMap.Remove(Key);
		}
	}


}

