// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowRenderingViewMode.h"
#include "Misc/LazySingleton.h"
#include "Dataflow/DataflowEditorCommands.h"

#define LOCTEXT_NAMESPACE "DataflowRenderingViewMode"

namespace UE::Dataflow
{
	bool IDataflowConstructionViewMode::IsPerspective() const
	{
		return GetViewportType() == ELevelViewportType::LVT_Perspective;
	}

	ELevelViewportType FDataflowConstruction2DViewModeBase::GetViewportType() const
	{
		return ELevelViewportType::LVT_OrthoNegativeXY;
	}

	ELevelViewportType FDataflowConstruction3DViewModeBase::GetViewportType() const
	{
		return ELevelViewportType::LVT_Perspective;
	}

	////////////////////////////////////////////////////////////////////////////////////////////////

	FName FDataflowConstruction2DViewMode::Name = FName("2DView");

	FName FDataflowConstruction2DViewMode::GetName() const
	{
		return Name;
	}

	FText FDataflowConstruction2DViewMode::GetButtonText() const
	{
		return FText(LOCTEXT("2DViewButtonText", "2DView"));
	}

	FText FDataflowConstruction2DViewMode::GetTooltipText() const
	{
		return FText(LOCTEXT("2DViewTooltipText", "Default 2D View"));
	}

	FRotator FDataflowConstruction2DViewMode::GetViewportOrientation() const
	{
		return FRotator(90, -90, 0);
	}
	
	////////////////////////////////////////////////////////////////////////////////////////////////

	FName FDataflowConstruction3DViewMode::Name = FName("3DView");

	FName FDataflowConstruction3DViewMode::GetName() const
	{
		return Name;
	}

	FText FDataflowConstruction3DViewMode::GetButtonText() const
	{
		return FText(LOCTEXT("3DViewButtonText", "3DView"));
	}

	FText FDataflowConstruction3DViewMode::GetTooltipText() const
	{
		return FText(LOCTEXT("3DViewTooltipText", "Default 3D View"));
	}

	FRotator FDataflowConstruction3DViewMode::GetViewportOrientation() const
	{ 
		return FRotator(-30, -30, 0);
	}

	////////////////////////////////////////////////////////////////////////////////////////////////

	FName FDataflowConstructionUVViewMode::Name = FName("UVView");

	FName FDataflowConstructionUVViewMode::GetName() const
	{
		return Name;
	}

	FText FDataflowConstructionUVViewMode::GetButtonText() const
	{
		return FText(LOCTEXT("UVViewButtonText", "UVView"));
	}

	FText FDataflowConstructionUVViewMode::GetTooltipText() const
	{
		return FText(LOCTEXT("UVViewTooltipText", "UV/Texture Coordinate View"));
	}

	FRotator FDataflowConstructionUVViewMode::GetViewportOrientation() const
	{
		return FRotator(-90, 0, 0);
	}

	////////////////////////////////////////////////////////////////////////////////////////////////

	FName FDataflowConstructionMaterialViewMode::Name = FName("MaterialView");

	FName FDataflowConstructionMaterialViewMode::GetName() const
	{
		return Name;
	}

	FText FDataflowConstructionMaterialViewMode::GetButtonText() const
	{
		return FText(LOCTEXT("MaterialViewButtonText", "MaterialView"));
	}

	FText FDataflowConstructionMaterialViewMode::GetTooltipText() const
	{
		return FText(LOCTEXT("MaterialViewTooltipText", "Material View"));
	}

	//
	// Factory
	//

	FRenderingViewModeFactory::FRenderingViewModeFactory()
	{
		// Default modes
		ViewModeMap.Add(FDataflowConstruction2DViewMode::Name, MakeUnique<FDataflowConstruction2DViewMode>());
		ViewModeMap.Add(FDataflowConstruction3DViewMode::Name, MakeUnique<FDataflowConstruction3DViewMode>());
		ViewModeMap.Add(FDataflowConstructionUVViewMode::Name, MakeUnique<FDataflowConstructionUVViewMode>());
		ViewModeMap.Add(FDataflowConstructionMaterialViewMode::Name, MakeUnique<FDataflowConstructionMaterialViewMode>());
	}

	FRenderingViewModeFactory& FRenderingViewModeFactory::GetInstance()
	{
		return TLazySingleton<FRenderingViewModeFactory>::Get();
	}

	void FRenderingViewModeFactory::TearDown()
	{
		return TLazySingleton<FRenderingViewModeFactory>::TearDown();
	}

	void FRenderingViewModeFactory::RegisterViewMode(TUniquePtr<IDataflowConstructionViewMode>&& ViewMode)
	{
		ensureMsgf(!FDataflowEditorCommands::IsRegistered(), TEXT("FRenderingViewModeFactory: DataflowEditorCommands have already been registered. \
			Newly registered View Modes may not be available in the Editor. \
			Ensure that RegisterViewMode is called before the DataflowEditor module is loaded."));

		const FName NewViewModeName = ViewMode->GetName();
		if (ViewModeMap.Contains(NewViewModeName))
		{
			UE_LOGF(LogChaos, Warning, "Dataflow rendering view mode registration conflicts with existing view mode : %ls", *NewViewModeName.ToString());
		}
		else
		{
			ViewModeMap.Add(NewViewModeName, MoveTemp(ViewMode));
		}
	}

	void FRenderingViewModeFactory::DeregisterViewMode(const FName& ViewModeName)
	{
		if (!ViewModeMap.Contains(ViewModeName))
		{
			UE_LOGF(LogChaos, Warning, "Dataflow rendering view mode deregistration -- view mode not registered : %ls", *ViewModeName.ToString());
		}
		else
		{
			ViewModeMap.Remove(ViewModeName);
		}
	}

	const IDataflowConstructionViewMode* FRenderingViewModeFactory::GetViewMode(const FName& ViewModeName) const
	{
		if (ViewModeMap.Contains(ViewModeName))
		{
			return ViewModeMap[ViewModeName].Get();
		}
		return nullptr;
	}

	const TMap<FName, TUniquePtr<IDataflowConstructionViewMode>>& FRenderingViewModeFactory::GetViewModes() const
	{
		return ViewModeMap;
	}

}

#undef LOCTEXT_NAMESPACE 

