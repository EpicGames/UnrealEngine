// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowSCommentNode.h"

#include "EdGraphNode_Comment.h"
#include "Dataflow/DataflowEdNode.h"
#include "Dataflow/DataflowNodeFactory.h"
#include "Dataflow/DataflowObject.h"
#include "Dataflow/DataflowCore.h"
#include "Logging/LogMacros.h"
#include "Widgets/Layout/SBorder.h"
#include "Settings/EditorStyleSettings.h"
#include "ScopedTransaction.h"
#include "Dataflow/DataflowAssetEditUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowSCommentNode)

#define LOCTEXT_NAMESPACE "SDataflowEdNodeComment"

//
// Add a menu option to create a graph node.
//
TSharedPtr<FAssetSchemaAction_Dataflow_CreateCommentNode_DataflowEdNode> FAssetSchemaAction_Dataflow_CreateCommentNode_DataflowEdNode::CreateAction(UEdGraph* ParentGraph, const TSharedPtr<SGraphEditor>& GraphEditor)
{
	return MakeShared<FAssetSchemaAction_Dataflow_CreateCommentNode_DataflowEdNode>(GraphEditor);
}

//
//  Created comment node
//
UEdGraphNode* FAssetSchemaAction_Dataflow_CreateCommentNode_DataflowEdNode::PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode)
{
	constexpr float MinSizeX = 500.f;
	constexpr float MinSizeY = 250.f;

	// set default bounds 
	FSlateRect Bounds
	{
		Location.X,
		Location.Y,
		Location.X + MinSizeX,
		Location.Y + MinSizeY
	};
	GraphEditor->GetBoundsForSelectedNodes(Bounds, 50.0f);

	return UE::Dataflow::FEditAssetUtils::AddNewComment(ParentGraph, Bounds.GetTopLeft(), Bounds.GetSize());
}

#undef LOCTEXT_NAMESPACE
