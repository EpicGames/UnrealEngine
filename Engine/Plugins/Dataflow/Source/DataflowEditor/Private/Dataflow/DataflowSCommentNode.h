// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraphUtilities.h"
#include "SGraphNode.h"
#include "SGraphNodeComment.h"

#include "DataflowSCommentNode.generated.h"

#define UE_API DATAFLOWEDITOR_API

//
// SDataflowEdNodeComment
//

class SDataflowEdNodeComment : public SGraphNodeComment
{
	typedef SGraphNodeComment Super;

public:
	SLATE_BEGIN_ARGS(SDataflowEdNodeComment) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UEdGraphNode_Comment* InNode) { Super::Construct(SGraphNodeComment::FArguments(), InNode); }
};

//
// Action to add a node to the graph
//
USTRUCT()
struct FAssetSchemaAction_Dataflow_CreateCommentNode_DataflowEdNode : public FEdGraphSchemaAction
{
	GENERATED_USTRUCT_BODY();

public:
	FAssetSchemaAction_Dataflow_CreateCommentNode_DataflowEdNode(const TSharedPtr<SGraphEditor>& InGraphEditor) : FEdGraphSchemaAction()
		, GraphEditor(InGraphEditor)
		{}

	FAssetSchemaAction_Dataflow_CreateCommentNode_DataflowEdNode() {}

	static UE_API TSharedPtr<FAssetSchemaAction_Dataflow_CreateCommentNode_DataflowEdNode> CreateAction(UEdGraph* ParentGraph, const TSharedPtr<SGraphEditor>& GraphEditor);

	using FEdGraphSchemaAction::PerformAction; // Prevent hiding of deprecated base class function with FVector2D
	UE_API virtual UEdGraphNode* PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
	//virtual void AddReferencedObjects(FReferenceCollector& Collector) override;

	TSharedPtr<SGraphEditor> GraphEditor;
};

#undef UE_API
