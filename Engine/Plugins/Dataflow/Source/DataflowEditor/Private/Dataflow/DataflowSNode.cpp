// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowSNode.h"

#include "Dataflow/DataflowEdNode.h"
#include "Dataflow/DataflowEditorStyle.h"
#include "Dataflow/DataflowEngineUtil.h"
#include "Dataflow/DataflowGraphEditor.h"
#include "Dataflow/DataflowNodeFactory.h"
#include "Dataflow/DataflowObject.h"
#include "Dataflow/DataflowTypePolicy.h"
#include "Framework/Application/SlateApplication.h"
#include "Logging/LogMacros.h"
#include "SourceCodeNavigation.h"
#include "Styling/SlateTypes.h"
#include "Styling/CoreStyle.h"
#include "Styling/AppStyle.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Editor/Transactor.h"
#include "GraphEditorSettings.h"
#include "Dataflow/DataflowSchema.h"
#include "Dataflow/DataflowSPin.h"
#include "Dataflow/DataflowPoints.h"
#include "GeometryCollection/Facades/PointsFacade.h"


#define LOCTEXT_NAMESPACE "SDataflowEdNode"

namespace UE::Dataflow::Private
{
	void AddOutputValueToString(const FDataflowOutput& Output, FContext& Context, FString& OutString)
	{
		using namespace UE::Dataflow::Private;

		if (!Output.HasCachedValue(Context))
		{
			OutString += TEXT("/!\\ ");
		}
		if (Output.GetType() == "FBox")
		{
			const FBox DefaultValue(EForceInit::ForceInit);
			const FBox& Value = Output.ReadValue(Context, DefaultValue);
			if (Value.IsValid)
			{
				OutString += FString::Printf(TEXT("%s : %s\n"), *Output.GetName().ToString(), *Value.ToCompactString());
			}
			else
			{
				OutString += FString::Printf(TEXT("%s : FBox not set\n"), *Output.GetName().ToString());
			}

		}
		else if (Output.GetType() == "FSphere")
		{
			const FSphere DefaultValue(EForceInit::ForceInit);
			const FSphere& Value = Output.ReadValue(Context, DefaultValue);

			OutString += FString::Printf(TEXT("%s : %s\n"), *Output.GetName().ToString(), *Value.ToString());
		}
		else if (Output.GetType() == "FDataflowPoints")
		{
			const FDataflowPoints DefaultValue;
			const FDataflowPoints& Value = Output.ReadValue(Context, DefaultValue);

			GeometryCollection::Facades::FPointsFacade PointFacade = Value.GetPointsFacade();

			const int32 NumPoints = PointFacade.GetNumPoints();
			const int32 NumAttrs = PointFacade.GetNumAttrs() - 1;
			FString Attrs = PointFacade.GetAttrsAsString(true);

			OutString += FString::Printf(TEXT("%s : %d point(s) | %d attribute(s): %s\n"), *Output.GetName().ToString(), NumPoints, NumAttrs, *Attrs);
		}
		else if (FDataflowSelectionTypePolicy::SupportsTypeStatic(Output.GetType()))
		{
			const FDataflowSelection DefaultValue;
			const FDataflowSelection& Value = Output.ReadValue(Context, DefaultValue);
			OutString += FString::Printf(TEXT("%s : (%s) %d/%d\n"), *Output.GetName().ToString(), *Value.GetGroupName().ToString(), Value.NumSelected(), Value.Num());
		}
		else if (FDataflowStringConvertibleTypePolicy::SupportsTypeStatic(Output.GetType()))
		{
			// Try converting to a string using FDataflowStringConvertibleTypes
			const FString DefaultValue;
			const FString Value = Output.ReadValue<FDataflowStringConvertibleTypes>(Context, DefaultValue);
			OutString += FString::Printf(TEXT("%s : %s\n"), *Output.GetName().ToString(), *Value);
		}
		else if (FDataflowUObjectConvertibleTypePolicy::SupportsTypeStatic(Output.GetType()))
		{
			static const FName NullObject(TEXT("(null)"));

			// Try converting to a string using FDataflowUObjectConvertibleTypes
			const TObjectPtr<UObject> DefaultValue;
			const TObjectPtr<UObject> Value = Output.ReadValue<FDataflowUObjectConvertibleTypes>(Context, DefaultValue);
			const FName ObjectName = Value ? Value->GetFName() : NullObject;
			OutString += FString::Printf(TEXT("%s : %s\n"), *Output.GetName().ToString(), *ObjectName.ToString());
		}
		else if (FDataflowArrayTypePolicy::SupportsTypeStatic(Output.GetType()))
		{
			UE::Dataflow::FContextValue ContextValue(Context, Output.CacheKey());
			OutString += FString::Printf(TEXT("%s : Array Size : %d\n"), *Output.GetName().ToString(), ContextValue.Num());
		}
		else
		{
			OutString += FString::Printf(TEXT("%s : (Cannot watch this type)\n"), *Output.GetName().ToString());
		}
	}

}

const FSlateBrush* SDataflowEdNode::GetPinButtonImage() const
{
	if (DataflowGraphNode && DataflowGraphNode->ShouldWireframeRenderNode())
	{
		return FAppStyle::Get().GetBrush("Icons.Pinned");
	}
	else
	{
		return FAppStyle::Get().GetBrush("Icons.Unpinned");
	}
}

void SDataflowEdNode::Construct(const FArguments& InArgs, UDataflowEdNode* InNode)
{
	GraphNode = InNode;
	DataflowGraphNode = Cast<UDataflowEdNode>(InNode);
	DataflowInterface = InArgs._DataflowInterface;

	this->TitleBorderMargin = FMargin(5.f, 5.f, 5.f, 5.f);

	UpdateGraphNode();

	//
	// Freeze
	//
	FreezeImageWidget = SNew(SImage)
		.Image(FDataflowEditorStyle::Get().GetBrush("Dataflow.FreezeNode"))
		.DesiredSizeOverride(FVector2D(24.f, 24.f))
		.Visibility_Lambda([this]()->EVisibility
			{
				if (DataflowGraphNode && DataflowGraphNode->GetDataflowNode())
				{
					if (DataflowGraphNode->GetDataflowNode()->IsFrozen())
					{
						return EVisibility::Visible;
					}
				}
				return EVisibility::Collapsed;
			});

	PerfWidget = SNew(STextBlock);
	WatchWidget = SNew(STextBlock);
}

TSharedPtr<SGraphPin> SDataflowEdNode::CreatePinWidget(UEdGraphPin* Pin) const
{
	if (DataflowGraphNode)
	{
		if (TSharedPtr<FDataflowNode> DataflowNode = DataflowGraphNode->GetDataflowNode())
		{
			bool bIsOutputInvalid = false;

			if (Pin->Direction == EEdGraphPinDirection::EGPD_Output)
			{
				if (FDataflowOutput* Output = DataflowNode->FindOutput(Pin->GetFName()))
				{
					if (DataflowInterface)
					{
						if (const TSharedPtr<UE::Dataflow::FContext> DataflowContext = DataflowInterface->GetDataflowContext())
						{
							TSet<UE::Dataflow::FContextCacheKey> CacheKeys;
							const int32 NumKeys = DataflowContext->GetKeys(CacheKeys);

							//
							// DataStore is empty or 
							// CacheKey is not in DataStore or
							// Node's Timestamp is invalid or
							// Node's Timestamp is greater than CacheKey's Timestamp -> Pin is invalid
							//
							bIsOutputInvalid = !NumKeys ||
								!CacheKeys.Contains(Output->CacheKey()) ||
								DataflowNode->GetTimestamp().IsInvalid() ||
								!DataflowContext->IsCacheEntryAfterTimestamp(Output->CacheKey(), DataflowNode->GetTimestamp());
						}
					}
				}
			}
				
			bool bColorOverriden = false;
			FLinearColor OverrideColor = FLinearColor::Black;

			if (const UDataflowSchema* DataflowSchema = Cast<UDataflowSchema>(Pin->GetSchema()))
			{
				TOptional<FLinearColor> OverrideColorOpt = DataflowSchema->GetPinColorOverride(DataflowNode, Pin);
				if (OverrideColorOpt.IsSet())
				{
					OverrideColor = OverrideColorOpt.GetValue();
					bColorOverriden = true;
				}
			}

			return SNew(SDataflowPin, Pin)
				.IsPinInvalid(bIsOutputInvalid)
				.bIsPinColorOverriden(bColorOverriden)
				.PinColorOverride(OverrideColor);
		}
	}

	return SGraphNode::CreatePinWidget(Pin);
}

TSharedRef<SWidget> SDataflowEdNode::CreateTitleWidget(TSharedPtr<SNodeTitle> NodeTitle)
{
	TSharedRef<SWidget> DefaultTitle = SGraphNode::CreateTitleWidget(NodeTitle);
	
	static const FName UnknownTypeName = TEXT("Unknown");
	FName NodeType = UnknownTypeName;
	if (TSharedPtr<FDataflowNode> DataflowNode = DataflowGraphNode->GetDataflowNode())
	{
		NodeType = DataflowNode->GetDisplayName();
	}
	return SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		[
			DefaultTitle
		]
		+ SVerticalBox::Slot()
		[
			SNew(STextBlock)
			.Text(FText::FromName(NodeType))
			.Font(FCoreStyle::Get().GetFontStyle("SmallFont"))
			.Visibility_Lambda([InlineEditableText = InlineEditableText]()
				{
					return InlineEditableText->IsInEditMode()
						? EVisibility::Collapsed
						: EVisibility::Visible;
				})
		]
		;
}

TSharedRef<SWidget> SDataflowEdNode::CreateTitleRightWidget()
{
	RenderCheckBoxWidget = SNew(SCheckBox)
		.Style(FAppStyle::Get(), "ToggleButtonCheckbox")
		.IsChecked_Lambda([this]()-> ECheckBoxState
			{
				if (DataflowGraphNode && DataflowGraphNode->ShouldWireframeRenderNode())
				{
					return ECheckBoxState::Checked;
				}
				return ECheckBoxState::Unchecked;
			})
		.OnCheckStateChanged_Lambda([this](const ECheckBoxState NewState)
			{
				if (DataflowGraphNode && DataflowInterface)
				{
					const bool bShouldWireframeRenderNode = (NewState == ECheckBoxState::Checked);
					DataflowGraphNode->SetShouldWireframeRenderNode(bShouldWireframeRenderNode);
					DataflowInterface->OnRenderToggleChanged(DataflowGraphNode);
				}
			})
				.IsEnabled_Lambda([this]()->bool
					{
						if (DataflowGraphNode)
						{
							return DataflowGraphNode->CanEnableWireframeRenderNode();
						}
						return false;
					})
				.Padding(2.0f)
				.HAlign(HAlign_Center)
				[
					SNew(SImage)
						.ColorAndOpacity(FSlateColor::UseForeground())
						.Image(this, &SDataflowEdNode::GetPinButtonImage)
				];

	const TSharedPtr<const FDataflowNode> DataflowNode = DataflowGraphNode ? DataflowGraphNode->GetDataflowNode() : TSharedPtr<const FDataflowNode>();

	const bool bHasRenderingCheckBox = (
		DataflowInterface &&
		DataflowInterface->NodesHaveToggleWidget() &&
		DataflowNode && 
		(DataflowNode->NumOutputs() > 0 || DataflowNode->CanDebugDraw())
		);

	if (bHasRenderingCheckBox)
	{
		return SNew(SBox)
			.HAlign(HAlign_Fill)
			.Padding(0.0f, 2.0f, 0.0f, 2.0f)
			[
				RenderCheckBoxWidget.ToSharedRef()
			];
	}
	return SNullWidget::NullWidget;
}

TArray<FOverlayWidgetInfo> SDataflowEdNode::GetOverlayWidgets(bool bSelected, const FVector2f& WidgetSize) const
{
	TArray<FOverlayWidgetInfo> Widgets = SGraphNode::GetOverlayWidgets(bSelected, WidgetSize);

	if (DataflowGraphNode && DataflowGraphNode->GetDataflowNode() && DataflowInterface)
	{
		if (DataflowInterface->NodesHaveFreezeWidget())
		{
			static const FVector2f FreezeImageOverhang(10.f);  // The icon must slightly overhang to make space for the lower output pins
			const FVector2f FreezeImageSize = FreezeImageWidget->GetDesiredSize();
			FOverlayWidgetInfo FreezeImageInfo;
			FreezeImageInfo.OverlayOffset = WidgetSize - FreezeImageSize + FreezeImageOverhang;
			FreezeImageInfo.Widget = FreezeImageWidget;
			Widgets.Add(FreezeImageInfo);
		}

		if (TSharedPtr<UE::Dataflow::FContext> Context = DataflowInterface->GetDataflowContext())
		{
			if (TSharedPtr<const FDataflowNode> DataflowNode = DataflowGraphNode->GetDataflowNode())
			{
				UE::Dataflow::FContextPerfData::FData PerfData = Context->GetPerfDataForNode(*DataflowNode);
				if (DataflowNode->IsAsyncEvaluating())
				{
					constexpr double BlinkPeriod = 2000;
					const double ElapsedMs = FPlatformTime::ToMilliseconds64(FPlatformTime::Cycles64());
					const double Fraction = FMath::Fractional(ElapsedMs / BlinkPeriod);
					const double LerpFactor = FMath::Sin(Fraction * 2.0 * UE_PI) * 0.5 + 0.5;
					const FLinearColor Color = FMath::Lerp(FLinearColor::Red, FLinearColor::White, LerpFactor);
					PerfWidget->SetColorAndOpacity(Color);
					PerfWidget->SetVisibility(EVisibility::Visible);
					PerfWidget->SetText(LOCTEXT("DataflowNodeEvaluatingMessage", "Evaluating..."));
				}
				else if (Context->IsPerfDataEnabled() && PerfData.ExclusiveTimeMs > 0 && PerfData.InclusiveTimeMs > 0)
				{
					constexpr double FadeTimeMs = 5000;
					const double MsSinceUpdate = FPlatformTime::ToMilliseconds64(FPlatformTime::Cycles64() - PerfData.LastTimestamp.Value);
					const double LerpFactor = FMath::Clamp(MsSinceUpdate / FadeTimeMs, 0.0, 1.0);
					const FLinearColor Color = FMath::Lerp(FLinearColor::Red, FLinearColor::White, LerpFactor);
					PerfWidget->SetColorAndOpacity(Color);
					PerfWidget->SetVisibility(EVisibility::Visible);
					FNumberFormattingOptions FmtOptions;
					FmtOptions.MaximumFractionalDigits = 2;
					FmtOptions.MinimumFractionalDigits = 2;

					FText PerfText;
					if (PerfData.NumCalls > 1)
					{
						PerfText = FText::Format(LOCTEXT("DataflowNodePerfDataWithCallFormat", "Time: {0} ms - {1} calls"), FText::AsNumber(PerfData.ExclusiveTimeMs, &FmtOptions), FText::AsNumber(PerfData.NumCalls));
					}
					else
					{
						PerfText = FText::Format(LOCTEXT("DataflowNodePerfDataFormat", "Time: {0} ms"), FText::AsNumber(PerfData.ExclusiveTimeMs, &FmtOptions));
					}
					
					PerfWidget->SetText(PerfText);
				}
				else
				{
					PerfWidget->SetVisibility(EVisibility::Collapsed);
					PerfWidget->SetText(FText());
				}

				FOverlayWidgetInfo PerfWidgetInfo;
				PerfWidgetInfo.OverlayOffset = FVector2f{ 0.0f, -20.0f };
				PerfWidgetInfo.Widget = PerfWidget;
				Widgets.Add(PerfWidgetInfo);
			}
		}

		FString WatchString;
		if (DataflowGraphNode && DataflowGraphNode->HasAnyWatchedConnection())
		{
			if (TSharedPtr<UE::Dataflow::FContext> Context = DataflowInterface->GetDataflowContext())
			{
				if (TSharedPtr<const FDataflowNode> DataflowNode = DataflowGraphNode->GetDataflowNode())
				{
					for (const FDataflowOutput* Output : DataflowNode->GetOutputs())
					{
						if (Output && DataflowGraphNode->IsConnectionWatched(*Output))
						{
							UE::Dataflow::Private::AddOutputValueToString(*Output, *Context, WatchString);
						}
					}
				}

				WatchWidget->SetVisibility(WatchString.IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible);
				WatchWidget->SetText(FText::FromString(WatchString));

				FOverlayWidgetInfo WatchWidgetInfo;
				WatchWidgetInfo.OverlayOffset = FVector2f{ 0.0f, WidgetSize.Y + 4.0f };
				WatchWidgetInfo.Widget = WatchWidget;
				Widgets.Add(WatchWidgetInfo);
			}
		}
	}

	return Widgets;
}

const FSlateColor CExperimentalColor(FColor(13, 94, 78, 255)); // Darker Turquoise
const FSlateColor CDeprecatedColor(FColor(46, 204, 113, 255)); // Emerald
const FSlateColor CFailedColor(FColor(243, 156, 18, 255)); // Orange

void SDataflowEdNode::UpdateErrorInfo()
{
	// Priority should be:
	// 1. Error
	// 2. Warning
	// 4. Deprecated
	// 5. Experimental

	if (DataflowGraphNode)
	{
		if (const TSharedPtr<FDataflowNode> DataflowNode = DataflowGraphNode->GetDataflowNode())
		{
			if (DataflowInterface)
			{
				const EMessageSeverity::Type Severity = DataflowInterface->GetMessageMaxSeverity(*DataflowGraphNode);
				switch (Severity)
				{
				case EMessageSeverity::Error:
					ErrorMsg = FString(TEXT("ERROR"));
					ErrorColor = FLinearColor(1.f, 0.02f, 0.003f);
					return;
				case EMessageSeverity::PerformanceWarning:
				case EMessageSeverity::Warning:
					ErrorMsg = FString(TEXT("WARNING"));
					ErrorColor = FAppStyle::GetColor("ErrorReporting.WarningBackgroundColor");
					return;
				case EMessageSeverity::Info:
				default:
					break; // continue to other cases 
				}
			}

			if (UE::Dataflow::FNodeFactory::IsNodeDeprecated(DataflowNode->GetType()))
			{
				ErrorMsg = FString(TEXT("Deprecated"));
				ErrorColor = CDeprecatedColor;
				return;
			}
			else if (UE::Dataflow::FNodeFactory::IsNodeExperimental(DataflowNode->GetType()))
			{
				ErrorMsg = FString(TEXT("Experimental"));
				ErrorColor = CExperimentalColor;
				return;
			}
		}
	}
}

FText SDataflowEdNode::GetNodeTooltip() const
{
	FText TooltipText = SGraphNode::GetNodeTooltip();
	if (DataflowInterface)
	{
		if (UDataflowEdNode* DataflowNode = Cast<UDataflowEdNode>(GraphNode))
		{
			const FText MessageText = DataflowInterface->GetMessageText(*DataflowNode);
			if (!MessageText.IsEmpty())
			{
				TooltipText = FText::Format(LOCTEXT("SDataflowNode_MergeText", "{0}\n{1}"), TooltipText, MessageText);
			}
		}
	}
	return TooltipText;
}

FReply SDataflowEdNode::OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	TSharedPtr<FDataflowNode> Node;
	if (GraphNode)
	{
		if (UDataflowEdNode* DataflowNode = Cast<UDataflowEdNode>(GraphNode))
		{
			if (TSharedPtr<UE::Dataflow::FGraph> Graph = DataflowNode->GetDataflowGraph())
			{
				Node = Graph->FindBaseNode(DataflowNode->GetDataflowNodeGuid());
			}
		}
	}

	if (Node)
	{
		if (InMouseEvent.GetModifierKeys().IsControlDown())
		{
			if (FSourceCodeNavigation::CanNavigateToStruct(Node->TypedScriptStruct()))
			{
				FSourceCodeNavigation::NavigateToStruct(Node->TypedScriptStruct());
			}
		}
		else 
		{
			if (DataflowInterface)
			{
				TSharedPtr<UE::Dataflow::FContext> ContextPtr = DataflowInterface->GetDataflowContext();
				Node->OnDoubleClicked(ContextPtr.Get());
			}
		}
	}
	
	return Super::OnMouseButtonDoubleClick(InMyGeometry, InMouseEvent);
}

void SDataflowEdNode::AddReferencedObjects(FReferenceCollector& Collector)
{
	if (DataflowGraphNode)
	{
		Collector.AddReferencedObject(DataflowGraphNode);
		if (TSharedPtr<FDataflowNode> DataflowNode = DataflowGraphNode->GetDataflowNode())
		{
			Collector.AddPropertyReferences(DataflowNode->TypedScriptStruct(), DataflowNode.Get());
		}
	}
}

void SDataflowEdNode::CreateOutputSideAddButton(TSharedPtr<SVerticalBox> OutputBox)
{
	if (DataflowGraphNode)
	{
		if (TSharedPtr<FDataflowNode> DataflowNode = DataflowGraphNode->GetDataflowNode())
		{
			if (DataflowNode->CanAddPin())
			{
				TSharedRef<SWidget> AddPinButton = AddPinButtonContent(
					LOCTEXT("AddPinButton_Text", "Add Pin"),
					LOCTEXT("AddPinButton_Tooltip", "Add an optional input pin"),
					true
				);

				const FMargin AddPinPadding = Settings->GetOutputPinPadding();

				OutputBox->AddSlot()
					.AutoHeight()
					.VAlign(VAlign_Center)
					.Padding(AddPinPadding)
					[
						AddPinButton
					];
			}
		}
	}
}

void SDataflowEdNode::CreateInputSideAddButton(TSharedPtr<SVerticalBox> InputBox)
{
	if (DataflowGraphNode)
	{
		if (TSharedPtr<FDataflowNode> DataflowNode = DataflowGraphNode->GetDataflowNode())
		{
			if (DataflowNode->HasHideableInputs())
			{
				TSharedRef<SWidget> ShowHideInputsButton = AddPinButtonContent(
					LOCTEXT("ShowHideInputsButton_Text", "Show/Hide Inputs"),
					LOCTEXT("ShowHideInputsButton_Tooltip", "Show/Hide input pins."),
					false
				);

				// override the on clicked function
				if (ShowHideInputsButton->GetWidgetClass().GetWidgetType() == SButton::StaticWidgetClass().GetWidgetType())
				{
					TSharedRef<SButton> TypedButtonWidget = StaticCastSharedRef<SButton>(ShowHideInputsButton);
					TypedButtonWidget->SetOnClicked(FOnClicked::CreateSP(this, &SDataflowEdNode::OnShowHideInputs));
				}

				const FMargin AddPinPadding = Settings->GetInputPinPadding();

				InputBox->AddSlot()
					.AutoHeight()
					.VAlign(VAlign_Center)
					.Padding(AddPinPadding)
					[
						ShowHideInputsButton
					];
			}
		}
	}
}

FReply SDataflowEdNode::OnAddPin()
{
	if (DataflowGraphNode)
	{
		if (TSharedPtr<FDataflowNode> DataflowNode = DataflowGraphNode->GetDataflowNode())
		{
			if (DataflowNode->CanAddPin())
			{
				DataflowGraphNode->AddOptionPin();
			}
		}
	}
	return FReply::Handled();
}

FReply SDataflowEdNode::OnShowHideInputs()
{
	if (DataflowGraphNode)
	{
		FMenuBuilder MenuBuilder(false, nullptr);
		if (TSharedPtr<FDataflowNode> DataflowNode = DataflowGraphNode->GetDataflowNode())
		{
			if (DataflowNode->HasHideableInputs())
			{
				MenuBuilder.AddMenuEntry(LOCTEXT("HideAllInputs", "Hide all"), LOCTEXT("HideAllInputsTooltip", "Hide all hideable input pins"), FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateUObject(DataflowGraphNode, &UDataflowEdNode::HideAllInputPins)));
				MenuBuilder.AddMenuEntry(LOCTEXT("UnhideAllInputs", "Show all"), LOCTEXT("UnhideAllInputsTooltip", "Show all hideable input pins"), FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateUObject(DataflowGraphNode, &UDataflowEdNode::ShowAllInputPins)));


				TArray<FDataflowInput*> Inputs = DataflowNode->GetInputs();
				for (FDataflowInput* Input : Inputs)
				{
					if (Input->GetCanHidePin())
					{
						FText InputName = DataflowNode->GetPinDisplayName(Input->GetName(), UE::Dataflow::FPin::EDirection::INPUT);
						MenuBuilder.AddMenuEntry(InputName, LOCTEXT("UnhidePinTooltip", "Show/Hide pin"), FSlateIcon(),
							FUIAction(
								FExecuteAction::CreateUObject(DataflowGraphNode, &UDataflowEdNode::ToggleHideInputPin, Input->GetName()),
								FCanExecuteAction::CreateUObject(DataflowGraphNode, &UDataflowEdNode::CanToggleHideInputPin, Input->GetName()),
								FIsActionChecked::CreateUObject(DataflowGraphNode, &UDataflowEdNode::IsInputPinShown, Input->GetName())),
							NAME_None, EUserInterfaceActionType::ToggleButton);
					}
				}
			}
		}
		FSlateApplication::Get().PushMenu(AsShared(),
			FWidgetPath(),
			MenuBuilder.MakeWidget(),
			FSlateApplication::Get().GetCursorPos(),
			FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu)
		);
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

EVisibility SDataflowEdNode::IsAddPinButtonVisible() const
{
	EVisibility Visibility = Super::IsAddPinButtonVisible();
	if (Visibility == EVisibility::Collapsed)
	{
		return Visibility;
	}

	if (DataflowGraphNode)
	{
		if (const TSharedPtr<FDataflowNode> DataflowNode = DataflowGraphNode->GetDataflowNode())
		{
			if (DataflowNode->HasHideableInputs() || DataflowNode->CanAddPin())
			{
				return Visibility;
			}
		}
	}

	return EVisibility::Collapsed;
}

void SDataflowEdNode::CopyDataflowNodeSettings(TSharedPtr<FDataflowNode> SourceDataflowNode, TSharedPtr<FDataflowNode> TargetDataflowNode)
{
	using namespace UE::Transaction;
	FSerializedObject SerializationObject;

	FSerializedObjectDataWriter ArWriter(SerializationObject);
	SourceDataflowNode->SerializeInternal(ArWriter);

	FSerializedObjectDataReader ArReader(SerializationObject);
	TargetDataflowNode->SerializeInternal(ArReader);
}

#undef LOCTEXT_NAMESPACE

