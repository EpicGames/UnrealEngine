// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowSNodeFactories.h"
#include "Dataflow/DataflowSNode.h"
#include "Dataflow/DataflowEdNode.h"
#include "Dataflow/DataflowEditor.h"
#include "Dataflow/DataflowCoreNodes.h"
#include "EdGraphNode_Comment.h"
#include "Dataflow/DataflowSCommentNode.h"
#include "Dataflow/DataflowSchema.h"
#include "SGraphNodeKnot.h"


TSharedPtr<SGraphNode> FDataflowGraphNodeFactory::CreateNodeWidget(UEdGraphNode* InNode)
{
	if (UDataflowEdNode* Node = Cast<UDataflowEdNode>(InNode))
	{
		if (const TSharedPtr<const FDataflowNode> DataflowNode = Node->GetDataflowNode())
		{
			if (DataflowNode->GetType() == FDataflowReRouteNode::StaticType())
			{
				return SNew(SGraphNodeKnot, Node);
			}
		}
		return SNew(SDataflowEdNode, Node)
			.DataflowInterface(DataflowInterface);
	}
	else if (UEdGraphNode_Comment* CommentNode = Cast<UEdGraphNode_Comment>(InNode))
	{
		if (CommentNode->GetSchema()->IsA(UDataflowSchema::StaticClass()))
		{
			return SNew(SDataflowEdNodeComment, CommentNode);
		}
	}
	return NULL;
}


