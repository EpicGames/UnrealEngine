// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowSPin.h"
#include "Dataflow/DataflowEdNode.h"

#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Layout/SBox.h"

#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "SDataflowPin"

void SDataflowPin::Construct(const FArguments& InArgs, UEdGraphPin* InPin)
{
	bIsPinInvalid = InArgs._IsPinInvalid.Get();
	PinColorOverride = InArgs._PinColorOverride.Get();
	bIsPinColorOverriden = InArgs._bIsPinColorOverriden.Get();

	const FText InvalidPinDisplayText = bIsPinInvalid.Get() ? LOCTEXT("DataflowOutputPinInvalidText", "*") : LOCTEXT("DataflowOutputPinValidText", " ");

	SGraphPin::Construct(SGraphPin::FArguments(), InPin);

	GetLabelAndValue()->AddSlot()
		.Padding(2.0f, 0.0f, 0.0f, 0.0f)
		[
			SNew(STextBlock)
			.Text_Lambda([InvalidPinDisplayText]()
				{
					return InvalidPinDisplayText;
				})
		.MinDesiredWidth(5)
		];
}

FSlateColor SDataflowPin::GetPinColor() const
{
	const bool IsColorOverriden = bIsPinColorOverriden.Get();
	const FLinearColor ColorOverride = PinColorOverride.Get();

	if (UEdGraphPin* GraphPin = GetPinObj())
	{
		if (bIsDiffHighlighted)
		{
			return FSlateColor(FLinearColor(0.9f, 0.2f, 0.15f));
		}
		if (GraphPin->bOrphanedPin)
		{
			return FSlateColor(FLinearColor::Red);
		}
		if (const UEdGraphSchema* Schema = GraphPin->GetSchema())
		{
			if (!GetPinObj()->GetOwningNode()->IsNodeEnabled() || GetPinObj()->GetOwningNode()->IsDisplayAsDisabledForced() || !IsEditingEnabled() || GetPinObj()->GetOwningNode()->IsNodeUnrelated())
			{
				if (IsColorOverriden)
				{
					return FSlateColor(ColorOverride);
				}

				return Schema->GetPinTypeColor(GraphPin->PinType) * FLinearColor(1.0f, 1.0f, 1.0f, 0.5f);
			}

			if (IsColorOverriden)
			{
				return FSlateColor(ColorOverride);
			}

			return Schema->GetPinTypeColor(GraphPin->PinType) * PinColorModifier;
		}
	}

	return FLinearColor::White;
}

TSharedRef<SWidget> SDataflowPin::GetDefaultValueWidget()
{
	if (UEdGraphPin* GraphPin = GetPinObj())
	{
		if (UDataflowEdNode::SupportsEditablePinType(*GraphPin))
		{
			const FName Pintype = GraphPin->PinType.PinCategory;
			if (!GraphPin->PinType.IsArray())
			{
				if (Pintype == TDataflowPolicyTypeName<bool>::GetName())
				{
					return MakeBoolWidget(*GraphPin);
				}
				else if (Pintype == TDataflowPolicyTypeName<int32>::GetName())
				{
					return MakeIntWidget(*GraphPin);
				}
				else if (Pintype == TDataflowPolicyTypeName<float>::GetName())
				{
					return MakeFloatWidget(*GraphPin);
				}
				else if (Pintype == TDataflowPolicyTypeName<FString>::GetName())
				{
					return MakeStringWidget(*GraphPin);
				}
			}
		}
	}
	return SGraphPin::GetDefaultValueWidget();
}

TSharedPtr<FDataflowNode> SDataflowPin::GetDataflowNode() const
{
	if (UEdGraphPin* GraphPin = GetPinObj())
	{
		if (UDataflowEdNode* EdNode = Cast<UDataflowEdNode>(GraphPin->GetOwningNode()))
		{
			return EdNode->GetDataflowNode();
		}
	}
	return {};
}

//////////////////////////////////////////////////////////////////////////////////////////

ECheckBoxState SDataflowPin::GetBoolValue() const
{
	if (UEdGraphPin* GraphPin = GetPinObj()) 
	{
		FString CurrentValue = GraphPin->GetDefaultAsString();
		return CurrentValue.ToBool() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}
	return ECheckBoxState::Unchecked;
}

void SDataflowPin::SetBoolValue(ECheckBoxState InIsChecked)
{
	if (UEdGraphPin* GraphPin = GetPinObj()) 
	{
		const FString BoolString = (InIsChecked == ECheckBoxState::Checked) ? TEXT("true") : TEXT("false");
		if (GraphPin->GetDefaultAsString() != BoolString)
		{
			const FScopedTransaction Transaction(LOCTEXT("ChangeBoolPinValue", "Change Bool Pin Value"));
			GraphPin->Modify();
			GraphPin->GetSchema()->TrySetDefaultValue(*GraphPin, BoolString);
		}
	}
}

TSharedRef<SWidget> SDataflowPin::MakeBoolWidget(const UEdGraphPin& Pin)
{
	return SNew(SCheckBox)
		.IsEnabled(this, &SGraphPin::GetDefaultValueIsEditable)
		.Visibility(this, &SGraphPin::GetDefaultValueVisibility)
		.IsChecked(this, &SDataflowPin::GetBoolValue)
		.OnCheckStateChanged(this, &SDataflowPin::SetBoolValue)
		;
}

//////////////////////////////////////////////////////////////////////////////////////////

TOptional<int32> SDataflowPin::GetIntValue() const
{
	int32 Value = 0;
	if (UEdGraphPin* GraphPin = GetPinObj())
	{
		LexFromString(Value, *GraphPin->GetDefaultAsString());
	}
	return Value;
}

void SDataflowPin::SetIntValue(int32 InValue, ETextCommit::Type CommitType)
{
	if (UEdGraphPin* GraphPin = GetPinObj())
	{
		int32 CurrentValue = 0;
		LexFromString(CurrentValue, *GraphPin->GetDefaultAsString());
		if (CurrentValue != InValue)
		{
			const FScopedTransaction Transaction(LOCTEXT("ChangeIntPinValue", "Change Integer Pin Value"));
			GraphPin->Modify();
			GraphPin->GetSchema()->TrySetDefaultValue(*GraphPin, *LexToString(InValue));
		}
	}
}

TSharedRef<SWidget>	SDataflowPin::MakeIntWidget(const UEdGraphPin& Pin)
{
	return SNew(SBox)
		.MinDesiredWidth(35.0f)
		.MaxDesiredWidth(100.0f)
		[
			SNew(SNumericEntryBox<int32>)
				.IsEnabled(this, &SGraphPin::GetDefaultValueIsEditable)
				.Visibility(this, &SGraphPin::GetDefaultValueVisibility)
				.EditableTextBoxStyle(FAppStyle::Get(), "Graph.EditableTextBox")
				.BorderForegroundColor(FSlateColor::UseForeground())
				.Value(this, &SDataflowPin::GetIntValue)
				.OnValueCommitted(this, &SDataflowPin::SetIntValue)
		];
}

//////////////////////////////////////////////////////////////////////////////////////////

TOptional<float> SDataflowPin::GetFloatValue() const
{
	float Value = 0;
	if (UEdGraphPin* GraphPin = GetPinObj())
	{
		LexFromString(Value, *GraphPin->GetDefaultAsString());
	}
	return Value;
}

void SDataflowPin::SetFloatValue(float InValue, ETextCommit::Type CommitType)
{
	if (UEdGraphPin* GraphPin = GetPinObj())
	{
		float CurrentValue = 0.0f;
		LexFromString(CurrentValue, *GraphPin->GetDefaultAsString());
		if (CurrentValue != InValue)
		{
			const FScopedTransaction Transaction(LOCTEXT("ChangeFloatPinValue", "Change Float Pin Value"));
			GraphPin->Modify();
			GraphPin->GetSchema()->TrySetDefaultValue(*GraphPin, *LexToString(InValue));
		}
	}
}

TSharedRef<SWidget>	SDataflowPin::MakeFloatWidget(const UEdGraphPin& Pin)
{
	return SNew(SBox)
		.MinDesiredWidth(35.0f)
		.MaxDesiredWidth(100.0f)
		[
			SNew(SNumericEntryBox<float>)
				.IsEnabled(this, &SGraphPin::GetDefaultValueIsEditable)
				.Visibility(this, &SGraphPin::GetDefaultValueVisibility)
				.EditableTextBoxStyle(FAppStyle::Get(), "Graph.EditableTextBox")
				.BorderForegroundColor(FSlateColor::UseForeground())
				.Value(this, &SDataflowPin::GetFloatValue)
				.OnValueCommitted(this, &SDataflowPin::SetFloatValue)
		];
}

//////////////////////////////////////////////////////////////////////////////////////////

FText SDataflowPin::GetStringValue() const
{
	if (UEdGraphPin* GraphPin = GetPinObj())
	{
		return FText::FromString(GraphPin->GetDefaultAsString());
	}
	return {};
}

void SDataflowPin::SetStringValue(const FText& NewTypeInValue, ETextCommit::Type /*CommitInfo*/)
{
	if (UEdGraphPin* GraphPin = GetPinObj())
	{
		if (!GraphPin->GetDefaultAsString().Equals(NewTypeInValue.ToString()))
		{
			const FScopedTransaction Transaction(LOCTEXT("ChangeStringPinValue", "Change String Pin Value"));
			GraphPin->Modify();
			GraphPin->GetSchema()->TrySetDefaultValue(*GraphPinObj, NewTypeInValue.ToString());
		}
	}
}

TSharedRef<SWidget>	SDataflowPin::MakeStringWidget(const UEdGraphPin& Pin)
{
	return SNew(SBox)
		.MinDesiredWidth(100.0f)
		.MaxDesiredHeight(200.0f)
		[
			SNew(SMultiLineEditableTextBox)
				.IsEnabled(this, &SGraphPin::GetDefaultValueIsEditable)
				.Visibility(this, &SGraphPin::GetDefaultValueVisibility)
				.Style(FAppStyle::Get(), "Graph.EditableTextBox")
				.Text(this, &SDataflowPin::GetStringValue)
				.SelectAllTextWhenFocused(true)
				.OnTextCommitted(this, &SDataflowPin::SetStringValue)
				.ForegroundColor(FSlateColor::UseForeground())
				.WrapTextAt(400.0f)
				.ModiferKeyForNewLine(EModifierKey::Shift)
		];
}


#undef LOCTEXT_NAMESPACE
