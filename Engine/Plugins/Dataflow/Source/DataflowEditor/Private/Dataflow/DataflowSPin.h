// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SGraphPin.h"

class UEdGraphPin;
struct FDataflowNode;

/**
* class used to for all dataflow node pins 
*/
class SDataflowPin : public SGraphPin
{
public:
	SLATE_BEGIN_ARGS(SDataflowPin)
		: _IsPinInvalid(false)
		, _PinColorOverride(FLinearColor::White)
		, _bIsPinColorOverriden(false)
	{}
		SLATE_ATTRIBUTE(bool, IsPinInvalid)
		SLATE_ATTRIBUTE(FLinearColor, PinColorOverride)
		SLATE_ATTRIBUTE(bool, bIsPinColorOverriden)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UEdGraphPin* InPin);

	virtual FSlateColor GetPinColor() const override;
	virtual TSharedRef<SWidget> GetDefaultValueWidget() override;

private:
	TSharedPtr<FDataflowNode> GetDataflowNode() const;

	ECheckBoxState GetBoolValue() const;
	void SetBoolValue(ECheckBoxState InIsChecked);
	TSharedRef<SWidget> MakeBoolWidget(const UEdGraphPin& Pin);

	TSharedRef<SWidget>	MakeIntWidget(const UEdGraphPin& Pin);
	TOptional<int32> GetIntValue() const;
	void SetIntValue(int32 InValue, ETextCommit::Type CommitType);

	TSharedRef<SWidget>	MakeFloatWidget(const UEdGraphPin& Pin);
	TOptional<float> GetFloatValue() const;
	void SetFloatValue(float InValue, ETextCommit::Type CommitType);

	TSharedRef<SWidget>	MakeStringWidget(const UEdGraphPin& Pin);
	FText GetStringValue() const;
	void SetStringValue(const FText& NewTypeInValue, ETextCommit::Type /*CommitInfo*/);


	TAttribute<bool> bIsPinInvalid;
	TAttribute<FLinearColor> PinColorOverride;
	TAttribute<bool> bIsPinColorOverriden;
};

