// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataflowSceneProfileIndexStorage.h"
#include "Dataflow/DataflowConstructionScene.h"
#include "Dataflow/DataflowSimulationScene.h"
#include "Dataflow/DataflowEditorOptions.h"
#include "AssetViewerSettings.h"


namespace UE::Dataflow::Private
{
	int32 GetProfileIndex(const FString& SearchName)
	{
		if (UAssetViewerSettings* AssetViewerSettings = UAssetViewerSettings::Get())
		{
			const int32 FoundIndex = AssetViewerSettings->Profiles.IndexOfByPredicate([SearchName](const FPreviewSceneProfile& Profile)
			{
				return Profile.ProfileName == SearchName;
			});

			return FoundIndex;
		}

		return INDEX_NONE;
	}
}

FDataflowConstructionSceneProfileIndexStorage::FDataflowConstructionSceneProfileIndexStorage(TWeakPtr<FDataflowConstructionScene> InConstructionScene) :
	ConstructionScene(InConstructionScene)
{
}

void FDataflowConstructionSceneProfileIndexStorage::StoreProfileIndex(int32 Index)
{
	if (const UAssetViewerSettings* const AssetViewerSettings = UAssetViewerSettings::Get())
	{
		if (AssetViewerSettings->Profiles.IsValidIndex(Index))
		{
			if (UDataflowEditorOptions* const Options = UDataflowEditorOptions::StaticClass()->GetDefaultObject<UDataflowEditorOptions>())
			{
				Options->ConstructionProfileName = AssetViewerSettings->Profiles[Index].ProfileName;
				Options->SaveConfig();
			}

			if (const TSharedPtr<FDataflowConstructionScene> ConstructionScenePtr = ConstructionScene.Pin())
			{
				ConstructionScenePtr->SetCurrentProfileIndex(Index);
			}
		}
	}
}

int32 FDataflowConstructionSceneProfileIndexStorage::RetrieveProfileIndex()
{
	if (const UDataflowEditorOptions* const Options = UDataflowEditorOptions::StaticClass()->GetDefaultObject<UDataflowEditorOptions>())
	{
		const int32 FoundIndex = UE::Dataflow::Private::GetProfileIndex(Options->ConstructionProfileName);
		if (FoundIndex != INDEX_NONE)
		{
			const TSharedPtr<FDataflowConstructionScene> ConstructionScenePtr = ConstructionScene.Pin();
			if (ConstructionScenePtr && FoundIndex != ConstructionScenePtr->GetCurrentProfileIndex())
			{
				ConstructionScenePtr->SetCurrentProfileIndex(FoundIndex);
			}
			return FoundIndex;
		}
	}
	return INDEX_NONE;
}


FDataflowSimulationSceneProfileIndexStorage::FDataflowSimulationSceneProfileIndexStorage(TWeakPtr<FDataflowSimulationScene> InSimulationScene) :
	SimulationScene(InSimulationScene)
{
}

void FDataflowSimulationSceneProfileIndexStorage::StoreProfileIndex(int32 Index)
{
	if (const UAssetViewerSettings* const AssetViewerSettings = UAssetViewerSettings::Get())
	{
		if (AssetViewerSettings->Profiles.IsValidIndex(Index))
		{
			if (UDataflowEditorOptions* const Options = UDataflowEditorOptions::StaticClass()->GetDefaultObject<UDataflowEditorOptions>())
			{
				Options->SimulationProfileName = AssetViewerSettings->Profiles[Index].ProfileName;
				Options->SaveConfig();
			}

			if (const TSharedPtr<FDataflowSimulationScene> SimulationScenePtr = SimulationScene.Pin())
			{
				SimulationScenePtr->SetCurrentProfileIndex(Index);
			}
		}
	}
}

int32 FDataflowSimulationSceneProfileIndexStorage::RetrieveProfileIndex()
{
	if (const UDataflowEditorOptions* const Options = UDataflowEditorOptions::StaticClass()->GetDefaultObject<UDataflowEditorOptions>())
	{
		const int32 FoundIndex = UE::Dataflow::Private::GetProfileIndex(Options->SimulationProfileName);
		if (FoundIndex != INDEX_NONE)
		{
			const TSharedPtr<FDataflowSimulationScene> SimulationScenePtr = SimulationScene.Pin();
			if (SimulationScenePtr && FoundIndex != SimulationScenePtr->GetCurrentProfileIndex())
			{
				SimulationScenePtr->SetCurrentProfileIndex(FoundIndex);
			}
			return FoundIndex;
		}
	}
	return INDEX_NONE;
}
