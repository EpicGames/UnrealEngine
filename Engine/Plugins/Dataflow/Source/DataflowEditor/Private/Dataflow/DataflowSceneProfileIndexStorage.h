// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Dataflow/DataflowPreviewProfileController.h"

class FDataflowConstructionScene;
class FDataflowSimulationScene;

class FDataflowConstructionSceneProfileIndexStorage : public FDataflowPreviewProfileController::IProfileIndexStorage
{
public:

	explicit FDataflowConstructionSceneProfileIndexStorage(TWeakPtr<FDataflowConstructionScene> InConstructionScene);
	virtual ~FDataflowConstructionSceneProfileIndexStorage() override = default;

	virtual void StoreProfileIndex(int32 Index) override;
	virtual int32 RetrieveProfileIndex() override;

private:

	TWeakPtr<FDataflowConstructionScene> ConstructionScene;
};

class FDataflowSimulationSceneProfileIndexStorage : public FDataflowPreviewProfileController::IProfileIndexStorage
{
public:

	explicit FDataflowSimulationSceneProfileIndexStorage(TWeakPtr<FDataflowSimulationScene> InSimulationScene);
	virtual ~FDataflowSimulationSceneProfileIndexStorage() override = default;

	virtual void StoreProfileIndex(int32 Index) override;
	virtual int32 RetrieveProfileIndex() override;

private:

	TWeakPtr<FDataflowSimulationScene> SimulationScene;
};

