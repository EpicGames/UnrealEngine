// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowSchema.h"

#include "Dataflow/DataflowCoreNodes.h"
#include "Dataflow/DataflowEdNode.h"
#include "Dataflow/DataflowEditorCommands.h"
#include "Dataflow/DataflowEdNode.h"
#include "Dataflow/DataflowGraphEditor.h"
#include "Dataflow/DataflowGraphSchemaAction.h"
#include "Dataflow/DataflowNodeFactory.h"
#include "Dataflow/DataflowAssetEditUtils.h"
#include "Editor/AssetReferenceFilter.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraph.h"
#include "Framework/Commands/GenericCommands.h"
#include "Logging/LogMacros.h"
#include "ToolMenuSection.h"
#include "ToolMenu.h"
#include "GraphEditorActions.h"
#include "Dataflow/DataflowAnyTypeRegistry.h"
#include "Dataflow/DataflowSettings.h"
#include "Dataflow/DataflowObject.h"
#include "Dataflow/DataflowSubGraph.h"
#include "Dataflow/DataflowCategoryRegistry.h"
#include "ScopedTransaction.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowSchema)

#define LOCTEXT_NAMESPACE "DataflowNode"

static const float CDefaultWireThickness = 1.5f;

int32 UDataflowSchema::CurrentCacheRefreshID = 0;

UDataflowSchema::UDataflowSchema()
{
}

void UDataflowSchema::GetContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const
{
	if (Context->Node && !Context->Pin)
	{
		{
			FToolMenuSection& Section = Menu->AddSection("TestGraphSchemaNodeActions", LOCTEXT("GraphSchemaNodeActions_MenuHeader", "Node Actions"));
			{
				Section.AddMenuEntry(FGenericCommands::Get().Rename);
				Section.AddMenuEntry(FGenericCommands::Get().Delete);
				Section.AddMenuEntry(FGenericCommands::Get().Cut);
				Section.AddMenuEntry(FGenericCommands::Get().Copy);
				Section.AddMenuEntry(FGenericCommands::Get().Duplicate);
				Section.AddMenuEntry(FDataflowEditorCommands::Get().ToggleEnabledState, LOCTEXT("DataflowContextMenu_ToggleEnabledState", "Toggle Enabled State"));
				Section.AddMenuEntry(FGraphEditorCommands::Get().BreakNodeLinks);
				Section.AddSeparator(TEXT("ActionsSeparator1"));
				Section.AddMenuEntry(FDataflowEditorCommands::Get().AddOptionPin, LOCTEXT("DataflowContextMenu_AddOptionPin", "Add Option Pin"));
				Section.AddMenuEntry(FDataflowEditorCommands::Get().RemoveOptionPin, LOCTEXT("DataflowContextMenu_RemoveOptionPin", "Remove Option Pin"));
				Section.AddSeparator(TEXT("ActionsSeparator2"));
				Section.AddMenuEntry(FDataflowEditorCommands::Get().EvaluateNode);
				Section.AddSeparator(TEXT("ActionsSeparator3"));
#if 0  // Disabled for 5.6
				Section.AddMenuEntry(FDataflowEditorCommands::Get().FreezeNodes, LOCTEXT("DataflowContextMenu_Freeze", "Freeze"));
				Section.AddMenuEntry(FDataflowEditorCommands::Get().UnfreezeNodes, LOCTEXT("DataflowContextMenu_Unfreeze", "Unfreeze"));
#endif
			}
		}
		{
			FToolMenuSection& Section = Menu->AddSection("TestGraphSchemaOrganization", LOCTEXT("GraphSchemaOrganization_MenuHeader", "Organization"));
			{
				Section.AddSubMenu("Alignment", LOCTEXT("AlignmentHeader", "Alignment"), FText(), FNewToolMenuDelegate::CreateLambda([](UToolMenu* AlignmentMenu)
				{
					{
						FToolMenuSection& InSection = AlignmentMenu->AddSection("TestGraphSchemaAlignment", LOCTEXT("GraphSchemaAlignment_MenuHeader", "Align"));

						InSection.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesTop);
						InSection.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesMiddle);
						InSection.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesBottom);
						InSection.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesLeft);
						InSection.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesCenter);
						InSection.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesRight);
						InSection.AddMenuEntry(FGraphEditorCommands::Get().StraightenConnections);
					}

					{
						FToolMenuSection& InSection = AlignmentMenu->AddSection("TestGraphSchemaDistribution", LOCTEXT("GraphSchemaDistribution_MenuHeader", "Distribution"));
						InSection.AddMenuEntry(FGraphEditorCommands::Get().DistributeNodesHorizontally);
						InSection.AddMenuEntry(FGraphEditorCommands::Get().DistributeNodesVertically);
					}
				}));
			}
		}
		{
			FToolMenuSection& Section = Menu->AddSection("TestGraphSchemaDisplay", LOCTEXT("GraphSchemaDisplay_MenuHeader", "Display"));
			{
				Section.AddSubMenu("PinVisibility", LOCTEXT("PinVisibilityHeader", "Pin Visibility"), FText(),
					FNewToolMenuDelegate::CreateLambda([](UToolMenu* PinVisibilityMenu)
				{
					FToolMenuSection& InSection = PinVisibilityMenu->AddSection("TestGraphSchemaPinVisibility");
					InSection.AddMenuEntry(FGraphEditorCommands::Get().ShowAllPins);
					InSection.AddMenuEntry(FGraphEditorCommands::Get().HideNoConnectionPins);
				}));
			}

		}
	}
	Super::GetContextMenuActions(Menu, Context);

	if (const UDataflowEdNode* EdNode = Cast<UDataflowEdNode>(Context->Node))
	{
		if (Context->Pin && Context->Pin->Direction == EGPD_Output)
		{
			if (FDataflowConnection* Connection = UDataflowEdNode::GetConnectionFromPin(Context->Pin))
			{
				if (Connection->IsAnyType() && !Connection->IsTypeLocked())
				{
					TArray<FName> SupportedTypes;
					Connection->GetSupportedTypes(SupportedTypes);
					if (SupportedTypes.Num() > 0)
					{
						FToolMenuSection& Section = Menu->AddSection("DataflowSchema_PinContextMenu_SectionTypes", LOCTEXT("DataflowSchema_PinContextMenu_SectionTypes_Text", "Compatible Types"));
						for (FName SupportedType : SupportedTypes)
						{
							Section.AddMenuEntry(
								SupportedType,
								FText::Format(LOCTEXT("PinConvertType_Label", "Convert to {0}"), FText::FromName(SupportedType)),
								FText::Format(LOCTEXT("PinConvertType_Tooltip", "Convert the output type {0}"), FText::FromName(SupportedType)),
								FSlateIcon(),
								FUIAction(
									// Fine to capture raw pointers because the lifetime of teh menu is shorter than the connection and node lifetime
									FExecuteAction::CreateLambda([SupportedType, Connection, EdNode]()
										{
											if (FDataflowNode* Node = Connection->GetOwningNode())
											{
												Node->SetOutputConnectionConcreteType(Connection, SupportedType);
												Node->NotifyConnectionTypeChanged(Connection);

												// teh context only gives a const node, but we need to update the pins 
												if (UDataflowEdNode* MutableEdNode = const_cast<UDataflowEdNode*>(EdNode))
												{
													MutableEdNode->UpdatePinsFromDataflowNode();
												}
												Node->Invalidate();
											}
										}),
									// Fine to capture raw pointers because the lifetime of teh menu is shorter than the connection and node lifetime
									FCanExecuteAction::CreateLambda([SupportedType, Connection]()
										{
											return (Connection 
												&& !Connection->IsTypeLocked() 
												&& !Connection->IsConnected() 
												&& Connection->GetType() != SupportedType
												&& !Connection->IsAnytypeDependencyConnected()
												);
										})
								)
							);
						}
					}
				}

				FToolMenuSection& Section = Menu->AddSection("DataflowSchema_PinContextMenu_SectionDebug", LOCTEXT("DataflowSchema_PinContextMenu_SectionDebug_Text", "Debug"));
				const bool bConnectionWatched = EdNode->IsConnectionWatched(*Connection);
				Section.AddMenuEntry(bConnectionWatched? FGraphEditorCommands::Get().StopWatchingPin: FGraphEditorCommands::Get().StartWatchingPin);
			}
		}
		else if (Context->Pin && Context->Pin->Direction == EGPD_Input)
		{
			if (FDataflowConnection* Connection = UDataflowEdNode::GetConnectionFromPin(Context->Pin))
			{
				FToolMenuSection& Section = Menu->FindOrAddSection("Pin Actions");
				Section.AddMenuEntry(FGraphEditorCommands::Get().PromoteToVariable);

				if (TSharedPtr<const FDataflowNode> DataflowNode = UDataflowEdNode::GetDataflowNodeFromEdNode(EdNode))
				{
					if (DataflowNode->CanAddPin())
					{
						Section.AddSeparator(TEXT("PinActionSeperator_OptionPins"));
						Section.AddMenuEntry(FDataflowEditorCommands::Get().AddOptionPin, LOCTEXT("DataflowContextPinMenu_AddOptionPin", "Add Option Pin"));
						Section.AddMenuEntry(FDataflowEditorCommands::Get().RemoveOptionPin, LOCTEXT("DataflowContextPinMenu_RemoveOptionPin", "Remove Last Option Pin"));
					}
				}
			}
		}
	}
}
bool UDataflowSchema::CanPinBeConnectedToNode(const UEdGraphPin* Pin, const UE::Dataflow::FFactoryParameters& NodeParameters)
{
	if (Pin == nullptr)
	{
		// if there's no pulled pin, then all nodes are compatible by default 
		return true; 
	}
	if (NodeParameters.DefaultNodeObject)
	{
		// get the type from the dataflow input/output as the pin type may not be precise enough when using anytypes
		TSharedPtr<const FDataflowNode> DataflowNode;
		TSharedPtr<const UE::Dataflow::FGraph> DataflowGraph;
		if (UDataflowEdNode* EdNode = Cast<UDataflowEdNode>(Pin->GetOwningNode()))
		{
			DataflowNode = EdNode->GetDataflowNode();
			DataflowGraph = EdNode->GetDataflowGraph();
		}

		if (DataflowNode && DataflowGraph)
		{
			if (Pin->Direction == EEdGraphPinDirection::EGPD_Output)
			{
				if (const FDataflowOutput* PinOutput = DataflowNode->FindOutput(Pin->PinName))
				{
					const TArray<FDataflowInput*> Inputs = NodeParameters.DefaultNodeObject->GetInputs();
					for (const FDataflowInput* Input : Inputs)
					{
						if (Input && DataflowGraph->CanConnect(*PinOutput, *Input))
						{
							return true;
						}
					}
				}
			}
			else if (Pin->Direction == EEdGraphPinDirection::EGPD_Input)
			{
				if (const FDataflowInput* PinInput = DataflowNode->FindInput(Pin->PinName))
				{
					const TArray<FDataflowOutput*> Outputs = NodeParameters.DefaultNodeObject->GetOutputs();
					for (const FDataflowOutput* Output : Outputs)
					{
						if (Output && DataflowGraph->CanConnect(*Output, *PinInput))
						{
							return true;
						}
					}
				}
			}
		}
	}
	return false;
}

FName UDataflowSchema::GetEditedAssetType()
{
	if (TSharedPtr<SDataflowGraphEditor> GraphEditor = SDataflowGraphEditor::GetLastActionMenuGraphEditor().Pin())
	{
		if (TSharedPtr<UE::Dataflow::FContext> DataflowContext = GraphEditor->GetDataflowContext())
		{
			if (const UE::Dataflow::FEngineContext* EngineContext = DataflowContext->AsType<UE::Dataflow::FEngineContext>())
			{
				const FName AssetType = EngineContext->Owner.GetClass()->GetFName();
				if (AssetType != UDataflow::StaticClass()->GetFName())
				{
					return AssetType;
				}
			}
		}
	}
	return NAME_None;
}

bool UDataflowSchema::IsCategorySupported(FName NodeCategory, FName AssetType)
{
	bool bFilteringByAssettypeEnable = true;
	if (TSharedPtr<SDataflowGraphEditor> GraphEditor = SDataflowGraphEditor::GetLastActionMenuGraphEditor().Pin())
	{
		bFilteringByAssettypeEnable &= GraphEditor->GetFilterActionMenyByAssetType();
	}
	if (!bFilteringByAssettypeEnable)
	{
		return true;
	}
	if (AssetType.IsNone())
	{
		return true;
	}
	FString NodeCategoryStr = NodeCategory.ToString();
	int32 SeparatorPos = NodeCategoryStr.Find("|");
	if (SeparatorPos != INDEX_NONE)
	{
		NodeCategoryStr.LeftInline(SeparatorPos, EAllowShrinking::No);
		return UE::Dataflow::FCategoryRegistry::Get().IsCategoryForAssetType(FName(NodeCategoryStr), AssetType);
	}
	return UE::Dataflow::FCategoryRegistry::Get().IsCategoryForAssetType(NodeCategory, AssetType);
}

void UDataflowSchema::GetGraphContextActions(FGraphContextMenuBuilder& ContextMenuBuilder) const
{
	const FName AssetType = UDataflowSchema::GetEditedAssetType();

	if (UDataflow* DataflowAsset = UDataflow::GetDataflowAssetFromEdGraph(const_cast<UEdGraph*>(ContextMenuBuilder.CurrentGraph)))
	{
		const bool bDisplayVariable = (ContextMenuBuilder.FromPin == nullptr || ContextMenuBuilder.FromPin->Direction == EEdGraphPinDirection::EGPD_Input);
		if (bDisplayVariable)
		{
			// Variables
			const FText VariablesCategory = LOCTEXT("DataflowContextActionVariablesCategory", "Variables");
			if (const UPropertyBag* PropertyBag = DataflowAsset->Variables.GetPropertyBagStruct())
			{
				for (const FPropertyBagPropertyDesc& PropertyDesc : PropertyBag->GetPropertyDescs())
				{
					// Todo : check if the type of the variable is compatible with the FromPin types
					TSharedPtr<FEdGraphSchemaAction_DataflowVariable> VariableAction =
						MakeShareable(new FEdGraphSchemaAction_DataflowVariable(DataflowAsset, PropertyDesc));
					if (VariableAction)
					{
						VariableAction->CosmeticUpdateRootCategory(VariablesCategory);
						ContextMenuBuilder.AddAction(VariableAction);
					}
				}
			}
		}
		// SubGraph/functions
		const FText SubGraphsCategory = LOCTEXT("DataflowContextActionSubGraphsCategory", "SubGraphs");
		for (UDataflowSubGraph* SubGraph : DataflowAsset->GetSubGraphs())
		{
			// Todo : check if the input / output node of the subgraph are compatible with the FromPin types
			TSharedPtr<FEdGraphSchemaAction_DataflowSubGraph> SubGraphAction =
				MakeShareable(new FEdGraphSchemaAction_DataflowSubGraph(DataflowAsset, SubGraph->GetSubGraphGuid()));
			if (SubGraphAction)
			{
				SubGraphAction->CosmeticUpdateRootCategory(SubGraphsCategory);
				ContextMenuBuilder.AddAction(SubGraphAction);
			}
		}
	}
	if (UE::Dataflow::FNodeFactory* Factory = UE::Dataflow::FNodeFactory::GetInstance())
	{
		for (UE::Dataflow::FFactoryParameters NodeParameters : Factory->RegisteredParameters())
		{
			if (UDataflowSchema::IsCategorySupported(NodeParameters.Category, AssetType))
			{
				// contextual filtering ( if pin is null always return true )
				if (UDataflowSchema::CanPinBeConnectedToNode(ContextMenuBuilder.FromPin, NodeParameters))
				{
					if (TSharedPtr<FAssetSchemaAction_Dataflow_CreateNode_DataflowEdNode> Action =
						FAssetSchemaAction_Dataflow_CreateNode_DataflowEdNode::CreateAction(ContextMenuBuilder.CurrentGraph, NodeParameters.TypeName, NodeParameters.DisplayName))
					{
						ContextMenuBuilder.AddAction(Action);
					}
				}
			}
		}
	}
}

bool HasLoopIfConnected(const UEdGraphNode* FromNode, const UEdGraphNode* ToNode)
{
	if (ToNode == FromNode)
	{
		return true;
	}

	// We only need to process from the FromNode and test if anything in the feeding nodes contains ToNode
	TArray<const UEdGraphNode*> NodesToProcess;
	NodesToProcess.Push(FromNode);

	// to speed things up, we do not revisit branches we have already look at  
	TSet<const UEdGraphNode*> VisitedNodes;

	while (NodesToProcess.Num() > 0)
	{
		const UEdGraphNode* NodeToProcess = NodesToProcess.Pop();
		if (!VisitedNodes.Contains(NodeToProcess))
		{
			VisitedNodes.Add(NodeToProcess);

			int32 NumConnectedInputPins = 0;
			for (UEdGraphPin* Pin : NodeToProcess->GetAllPins())
			{
				if (Pin->Direction == EEdGraphPinDirection::EGPD_Input)
				{
					if (Pin->HasAnyConnections())
					{
						NumConnectedInputPins++;
						if (ensure(Pin->LinkedTo.Num() == 1))
						{
							if (const UEdGraphNode* OwningNode = Pin->LinkedTo[0]->GetOwningNode())
							{
								if (OwningNode == ToNode)
								{
									return true;
								}
								NodesToProcess.Push(OwningNode);
							}
						}
					}
				}
			}
		}

	}

	return false;
}

TSharedPtr<UE::Dataflow::FGraph> UDataflowSchema::GetDataflowGraphFromPin(const UEdGraphPin& Pin)
{
	return UDataflowEdNode::GetDataflowGraphFromPin(Pin);
}

TSharedPtr<const FDataflowNode> UDataflowSchema::GetDataflowNodeFromPin(const UEdGraphPin& Pin)
{
	return UDataflowEdNode::GetDataflowNodeFromPin(Pin);
}

const FDataflowInput* UDataflowSchema::GetDataflowInputFromPin(const UEdGraphPin& Pin)
{
	return UDataflowEdNode::GetDataflowInputFromPin(Pin);
}

const FDataflowOutput* UDataflowSchema::GetDataflowOutputFromPin(const UEdGraphPin& Pin)
{
	return UDataflowEdNode::GetDataflowOutputFromPin(Pin);
}

bool UDataflowSchema::CanConnectPins(const UEdGraphPin& OutputPin, const UEdGraphPin& InputPin)
{
	return UDataflowEdNode::CanConnectPins(OutputPin, InputPin);
}

bool UDataflowSchema::CanConvertPins(const UEdGraphPin& OutputPin, const UEdGraphPin& InputPin)
{
	return UDataflowEdNode::CanConvertPins(OutputPin, InputPin);
}

const FPinConnectionResponse UDataflowSchema::CanCreateConnection(const UEdGraphPin* InPinA, const UEdGraphPin* InPinB) const
{
	bool bSwapped = false;
	const UEdGraphPin* PinA = InPinA;
	const UEdGraphPin* PinB = InPinB;
	if (PinA->Direction == EEdGraphPinDirection::EGPD_Input &&
		PinB->Direction == EEdGraphPinDirection::EGPD_Output)
	{
		bSwapped = true;
		PinA = InPinB; PinB = InPinA;
	}

	if (PinA->Direction == EEdGraphPinDirection::EGPD_Output)
	{
		if (PinB->Direction == EEdGraphPinDirection::EGPD_Input)
		{
			// Make sure the pins are not on the same node
			UDataflowEdNode* EdNodeA = Cast<UDataflowEdNode>(PinA->GetOwningNode());
			UDataflowEdNode* EdNodeB = Cast<UDataflowEdNode>(PinB->GetOwningNode());

			if (EdNodeA && EdNodeB && (EdNodeA != EdNodeB))
			{
				if (HasLoopIfConnected(EdNodeA, EdNodeB))
				{
					return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("PinError_Loop", "Graph Cycle"));
				}

				const bool bCompatibleTypes = CanConnectPins(*PinA, *PinB);
				const bool bCanConvert = CanConvertPins(*PinA, *PinB);

				if (bCanConvert && !bCompatibleTypes)
				{
					return FPinConnectionResponse(CONNECT_RESPONSE_MAKE_WITH_CONVERSION_NODE, LOCTEXT("PinConnectWithConversion", "Connect input to output using a conversion node."));
				}
				if (bCompatibleTypes)
				{
					if (PinB->LinkedTo.Num())
					{
						return (bSwapped) ?
							FPinConnectionResponse(CONNECT_RESPONSE_BREAK_OTHERS_A, LOCTEXT("PinSteal", "Disconnect existing input and connect new input."))
							:
							FPinConnectionResponse(CONNECT_RESPONSE_BREAK_OTHERS_B, LOCTEXT("PinSteal", "Disconnect existing input and connect new input."));
					}
					return FPinConnectionResponse(CONNECT_RESPONSE_MAKE, LOCTEXT("PinConnect", "Connect input to output."));
				}
				else
				{
					return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("PinError_Type mismatch", "Type Mismatch"));
				}
			}
		}
	}
	TArray<FText> NoConnectionResponse = {
		LOCTEXT("PinErrorSameNode_Nope", "Nope"),
		LOCTEXT("PinErrorSameNode_Sorry", "Sorry :("),
		LOCTEXT("PinErrorSameNode_NotGonnaWork", "Not gonna work."),
		LOCTEXT("PinErrorSameNode_StillNo", "Still no!"),
		LOCTEXT("PinErrorSameNode_TryAgain", "Try again?"),
	};
	return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, NoConnectionResponse[FMath::RandRange(0, NoConnectionResponse.Num()-1)]);
}

bool UDataflowSchema::CreateAutomaticConversionNodeAndConnections(UEdGraphPin* InSourcePin, UEdGraphPin* InTargetPin) const
{
	if (!InSourcePin || !InTargetPin)
	{
		return false;
	}

	UEdGraphPin* FromPin = InSourcePin;
	UEdGraphPin* ToPin = InTargetPin;
	if (FromPin->Direction == EEdGraphPinDirection::EGPD_Input)
	{
		Swap(FromPin, ToPin);
	}
	if (FromPin->Direction != EEdGraphPinDirection::EGPD_Output && ToPin->Direction != EEdGraphPinDirection::EGPD_Input)
	{
		return false;
	}

	UDataflowEdNode* const SourceEdNode = Cast<UDataflowEdNode>(FromPin->GetOwningNode());
	UDataflowEdNode* const TargetEdNode = Cast<UDataflowEdNode>(ToPin->GetOwningNode());

	if (!SourceEdNode || !TargetEdNode)
	{
		return false;
	}

	const FDataflowOutput* Output = GetDataflowOutputFromPin(*FromPin);
	const FDataflowInput* Input = GetDataflowInputFromPin(*ToPin);
	if (!Output || !Input)
	{
		return false;
	}

	const FName AutoConvertNodeName = UE::Dataflow::FAnyTypesRegistry::GetAutoConvertNodeTypeStatic(Output->GetType(), Input->GetType());
	if (AutoConvertNodeName == NAME_None)
	{
		return false;
	}

	UEdGraph* EdGraph = SourceEdNode->GetGraph();


	TSharedPtr<FAssetSchemaAction_Dataflow_CreateNode_DataflowEdNode> NewAutoConvertNodeAction
		= FAssetSchemaAction_Dataflow_CreateNode_DataflowEdNode::CreateAction(EdGraph, AutoConvertNodeName);
	if (NewAutoConvertNodeAction)
	{
		const FVector2f AutoConvertNodeLocation = (SourceEdNode->GetPosition() + TargetEdNode->GetPosition()) * 0.5f;
		if (UEdGraphNode* AutoConvertEdNode = NewAutoConvertNodeAction->PerformAction(EdGraph, nullptr, AutoConvertNodeLocation, false))
		{
			UEdGraphPin* InputPin = nullptr;
			UEdGraphPin* OutputPin = nullptr;
			for (UEdGraphPin* Pin : AutoConvertEdNode->GetAllPins())
			{
				if (InputPin == nullptr && Pin && Pin->Direction == EEdGraphPinDirection::EGPD_Input)
				{
					if (CanConnectPins(*FromPin, *Pin))
					{
						InputPin = Pin;
					}
				}
				if (OutputPin == nullptr && Pin && Pin->Direction == EEdGraphPinDirection::EGPD_Output)
				{
					if (CanConnectPins(*Pin, *ToPin))
					{
						OutputPin = Pin;
					}
				}
				if (InputPin && OutputPin)
				{
					break;
				}
			}
			if (InputPin && OutputPin)
			{
				ToPin->BreakAllPinLinks(true);
				EdGraph->GetSchema()->TryCreateConnection(FromPin, InputPin);
				EdGraph->GetSchema()->TryCreateConnection(OutputPin, ToPin);
				return true;
			}
		}
	}

	return false;
}

UE::Dataflow::FPin::EDirection UDataflowSchema::GetDirectionFromPinDirection(EEdGraphPinDirection InPinDirection)
{
	switch (InPinDirection)
	{
	case EEdGraphPinDirection::EGPD_Input:
		return UE::Dataflow::FPin::EDirection::INPUT;
	case EEdGraphPinDirection::EGPD_Output:
		return UE::Dataflow::FPin::EDirection::OUTPUT;
	default:
		return UE::Dataflow::FPin::EDirection::NONE;
	}
}

void UDataflowSchema::GetAssetsGraphHoverMessage(const TArray<FAssetData>&Assets, const UEdGraph * HoverGraph, FString& OutTooltipText, bool& OutOkIcon) const
{
	using FDataflowNodeFactory = UE::Dataflow::FNodeFactory;

	const TSharedPtr<IAssetReferenceFilter> AssetReferenceFilter = UE::Dataflow::FEditAssetUtils::MakeAssetReferenceFilter(HoverGraph);

	OutTooltipText.Reset();
	OutOkIcon = false;
	
	for (const FAssetData& AssetData : Assets)
	{
		if (UObject* AssetObject = AssetData.GetAsset())
		{
			const FName GetterNodeType = FDataflowNodeFactory::GetInstance()->GetGetterNodeFromAssetClass(*AssetObject->GetClass());
			if (!GetterNodeType.IsNone())
			{
				if (AssetReferenceFilter)
				{
					FText FailureReason;
					if (!AssetReferenceFilter->PassesFilter(AssetData, &FailureReason))
					{
						if (OutTooltipText.IsEmpty())
						{
							OutTooltipText = FailureReason.ToString();
						}
						continue;
					}
				}

				OutTooltipText = LOCTEXT("DataflowSchema_DropAssetOnGraphSupported", "Place as a getter node here.").ToString();
				OutOkIcon = true;
				return; // at least one is require to be a positive drop
			}
		}
	}

	if (OutTooltipText.IsEmpty())
	{
		OutTooltipText = LOCTEXT("DataflowSchema_NoDropAssetOnGraphSupported", "No supported node found").ToString();
	}
}

void UDataflowSchema::DroppedAssetsOnGraph(const TArray<FAssetData>& Assets, const FVector2f& GraphPosition, UEdGraph* Graph) const
{
	using FDataflowNodeFactory = UE::Dataflow::FNodeFactory;

	const TSharedPtr<IAssetReferenceFilter> AssetReferenceFilter = UE::Dataflow::FEditAssetUtils::MakeAssetReferenceFilter(Graph);
	const FVector2f NodeOffset{ 0.0f,100.0f };
	FVector2f NodePosition = GraphPosition;

	for (const FAssetData& AssetData: Assets)
	{
		if (UObject* AssetObject = AssetData.GetAsset())
		{
			const FName GetterNodeType = FDataflowNodeFactory::GetInstance()->GetGetterNodeFromAssetClass(*AssetObject->GetClass());
			if (!GetterNodeType.IsNone())
			{
				if (AssetReferenceFilter.IsValid() && !AssetReferenceFilter->PassesFilter(AssetData))
				{
					continue;
				}

				if (UDataflowEdNode* DataflowEdNode = UE::Dataflow::FEditAssetUtils::AddNewNode(Graph, FDeprecateSlateVector2D(NodePosition), AssetObject->GetFName(), GetterNodeType, nullptr))
				{
					NodePosition += NodeOffset;
					if (TSharedPtr<FDataflowNode> DataflowNode = DataflowEdNode->GetDataflowNode())
					{
						DataflowNode->SetAssetProperty(AssetObject);
					}
				}
			}
		}
	}
}

void UDataflowSchema::GetAssetsNodeHoverMessage(const TArray<struct FAssetData>& Assets, const UEdGraphNode* HoverNode, FString& OutTooltipText, bool& OutOkIcon) const
{
	OutTooltipText.Reset();
	OutOkIcon = false;

	if (TSharedPtr<const FDataflowNode> DataflowNode = UDataflowEdNode::GetDataflowNodeFromEdNode(HoverNode))
	{
		const TSharedPtr<IAssetReferenceFilter> AssetReferenceFilter = UE::Dataflow::FEditAssetUtils::MakeAssetReferenceFilter(HoverNode->GetGraph());
		bool bNeedInvalidation = false;
		for (const FAssetData& AssetData : Assets)
		{
			if (UObject* AssetObject = AssetData.GetAsset())
			{
				if (DataflowNode->SupportsAssetProperty(AssetObject))
				{
					if (AssetReferenceFilter)
					{
						FText FailureReason;
						if (!AssetReferenceFilter->PassesFilter(AssetData, &FailureReason))
						{
							if (OutTooltipText.IsEmpty())
							{
								OutTooltipText = FailureReason.ToString();
							}
							continue;
						}
					}

					OutTooltipText = LOCTEXT("DataflowSchema_DropAssetOnNodeSupported", "Set asset property on this node.").ToString();
					OutOkIcon = true;
					return; // at least one is require to be a positive drop
				}
			}
		}
	}

	if (OutTooltipText.IsEmpty())
	{
		OutTooltipText = LOCTEXT("DataflowSchema_NoDropAssetOnNodeSupported", "Asset type unsupported by this node").ToString();
	}
}

void UDataflowSchema::DroppedAssetsOnNode(const TArray<struct FAssetData>& Assets, const FVector2f& GraphPosition, UEdGraphNode* Node) const
{
	if (TSharedPtr<FDataflowNode> DataflowNode = UDataflowEdNode::GetDataflowNodeFromEdNode(Node))
	{
		const TSharedPtr<IAssetReferenceFilter> AssetReferenceFilter = UE::Dataflow::FEditAssetUtils::MakeAssetReferenceFilter(Node->GetGraph());
		bool bNeedInvalidation = false;
		for (const FAssetData& AssetData : Assets)
		{
			if (UObject* AssetObject = AssetData.GetAsset())
			{
				if (DataflowNode->SupportsAssetProperty(AssetObject))
				{
					if (AssetReferenceFilter.IsValid() && !AssetReferenceFilter->PassesFilter(AssetData))
					{
						continue;
					}

					DataflowNode->SetAssetProperty(AssetObject);
					bNeedInvalidation = true;
					break;
				}
			}
		}
		if (bNeedInvalidation)
		{
			DataflowNode->Invalidate();
		}
	}
}

bool UDataflowSchema::SupportsDropPinOnNode(UEdGraphNode* InTargetNode, const FEdGraphPinType& InSourcePinType, EEdGraphPinDirection InSourcePinDirection, FText& OutMessage) const
{
	if (TSharedPtr<FDataflowNode> TargetDataflowNode = UDataflowEdNode::GetDataflowNodeFromEdNode(InTargetNode))
	{
		const UE::Dataflow::FPin::EDirection SourceDirection = GetDirectionFromPinDirection(InSourcePinDirection);
		if (TargetDataflowNode->SupportsDropConnectionOnNode(InSourcePinType.PinCategory, SourceDirection))
		{
			OutMessage = LOCTEXT("DataflowSchema_DropPinOnNodeSupported", "Add pin to this node");
			return true;
		}
	}
	OutMessage = LOCTEXT("DataflowSchema_NoDropPinOnNodeSupport", "This node does not support this pin type");
	return false;
}

UEdGraphPin* UDataflowSchema::DropPinOnNode(UEdGraphNode* InTargetNode, const FName& InSourcePinName, const FEdGraphPinType& InSourcePinType, EEdGraphPinDirection InSourcePinDirection) const
{
	UEdGraphPin* ResultPin = nullptr;

	if (TSharedPtr<FDataflowNode> TargetDataflowNode = UDataflowEdNode::GetDataflowNodeFromEdNode(InTargetNode))
	{
		if (const FDataflowConnection* SourceConnection = UDataflowEdNode::GetConnectionFromPin(PinBeingDropped))
		{
			if (const FDataflowConnection* NewConnection = TargetDataflowNode->OnDropConnectionOnNode(*SourceConnection))
			{
				if (UDataflowEdNode* TargetDataflowEdNode = Cast<UDataflowEdNode>(InTargetNode))
				{
					TargetDataflowEdNode->UpdatePinsFromDataflowNode();
					ResultPin = TargetDataflowEdNode->FindPin(NewConnection->GetName());
				}
			}
		}
	}
	return ResultPin;
}

FLinearColor UDataflowSchema::GetPinTypeColor(const FEdGraphPinType& PinType) const
{
	return GetTypeColor(PinType.PinCategory);
}

FLinearColor UDataflowSchema::GetTypeColor(const FName& Type)
{
	const UGraphEditorSettings* Settings = GetDefault<UGraphEditorSettings>();
	const UDataflowSettings* DataflowSettings = GetDefault<UDataflowSettings>();

	if (::UE::Dataflow::FPinSettingsRegistry::Get().IsPinTypeRegistered(Type))
	{
		return ::UE::Dataflow::FPinSettingsRegistry::Get().GetPinColor(Type);
	}

	return Settings->DefaultPinTypeColor;
}

float UDataflowSchema::GetPinTypeWireThickness(const FName& Type) const
{
	const UGraphEditorSettings* Settings = GetDefault<UGraphEditorSettings>();
	const UDataflowSettings* DataflowSettings = GetDefault<UDataflowSettings>();

	if (::UE::Dataflow::FPinSettingsRegistry::Get().IsPinTypeRegistered(Type))
	{
		return ::UE::Dataflow::FPinSettingsRegistry::Get().GetPinWireThickness(Type);
	}

	return CDefaultWireThickness;
}

static void CreateAndConnectNewReRouteNode(UEdGraphPin* FromPin, UEdGraphPin* ToPin, const UE::Slate::FDeprecateVector2DParameter& GraphPosition)
{
	const UEdGraphNode* FromNode = FromPin->GetOwningNode();
	UEdGraph* EdGraph = FromNode->GetGraph();

	// Add the new reroute node and connect it 
	TSharedPtr<FAssetSchemaAction_Dataflow_CreateNode_DataflowEdNode> NewNodeAction 
		= FAssetSchemaAction_Dataflow_CreateNode_DataflowEdNode::CreateAction(EdGraph, FDataflowReRouteNode::StaticType());
	if (NewNodeAction)
	{
		UEdGraphNode* NewEdNode = NewNodeAction->PerformAction(EdGraph, nullptr, GraphPosition, false);
		if (NewEdNode)
		{
			const FName PinName = "Value";
			UEdGraphPin* InputPin = NewEdNode->FindPin(PinName, EGPD_Input);
			UEdGraphPin* OutputPin = NewEdNode->FindPin(PinName, EGPD_Output);
			if (InputPin && OutputPin)
			{
				EdGraph->GetSchema()->TryCreateConnection(FromPin, InputPin);
				EdGraph->GetSchema()->TryCreateConnection(OutputPin, ToPin);
			}
		}
	}
}


void UDataflowSchema::OnPinConnectionDoubleCicked(UEdGraphPin* PinA, UEdGraphPin* PinB, const FVector2f& GraphPosition) const
{
	CreateAndConnectNewReRouteNode(PinA, PinB, FDeprecateSlateVector2D(GraphPosition));
}

void UDataflowSchema::BreakPinLinks(UEdGraphPin& TargetPin, bool bSendsNodeNotifcation) const
{
	const FScopedTransaction Transaction(LOCTEXT("BreakPinLinks", "Break Pin Links"));
	Super::BreakPinLinks(TargetPin, bSendsNodeNotifcation);
}

bool UDataflowSchema::TryCreateConnection(UEdGraphPin* PinA, UEdGraphPin* PinB) const
{
	check(PinA && PinB);
	UDataflowEdNode* const DataflowEdNodeA = CastChecked<UDataflowEdNode>(PinA->GetOwningNodeUnchecked());
	UDataflowEdNode* const DataflowEdNodeB = CastChecked<UDataflowEdNode>(PinB->GetOwningNodeUnchecked());
	if (ensure(DataflowEdNodeA->IsBound() && DataflowEdNodeB->IsBound()))
	{
		const TSharedPtr<FDataflowNode> DataflowNodeA = DataflowEdNodeA->GetDataflowNode();
		const TSharedPtr<FDataflowNode> DataflowNodeB = DataflowEdNodeB->GetDataflowNode();
		if (ensure(DataflowNodeA && DataflowNodeB))
		{
			// Pausing invalidations is a quick hack while sorting the invalidation callbacks that are causing multiple evaluations
			DataflowNodeA->PauseInvalidations();
			DataflowNodeB->PauseInvalidations();
			const bool bModified = Super::TryCreateConnection(PinA, PinB);
			DataflowNodeA->ResumeInvalidations();
			DataflowNodeB->ResumeInvalidations();
			return bModified;
		}
	}
	return Super::TryCreateConnection(PinA, PinB);
}

TOptional<FLinearColor> UDataflowSchema::GetPinColorOverride(TSharedPtr<FDataflowNode> DataflowNode, UEdGraphPin* Pin) const
{
	TOptional<FLinearColor> OutColor;

	if (DataflowNode)
	{
		if (const UScriptStruct* ScriptStruct = DataflowNode->TypedScriptStruct())
		{
			FProperty* PinProperty = ScriptStruct->FindPropertyByName(Pin->GetFName());
			if (PinProperty && PinProperty->HasMetaData("PinColor"))
			{
				FLinearColor WireColor;

				const FString& ColorString = PinProperty->GetMetaData("PinColor");
				if (WireColor.InitFromString(ColorString))
				{
					OutColor = WireColor;
				}
			}
		}
	}

	return OutColor;
}

FConnectionDrawingPolicy* UDataflowSchema::CreateConnectionDrawingPolicy(int32 InBackLayerID, int32 InFrontLayerID, float InZoomFactor, const FSlateRect& InClippingRect, class FSlateWindowElementList& InDrawElements, class UEdGraph* InGraphObj) const
{
	return new FDataflowConnectionDrawingPolicy(InBackLayerID, InFrontLayerID, InZoomFactor, InClippingRect, InDrawElements, InGraphObj);
}


bool UDataflowSchema::IsCacheVisualizationOutOfDate(int32 InVisualizationCacheID) const
{
	return CurrentCacheRefreshID != InVisualizationCacheID;
}

int32 UDataflowSchema::GetCurrentVisualizationCacheID() const
{
	return CurrentCacheRefreshID;
}

void UDataflowSchema::ForceVisualizationCacheClear() const
{
	++CurrentCacheRefreshID;
}

//////////////////////////////////////////////////////////////////////////////////////////////////////////

FDataflowConnectionDrawingPolicy::FDataflowConnectionDrawingPolicy(int32 InBackLayerID, int32 InFrontLayerID, float InZoomFactor, const FSlateRect& InClippingRect, FSlateWindowElementList& InDrawElements, UEdGraph* InGraph)
	: FConnectionDrawingPolicy(InBackLayerID, InFrontLayerID, InZoomFactor, InClippingRect, InDrawElements)
	, Schema((UDataflowSchema*)(InGraph->GetSchema()))
{
	ArrowImage = nullptr;
	ArrowRadius = FVector2D::ZeroVector;
}

void FDataflowConnectionDrawingPolicy::DetermineWiringStyle(UEdGraphPin* OutputPin, UEdGraphPin* InputPin, /*inout*/ FConnectionParams& Params)
{
	FConnectionDrawingPolicy::DetermineWiringStyle(OutputPin, InputPin, Params);
	if (HoveredPins.Contains(InputPin) && HoveredPins.Contains(OutputPin))
	{
		Params.WireThickness = Params.WireThickness * 5;
	}

	const UDataflowSchema* DataflowSchema = GetSchema();
	if (DataflowSchema && OutputPin)
	{
		Params.WireColor = DataflowSchema->GetPinTypeColor(OutputPin->PinType);

		// Check if there is color override specified on the UProperty
		UDataflowEdNode* EdNode = Cast<UDataflowEdNode>(OutputPin->GetOwningNode());
		if (EdNode)
		{
			TSharedPtr<FDataflowNode> DataflowNode = EdNode->GetDataflowNode();
			if (DataflowNode)
			{
				TOptional<FLinearColor> OverrideColor = DataflowSchema->GetPinColorOverride(DataflowNode, OutputPin);
				if (OverrideColor.IsSet())
				{
					Params.WireColor = OverrideColor.GetValue();
				}
			}
		}

		float WireThickness = DataflowSchema->GetPinTypeWireThickness(OutputPin->PinType.PinCategory);
		WireThickness = WireThickness < CDefaultWireThickness ? CDefaultWireThickness : WireThickness;
		Params.WireThickness = WireThickness;
	}

	if (OutputPin && InputPin)
	{
		if (OutputPin->bOrphanedPin || InputPin->bOrphanedPin)
		{
			Params.WireColor = FLinearColor::Red;
		}
	}
}

void FDataflowConnectionDrawingPolicy::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(Schema);
}


#undef LOCTEXT_NAMESPACE

