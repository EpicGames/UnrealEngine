// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowSelectionView.h"

#include "Templates/EnableIf.h"
#include "Dataflow/DataflowContent.h"
//#include "Dataflow/DataflowEdNode.h"
#include "Dataflow/DataflowObjectInterface.h"
#include "Dataflow/DataflowSelection.h"
#include "Dataflow/SelectionViewWidget.h"

//#include "Widgets/SCompoundWidget.h"
//#include "Widgets/Views/SListView.h"


#define LOCTEXT_NAMESPACE "DataflowSelectionView"

FDataflowSelectionView::FDataflowSelectionView(TObjectPtr<UDataflowBaseContent> InContent)
	: FDataflowNodeView(InContent)
{

}

void FDataflowSelectionView::SetSupportedOutputTypes()
{
	GetSupportedOutputTypes().Empty();

	GetSupportedOutputTypes().Add("FDataflowTransformSelection");
	GetSupportedOutputTypes().Add("FDataflowVertexSelection");
	GetSupportedOutputTypes().Add("FDataflowFaceSelection");
}

void FDataflowSelectionView::UpdateViewData()
{
	if (SelectionView)
	{
		SelectionView->GetSelectionTable()->GetSelectionInfoMap().Empty();

		if (GetSelectedNode())
		{
			if (GetSelectedNode()->IsBound())
			{
				if (TSharedPtr<FDataflowNode> DataflowNode = GetSelectedNode()->DataflowGraph->FindBaseNode(GetSelectedNode()->DataflowNodeGuid))
				{
					if (const TObjectPtr<UDataflowBaseContent> Content = GetEditorContent())
					{
						if (TSharedPtr<UE::Dataflow::FEngineContext> Context = Content->GetDataflowContext())
						{
							TArray<FDataflowOutput*> Outputs = DataflowNode->GetOutputs();

							for (FDataflowOutput* Output : Outputs)
							{
								FName Name = Output->GetName();
								FName Type = Output->GetType();

								if (Output->GetType() == "FDataflowTransformSelection")
								{
									const FDataflowTransformSelection DefaultTransformSelection;
									const FDataflowTransformSelection& Value = Output->ReadValue(*Context, DefaultTransformSelection);

									SelectionView->GetSelectionTable()->GetSelectionInfoMap().Add(Name.ToString(), { Type.ToString(), TBitArray<>(Value.GetBitArray()) });
								}
								else if (Output->GetType() == "FDataflowVertexSelection")
								{
									const FDataflowVertexSelection DefaultVertexSelection;
									const FDataflowVertexSelection& Value = Output->ReadValue(*Context, DefaultVertexSelection);

									SelectionView->GetSelectionTable()->GetSelectionInfoMap().Add(Name.ToString(), { Type.ToString(), TBitArray<>(Value.GetBitArray()) });
								}
								else if (Output->GetType() == "FDataflowFaceSelection")
								{
									const FDataflowFaceSelection DefaultFaceSelection;
									const FDataflowFaceSelection& Value = Output->ReadValue(*Context, DefaultFaceSelection);

									SelectionView->GetSelectionTable()->GetSelectionInfoMap().Add(Name.ToString(), { Type.ToString(), TBitArray<>(Value.GetBitArray()) });
								}
							}
						}
					}
				}
			}

			SelectionView->SetData(GetSelectedNode()->GetName());
		}
		else
		{
			SelectionView->SetData(FString());
		}

		SelectionView->RefreshWidget();
	}
}


void FDataflowSelectionView::SetSelectionView(TSharedPtr<SSelectionViewWidget>& InSelectionView)
{
	ensure(!SelectionView);

	SelectionView = InSelectionView;

	if (SelectionView)
	{
		OnPinnedDownChangedDelegateHandle = SelectionView->GetOnPinnedDownChangedDelegate().AddRaw(this, &FDataflowSelectionView::OnPinnedDownChanged);
		OnRefreshLockedChangedDelegateHandle = SelectionView->GetOnRefreshLockedChangedDelegate().AddRaw(this, &FDataflowSelectionView::OnRefreshLockedChanged);
	}
}


FDataflowSelectionView::~FDataflowSelectionView()
{
	if (SelectionView)
	{
		SelectionView->GetOnPinnedDownChangedDelegate().Remove(OnPinnedDownChangedDelegateHandle);
	}
}


#undef LOCTEXT_NAMESPACE
