// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Dataflow/DataflowView.h"

class UDataflowEditor;
class SSelectionViewWidget;
class UPrimitiveComponent;

/**
*
* Class to handle the SelectionView widget
*
*/
class FDataflowSelectionView : public FDataflowNodeView
{
public:
	FDataflowSelectionView(TObjectPtr<UDataflowBaseContent> InContent = nullptr);
	~FDataflowSelectionView();

	virtual void SetSupportedOutputTypes() override;
	virtual void UpdateViewData() override;
	virtual void ConstructionViewSelectionChanged(const TArray<UPrimitiveComponent*>& SelectedComponents, const TArray<FDataflowBaseElement*>& SelectedElements) override {};
	virtual void SimulationViewSelectionChanged(const TArray<UPrimitiveComponent*>& SelectedComponents, const TArray<FDataflowBaseElement*>& SelectedElements) override {};

	void SetSelectionView(TSharedPtr<SSelectionViewWidget>& InSelectionView);

private:
	TSharedPtr<SSelectionViewWidget> SelectionView;

	FDelegateHandle OnPinnedDownChangedDelegateHandle;
	FDelegateHandle OnRefreshLockedChangedDelegateHandle;
};

