// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowSimulationControls.h"

#include "Animation/AnimSingleNodeInstance.h"
#include "Animation/AnimSequenceBase.h"
#include "Chaos/CacheCollection.h"
#include "Chaos/CacheManagerActor.h"
#include "Chaos/Adapters/CacheAdapter.h"
#include "Dataflow/DataflowContent.h"
#include "Dataflow/DataflowObject.h"
#include "Features/IModularFeatures.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/Blueprint.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/World.h"
#include "BonePose.h"
#include "AnimationRuntime.h"
#include "Dataflow/Interfaces/DataflowInterfaceGeometryCachable.h"
#define LOCTEXT_NAMESPACE "DataflowSimulationGenerator"

namespace UE::Dataflow
{
	bool ShouldResetWorld(const TObjectPtr<UDataflow>& SimulationGraph, const TObjectPtr<UWorld>& SimulationWorld, UE::Dataflow::FTimestamp& LastTimeStamp)
	{
		if(const TSharedPtr<UE::Dataflow::FGraph> DataflowGraph = SimulationGraph->GetDataflow())
		{
			UE::Dataflow::FTimestamp MaxTimeStamp = UE::Dataflow::FTimestamp::Invalid;
			for(const TSharedPtr<FDataflowNode>& TerminalNode : DataflowGraph->GetFilteredNodes(FDataflowTerminalNode::StaticType()))
			{
				MaxTimeStamp.Value = FMath::Max(MaxTimeStamp.Value, TerminalNode->GetTimestamp().Value);
			}
			if(MaxTimeStamp.Value > LastTimeStamp.Value)
			{
				LastTimeStamp = MaxTimeStamp.Value;
				return true;
			}
		}
		return false;
	}
	
	TObjectPtr<AActor> SpawnSimulatedActor(const TSubclassOf<AActor>& ActorClass,
		const TObjectPtr<AChaosCacheManager>& CacheManager, const TObjectPtr<UChaosCacheCollection>& CacheCollection,
		const bool bIsRecording, const TObjectPtr<UDataflowBaseContent>& DataflowContent, const FTransform& ActorTransform,
		const FInstancedPropertyBag& BlueprintVariables)
	{
		if(CacheManager)
		{
			const FString BaseName = CacheCollection ? CacheCollection->GetName() : TEXT("CacheActor");
			const uint32 CacheCollectionPathHash = CacheCollection ? GetTypeHash(CacheCollection->GetPathName()) : 0;
			const uint32 TerminalAssetPathHash = (DataflowContent && DataflowContent->GetTerminalAsset()) ? GetTypeHash(DataflowContent->GetTerminalAsset()->GetPathName()) : 0;
			const uint32 CacheActorHash = HashCombineFast(CacheCollectionPathHash, TerminalAssetPathHash);
			const FString CacheActorName = FString::Printf(TEXT("%s_%08X"), *BaseName, CacheActorHash);

			FActorSpawnParameters SpawnParameters;
			SpawnParameters.Name = FName(CacheActorName);
			SpawnParameters.NameMode = FActorSpawnParameters::ESpawnActorNameMode::Requested;
			SpawnParameters.Owner = CacheManager.Get(); 
			SpawnParameters.bDeferConstruction = true;
			
			TObjectPtr<AActor> PreviewActor = CacheManager->GetWorld()->SpawnActor<AActor>(ActorClass, SpawnParameters);
			if(PreviewActor)
			{
				// Init BP variables
				if (UBlueprintGeneratedClass* const BlueprintGeneratedClass = Cast<UBlueprintGeneratedClass>(PreviewActor->GetClass()))
				{
					if (UBlueprint* const Blueprint = Cast<UBlueprint>(BlueprintGeneratedClass->ClassGeneratedBy))
					{
						for (FBPVariableDescription& BPVariableDescription : Blueprint->NewVariables)
						{
							if (const FProperty* const BlueprintProperty = BlueprintGeneratedClass->FindPropertyByName(BPVariableDescription.VarName))
							{
								const FName VarGuidAsName(BPVariableDescription.VarGuid.ToString());  // VarGuid instead of the VarName, because the variable name can change and FInstancedPropertyBag::SanitizePropertyName
								if (const FPropertyBagPropertyDesc* const PropertyBagPropertyDesc = BlueprintVariables.FindPropertyDescByName(VarGuidAsName))
								{
									const FProperty* PropertyBagProperty = PropertyBagPropertyDesc->CachedProperty;
									if (BlueprintProperty->SameType(PropertyBagProperty))
									{
										const uint8* const SourceMemory = BlueprintVariables.GetValue().GetMemory() + PropertyBagProperty->GetOffset_ForInternal();
										uint8* const TargetMemory = BlueprintProperty->ContainerPtrToValuePtr<uint8>(PreviewActor);
										BlueprintProperty->CopyCompleteValue(TargetMemory, SourceMemory);
									}
								}
							}
						}
					}
				}

				// Link the editor content properties to the BP actor one 
				DataflowContent->SetActorProperties(PreviewActor);

				// Finish spawning
				PreviewActor->FinishSpawning(ActorTransform, true);
			}

			CacheManager->CacheCollection = CacheCollection;
			CacheManager->StartMode = EStartMode::Timed;
			CacheManager->CacheMode = bIsRecording ? ECacheMode::Record : ECacheMode::None;
			
			// Get the implementation of our adapters for identifying compatible components
			IModularFeatures&                      ModularFeatures = IModularFeatures::Get();
			TArray<Chaos::FComponentCacheAdapter*> Adapters = ModularFeatures.GetModularFeatureImplementations<Chaos::FComponentCacheAdapter>(Chaos::FComponentCacheAdapter::FeatureName);

			if(PreviewActor)
			{
				TInlineComponentArray<UPrimitiveComponent*> PrimComponents;
				PreviewActor->GetComponents(PrimComponents);
	
				for(UPrimitiveComponent* PrimComponent : PrimComponents)
				{
					if(Chaos::FAdapterUtil::GetBestAdapterForClass(PrimComponent->GetClass(), false))
					{
						const FName ChannelName(PrimComponent->GetName());
						CacheManager->FindOrAddObservedComponent(PrimComponent, ChannelName, true);
					}
				}
			}
			return PreviewActor;
		}
		return nullptr;
	}

	// UE_DEPRECATED(5.8, "Use BlueprintVariables version instead.")
	TObjectPtr<AActor> SpawnSimulatedActor(const TSubclassOf<AActor>& ActorClass,
		const TObjectPtr<AChaosCacheManager>& CacheManager, const TObjectPtr<UChaosCacheCollection>& CacheCollection,
		const bool bIsRecording, const TObjectPtr<UDataflowBaseContent>& DataflowContent, const FTransform& ActorTransform)
	{
		return SpawnSimulatedActor(ActorClass,
			CacheManager, CacheCollection,
			bIsRecording, DataflowContent, ActorTransform,
			FInstancedPropertyBag());
	}

	bool SetupSkeletonAnimation(const TObjectPtr<AActor>& PreviewActor, const bool bSkeletalMeshVisibility)
	{
		bool bAnimationWasSet = false;

		if(PreviewActor)
		{
			TInlineComponentArray<UPrimitiveComponent*> PrimComponents;
			PreviewActor->GetComponents(PrimComponents);

			TArray<IDataflowGeometryCachable*> GeometryCachables;
			for (UPrimitiveComponent* PrimComponent : PrimComponents)
			{
				if (IDataflowGeometryCachable* SkeletalMeshComponent = Cast<IDataflowGeometryCachable>(PrimComponent))
				{
					GeometryCachables.Add(SkeletalMeshComponent);
				}
			}

			for(UPrimitiveComponent* PrimComponent : PrimComponents)
			{
				if(USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(PrimComponent))
				{
					SkeletalMeshComponent->SetVisibility(bSkeletalMeshVisibility);
					SkeletalMeshComponent->SetAnimationMode(EAnimationMode::AnimationSingleNode);
					SkeletalMeshComponent->InitAnim(true);
					
					if(UAnimSingleNodeInstance* AnimNodeInstance = SkeletalMeshComponent->GetSingleNodeInstance())
					{
						for (IDataflowGeometryCachable* GeometryCachable : GeometryCachables)
						{
							if (!GeometryCachable->IsSkeletalMeshAnimationCompatible(SkeletalMeshComponent))
							{
								UE_LOGF(LogChaosSimulation, Warning, "Asset is not compatible with the skeletal mesh [%ls] for animation updates, check if Skeletons match", *SkeletalMeshComponent->GetSkeletalMeshAsset()->GetName());
							}
						}
						// Setup the animation instance
						AnimNodeInstance->SetAnimationAsset(SkeletalMeshComponent->AnimationData.AnimToPlay);
						AnimNodeInstance->InitializeAnimation();

						// Update the anim data
						SkeletalMeshComponent->AnimationData.PopulateFrom(AnimNodeInstance);
#if WITH_EDITOR
						SkeletalMeshComponent->ValidateAnimation();
#endif

						// Stop the animation 
						AnimNodeInstance->SetLooping(true);
						AnimNodeInstance->SetPlaying(false);

						bAnimationWasSet = true;
					}
				}
			}
		}
	
		return bAnimationWasSet;
	}

	static void FillAnimationDatas(const UAnimSequenceBase* AnimSequence, const float CurrentTime, USkeletalMeshComponent* SkeletalMeshComponent)
	{
		const USkeletalMesh* InSkeletalMesh = SkeletalMeshComponent->GetSkeletalMeshAsset();
		const FAnimExtractContext ExtractionContext(FMath::Clamp(CurrentTime, 0., AnimSequence->GetPlayLength()));

		if(const FReferenceSkeleton* ReferenceSkeleton = &InSkeletalMesh->GetRefSkeleton())
		{
			TArray<FTransform> ComponentSpaceTransforms = SkeletalMeshComponent->GetComponentSpaceTransforms();
			const int32 NumBones = ReferenceSkeleton->GetNum();

			TArray<FBoneIndexType> BoneIndices;
			BoneIndices.SetNumUninitialized(NumBones);
			for (int32 Index = 0; Index < NumBones; ++Index)
			{
				int32 SkeletonBoneIndex = InSkeletalMesh->GetSkeleton()->GetReferenceSkeleton().FindBoneIndex(ReferenceSkeleton->GetBoneName(Index));
				BoneIndices[Index] = StaticCast<FBoneIndexType>(SkeletonBoneIndex);
			}

			FBoneContainer BoneContainer;
			BoneContainer.SetUseRAWData(true);
			BoneContainer.InitializeTo(BoneIndices, UE::Anim::FCurveFilterSettings(), *InSkeletalMesh->GetSkeleton());
		
			FCompactPose CompactPose;
			CompactPose.SetBoneContainer(&BoneContainer);

			FBlendedCurve BlendedCurve;
			BlendedCurve.InitFrom(BoneContainer);

			UE::Anim::FStackAttributeContainer TempAttributes;
			FAnimationPoseData AnimationPoseData(CompactPose, BlendedCurve, TempAttributes);
			AnimSequence->GetAnimationPose(AnimationPoseData, ExtractionContext);
		
			FAnimationRuntime::FillUpComponentSpaceTransforms(*ReferenceSkeleton, AnimationPoseData.GetPose().GetBones(), ComponentSpaceTransforms);
			SkeletalMeshComponent->GetEditableComponentSpaceTransforms() = ComponentSpaceTransforms;

			FBlendedHeapCurve BlendedHeapCurve;
			BlendedHeapCurve.CopyFrom(AnimationPoseData.GetCurve());
			SkeletalMeshComponent->GetEditableAnimationCurves() = BlendedHeapCurve;
			SkeletalMeshComponent->ApplyEditedComponentSpaceTransforms();
		}
	}

	void ComputeSkeletonAnimation(const TObjectPtr<AActor>& PreviewActor, const float SimulationTime)
	{
		if(PreviewActor)
		{
			TInlineComponentArray<UPrimitiveComponent*> PrimComponents;
			PreviewActor->GetComponents(PrimComponents);
			// Update all the animation time 
			for(UPrimitiveComponent* PrimComponent : PrimComponents)
			{
				if(USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(PrimComponent))
				{
					if(UAnimSingleNodeInstance* SingleNodeInstance = Cast<UAnimSingleNodeInstance>(SkeletalMeshComponent->GetAnimInstance()))
					{
						if(const UAnimSequenceBase* AnimSequence = Cast<UAnimSequenceBase>(SingleNodeInstance->GetAnimationAsset()))
						{
							FillAnimationDatas(AnimSequence, SimulationTime, SkeletalMeshComponent);
						}
					}
				}
			}
		}
	}

	void UpdateSkeletonAnimation(const TObjectPtr<AActor>& PreviewActor, const float SimulationTime)
	{
		if(PreviewActor)
		{
			TInlineComponentArray<UPrimitiveComponent*> PrimComponents;
			PreviewActor->GetComponents(PrimComponents);

			// Update all the animation time 
			for(UPrimitiveComponent* PrimComponent : PrimComponents)
			{
				if(USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(PrimComponent))
				{
					// Make sure only Dataflow can update the animation to avoid the world tick to eventually update it again 
					// where this can cause issues where the animation jitter between two state when the dataflow timeline is paused
					// Note : we do not call SetEnableAnimation because it does not work in that way 
					// we want to explicily enable animation only for this portion fo the code and force it to false when done 
					SkeletalMeshComponent->bEnableAnimation = true;
					{
						SkeletalMeshComponent->SetPosition(SimulationTime);
						SkeletalMeshComponent->TickAnimation(0.f, false /*bNeedsValidRootMotion*/);
						SkeletalMeshComponent->RefreshBoneTransforms(nullptr /*TickFunction*/);

						SkeletalMeshComponent->RefreshFollowerComponents();
						SkeletalMeshComponent->UpdateComponentToWorld();
						SkeletalMeshComponent->FinalizeBoneTransform();
						SkeletalMeshComponent->MarkRenderTransformDirty();
						SkeletalMeshComponent->MarkRenderDynamicDataDirty();
					}
					SkeletalMeshComponent->bEnableAnimation = false;
				}
			}
		}
	}

	void StartSkeletonAnimation(const TObjectPtr<AActor>& PreviewActor)
	{
		if(PreviewActor)
		{
			TInlineComponentArray<UPrimitiveComponent*> PrimComponents;
			PreviewActor->GetComponents(PrimComponents);

			for(UPrimitiveComponent* PrimComponent : PrimComponents)
			{
				if(const USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(PrimComponent))
				{
					if(UAnimSingleNodeInstance* AnimNodeInstance = SkeletalMeshComponent->GetSingleNodeInstance())
					{
						AnimNodeInstance->SetPlaying(true);
					}
				}
			}
		}
	}
	
	void PauseSkeletonAnimation(const TObjectPtr<AActor>& PreviewActor)
	{
		if(PreviewActor)
		{
			TInlineComponentArray<UPrimitiveComponent*> PrimComponents;
			PreviewActor->GetComponents(PrimComponents);

			for(UPrimitiveComponent* PrimComponent : PrimComponents)
			{
				if(const USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(PrimComponent))
				{
					if(UAnimSingleNodeInstance* AnimNodeInstance = SkeletalMeshComponent->GetSingleNodeInstance())
					{
						AnimNodeInstance->SetPlaying(false);
					}
				}
			}
		}
	}
	
	void StepSkeletonAnimation(const TObjectPtr<AActor>& PreviewActor)
	{
		if(PreviewActor)
		{
			TInlineComponentArray<UPrimitiveComponent*> PrimComponents;
			PreviewActor->GetComponents(PrimComponents);

			for(UPrimitiveComponent* PrimComponent : PrimComponents)
			{
				if(const USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(PrimComponent))
				{
					if(UAnimSingleNodeInstance* AnimNodeInstance = SkeletalMeshComponent->GetSingleNodeInstance())
					{
						AnimNodeInstance->SetPlaying(false);
						AnimNodeInstance->StepForward();
					}
				}
			}
		}
	}
	
}

#undef LOCTEXT_NAMESPACE
