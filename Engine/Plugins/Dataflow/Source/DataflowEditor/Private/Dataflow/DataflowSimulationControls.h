// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Dataflow/DataflowObjectInterface.h"
#include "Dataflow/DataflowSimulationContext.h"

class UDataflow;
class UChaosCacheCollection;
class AChaosCacheManager;
class UDataflowBaseContent;
struct FInstancedPropertyBag;

namespace UE::Dataflow
{
	/** Check if the simulation cache nodes have changed to trigger a reset */
	bool ShouldResetWorld(const TObjectPtr<UDataflow>& SimulationGraph,
		const TObjectPtr<UWorld>& SimulationWorld, UE::Dataflow::FTimestamp& LastTimeStamp);

	/** Spawn an actor given a class type and attach it to the cache manager */
	TObjectPtr<AActor> SpawnSimulatedActor(const TSubclassOf<AActor>& ActorClass,
		const TObjectPtr<AChaosCacheManager>& CacheManager, const TObjectPtr<UChaosCacheCollection>& CacheCollection,
		const bool bIsRecording, const TObjectPtr<UDataflowBaseContent>& DataflowContent, const FTransform& ActorTransform,
		const FInstancedPropertyBag& BlueprintVariables);

	UE_DEPRECATED(5.8, "Use BlueprintVariables version instead.")
	TObjectPtr<AActor> SpawnSimulatedActor(const TSubclassOf<AActor>& ActorClass,
		const TObjectPtr<AChaosCacheManager>& CacheManager, const TObjectPtr<UChaosCacheCollection>& CacheCollection,
		const bool bIsRecording, const TObjectPtr<UDataflowBaseContent>& DataflowContent, const FTransform& ActorTransform);

	/** 
	* Setup the skelmesh animations to be used in the scene/generator 
	* return true if the animatiuon was properly set 
	*/
	bool SetupSkeletonAnimation(const TObjectPtr<AActor>& PreviewActor, const bool bSkeletalMeshVisibility = true);

	/** Update the skelmesh animation at some point in time (GT) */
	void UpdateSkeletonAnimation(const TObjectPtr<AActor>& PreviewActor, const float SimulationTime);

	/** Compute the skelmesh animation at some point in time (PT) */
	void ComputeSkeletonAnimation(const TObjectPtr<AActor>& PreviewActor, const float SimulationTime);

	/** Start the skelmesh animation */
	void StartSkeletonAnimation(const TObjectPtr<AActor>& PreviewActor);

	/** Pause the skelmesh animation */
	void PauseSkeletonAnimation(const TObjectPtr<AActor>& PreviewActor);
	
	/** Step the skelmesh animation */
	void StepSkeletonAnimation(const TObjectPtr<AActor>& PreviewActor);
};


