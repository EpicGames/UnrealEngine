// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowSimulationGenerator.h"

#include "Components/PrimitiveComponent.h"
#include "Dataflow/DataflowSimulationManager.h"
#include "Dataflow/DataflowSimulationControls.h"
#include "Dataflow/DataflowSimulationUtils.h"
#include "Chaos/CacheManagerActor.h"
#include "Chaos/CacheCollection.h"
#include "Engine/World.h"
#include "Engine/Engine.h"
#include "Misc/AsyncTaskNotification.h" 

DEFINE_LOG_CATEGORY(LogDataflowSimulationGenerator);

#define LOCTEXT_NAMESPACE "DataflowSimulationGenerator"

namespace UE::Dataflow
{
void FDataflowSimulationTask::DoWork()
{
	const int32 NumFrames = (MaxTime-MinTime+ UE_KINDA_SMALL_NUMBER) / DeltaTime;

	for (int32 FrameIndex = 0; FrameIndex < NumFrames; ++FrameIndex)
	{
		if (!TaskManager->bCancelled.load())
		{
			// Compute the simulation time that will be sent to the graph
			const float SimulationTime = MinTime + (FrameIndex+1) * DeltaTime;

			if(bAsyncCaching)
			{
				// Compute all the skelmesh animations at the simulation time
				UE::Dataflow::ComputeSkeletonAnimation(TaskManager->PreviewActor, SimulationTime);
				
				// Background task : run directly the advance simulation data without coming back to the game thread
				UDataflowSimulationManager* DataflowManager = SimulationWorld->GetSubsystem<UDataflowSimulationManager>();

				// Pre advance the proxies
				DataflowManager->ReadSimulationInterfaces(DeltaTime, true);

				// Advance the simulation proxies
                DataflowManager->AdvanceSimulationProxies(DeltaTime, SimulationTime);

				// Post advance the simulation proxies
				DataflowManager->WriteSimulationInterfaces(DeltaTime, true);
			}
			else
			{
				// Update all the skelmesh animations at the simulation time
				UE::Dataflow::UpdateSkeletonAnimation(TaskManager->PreviewActor, SimulationTime);
				
				// Foreground task : Run the world ticking 
				SimulationWorld->Tick(ELevelTick::LEVELTICK_All, DeltaTime);
			}

			// Finish the frame
			TaskManager->SimulationResource->FinishFrame();
		}
		else
		{
			break;
		}
	}
}

bool FDataflowTaskManager::AllocateSimulationResource(const FVector2f& TimeRange, const int32 FrameRate,
		const TObjectPtr<UChaosCacheCollection>& CacheAsset, const TSubclassOf<AActor>& ActorClass,
		const TObjectPtr<UDataflowBaseContent>& DataflowContent, const FTransform& BlueprintTransform, 
		const FInstancedPropertyBag& BlueprintVariables, const bool bSkeletalMeshVisibility, float DeltaTime)
{
	SimulationWorld = UWorld::CreateWorld(EWorldType::Editor, false);
	SimulationWorld->bPostTickComponentUpdate = false;

	FWorldContext& WorldContext = GEngine->CreateNewWorldContext( SimulationWorld->WorldType );
	WorldContext.SetCurrentWorld(SimulationWorld);
	
	CacheManager = SimulationWorld->SpawnActor<AChaosCacheManager>();
	UDataflowSimulationManager* DataflowManager = SimulationWorld->GetSubsystem<UDataflowSimulationManager>();

	PreviewActor = UE::Dataflow::SpawnSimulatedActor(ActorClass, CacheManager, CacheAsset, true, DataflowContent, BlueprintTransform, BlueprintVariables);
	UE::Dataflow::SetupSkeletonAnimation(PreviewActor, bSkeletalMeshVisibility);

	// Set simulation restart time
	if (SimulationTask->GetTask().bRestartSimulation)
	{
		CacheManager->SetRestartSimulation(SimulationTask->GetTask().bRestartSimulation);
		CacheManager->SetRestartTimeRange(TimeRange[0], TimeRange[1]);
		// Read restart positions
		CacheManager->ReadRestartData();
		DataflowManager->ReadRestartData();
	}
	CacheManager->SetObservedComponentProperties(CacheManager->CacheMode);
	// Init the cache manager
	CacheManager->BeginEvaluate();

	SimulationResource = MakeShared<FDataflowSimulationResource>();
	SimulationResource->NumSimulatedFrames = &NumSimulatedFrames;
	SimulationResource->RecentDateTimeTicks = &RecentDateTimeTicks;
	SimulationResource->bCancelled = &bCancelled;
	
	NumFrames = FMath::Floor((TimeRange[1] - TimeRange[0] + UE_KINDA_SMALL_NUMBER) * FrameRate);
	if(SimulationTask.IsValid())
	{
		SimulationTask->GetTask().SimulationWorld = SimulationWorld;
		SimulationTask->GetTask().MinTime = TimeRange[0];
		SimulationTask->GetTask().MaxTime = TimeRange[1];
		SimulationTask->GetTask().DeltaTime = DeltaTime;

		if(SimulationTask->GetTask().bAsyncCaching)
		{
			// Update all the skelmesh animations at the simulation time
			UE::Dataflow::UpdateSkeletonAnimation(PreviewActor, SimulationTask->GetTask().MinTime);
			
			// Foreground task : Run the world ticking 
			// DeltaSeconds = 0 because this is the preroll, i.e., frame 0 simulation
			SimulationWorld->Tick(ELevelTick::LEVELTICK_All, 0 /*DeltaSeconds*/);

			// Init simulation proxies from interface
			DataflowManager->InitSimulationInterfaces();
		}
		RecentDateTimeTicks.store(FDateTime::UtcNow().GetTicks());
		DataflowManager->SetSimulationEnabled(!SimulationTask->GetTask().bAsyncCaching);
	}

	if (PreviewActor)
	{
		// Prevent the component from having a rendering proxy that can cause unecessary render state invalidation 
		// and potential issue when calling those from a worker thread
		PreviewActor->ForEachComponent<UPrimitiveComponent>(true, [](UPrimitiveComponent* Component)
			{
				Component->DestroyRenderState_Concurrent();
			});
	}
	return true;
}

// UE_DEPRECATED(5.8, "Use BlueprintVariables version instead.")
bool FDataflowTaskManager::AllocateSimulationResource(const FVector2f& TimeRange, const int32 FrameRate,
	const TObjectPtr<UChaosCacheCollection>& CacheAsset, const TSubclassOf<AActor>& ActorClass,
	const TObjectPtr<UDataflowBaseContent>& DataflowContent, const FTransform& BlueprintTransform,
	const bool bSkeletalMeshVisibility, float DeltaTime)
{
	return AllocateSimulationResource(TimeRange, FrameRate,
		CacheAsset, ActorClass,
		DataflowContent, BlueprintTransform,
		FInstancedPropertyBag(), bSkeletalMeshVisibility, DeltaTime);
}

void FDataflowTaskManager::FreeSimulationResource()
{
	if (SimulationTask.IsValid())
	{
		SimulationTask->EnsureCompletion();

		if(SimulationTask->GetTask().bAsyncCaching)
		{
			UDataflowSimulationManager* DataflowManager = SimulationWorld->GetSubsystem<UDataflowSimulationManager>();
			DataflowManager->ResetSimulationInterfaces();
		}
	}

	if(CacheManager)
	{
		// Writes data into caches
		CacheManager->EndEvaluate();

		//  Clear the observed components
        CacheManager->ClearObservedComponents();
		SimulationWorld->DestroyActor(CacheManager);
	}
	
	SimulationResource.Reset();

	GEngine->DestroyWorldContext(SimulationWorld);
	SimulationWorld->DestroyWorld(false);
}

void FDataflowTaskManager::CancelSimulationGeneration()
{
	bCancelled.store(true);
	SimulationTask->TryAbandonTask();
}

FDataflowSimulationGenerator::FDataflowSimulationGenerator()
{}

FDataflowSimulationGenerator::~FDataflowSimulationGenerator()
{
	if (TaskManager != nullptr)
	{
		TaskManager->FreeSimulationResource();
	}
}

void FDataflowSimulationGenerator::Tick(float DeltaTime)
{
	if (PendingAction == EDataflowGeneratorActions::StartGenerate)
	{
		StartGenerateSimulation();
	}
	else if (PendingAction == EDataflowGeneratorActions::TickGenerate)
	{
		TickGenerateSimulation();
	}
}

TStatId FDataflowSimulationGenerator::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FDataflowSimulationGenerator, STATGROUP_Tickables);
}

void FDataflowSimulationGenerator::StartGenerateSimulation()
{
	check(PendingAction == EDataflowGeneratorActions::StartGenerate);
	
	if (TaskManager != nullptr)
	{
		UE_LOGF(LogDataflowSimulationGenerator, Error, "Previous generation is still running.");
		PendingAction = EDataflowGeneratorActions::NoAction;
		return;
	}
	TaskManager = TSharedPtr<FDataflowTaskManager>(new FDataflowTaskManager);

	TaskManager->SimulationTask = MakeUnique<FAsyncTask<FDataflowSimulationTask>>();
	TaskManager->SimulationTask->GetTask().TaskManager = TaskManager;
	TaskManager->SimulationTask->GetTask().bAsyncCaching = CacheParams.bAsyncCaching;
	TaskManager->SimulationTask->GetTask().bRestartSimulation = CacheParams.bRestartSimulation;
	FVector2f TimeRange = CacheParams.bRestartSimulation ? CacheParams.RestartTimeRange : CacheParams.TimeRange;
	TaskManager->StartTime = FDateTime::UtcNow();
	TaskManager->AllocateSimulationResource(TimeRange, CacheParams.FrameRate,
		CacheAsset, BlueprintClass,
		DataflowContent, BlueprintTransform,
		BlueprintVariables, bSkeletalMeshVisibility, GeneratorDeltaTime);

	if(CacheParams.bAsyncCaching)
	{
		TaskManager->SimulationTask->StartBackgroundTask();
	}
	else
	{
		TaskManager->SimulationTask->StartSynchronousTask();
	}

	FAsyncTaskNotificationConfig NotificationConfig;
	NotificationConfig.TitleText = LOCTEXT("SimulateDataflow", "Simulating Dataflow Content");
	NotificationConfig.ProgressText = FText::FromString(TEXT("0%"));
	NotificationConfig.bCanCancel = true;
	NotificationConfig.bKeepOpenOnSuccess = true;
	NotificationConfig.bKeepOpenOnFailure = true;
	TaskManager->AsyncNotification = MakeUnique<FAsyncTaskNotification>(NotificationConfig);
	TaskManager->LastUpdateTime = TaskManager->StartTime;

	PendingAction = EDataflowGeneratorActions::TickGenerate;
}

void FDataflowSimulationGenerator::TickGenerateSimulation()
{
	check(PendingAction == EDataflowGeneratorActions::TickGenerate && TaskManager != nullptr);
		
	bool bFinished = false;
	const bool bCancelled = TaskManager->AsyncNotification->GetPromptAction() == EAsyncTaskNotificationPromptAction::Cancel;
	if (TaskManager->SimulationTask->IsDone())
	{
		bFinished = true;
	}
	else if (bCancelled)
	{
		TaskManager->CancelSimulationGeneration();
		bFinished = true;
	}
		
	if (!bFinished)
	{
		const FDateTime CurrentTime = FDateTime::UtcNow();
		const double SinceLastUpdate = (CurrentTime - TaskManager->LastUpdateTime).GetTotalSeconds();
		if (SinceLastUpdate < 0.2)
		{
			return;
		}
		
		const int32 NumSimulatedFrames = TaskManager->NumSimulatedFrames.load() + 1; //Counting the preroll
		const int32 NumTotalFrames = TaskManager->NumFrames + 1;
		const FDateTime RecentFrameTime = FDateTime(TaskManager->RecentDateTimeTicks.load());
		const double AverageFrameTime = (RecentFrameTime - TaskManager->StartTime).GetTotalSeconds() / NumSimulatedFrames;
		const double EstimatedTime = FMath::Max(0, AverageFrameTime * (NumTotalFrames - NumSimulatedFrames) - (CurrentTime - RecentFrameTime).GetTotalSeconds());
		FString ProgressMessage = FString::Printf(TEXT("Finished %d/%d, %.1f%%\nAverage time: %.1f seconds/frame\nEstimated finish time: %s"),
			NumSimulatedFrames, NumTotalFrames, 100.0 * NumSimulatedFrames / NumTotalFrames, AverageFrameTime, *FText::AsTimespan(FTimespan::FromSeconds(EstimatedTime)).ToString());
		TaskManager->AsyncNotification->SetProgressText(FText::FromString(ProgressMessage));
		TaskManager->LastUpdateTime = CurrentTime;
	}
	else
	{
		FreeTaskResource(bCancelled);
		PendingAction = EDataflowGeneratorActions::NoAction;
	}
}
	
void FDataflowSimulationGenerator::SetCacheParams(const FDataflowPreviewCacheParams& InCacheParams)
{
	CacheParams = InCacheParams;
}

void FDataflowSimulationGenerator::SetCacheAsset(const TObjectPtr<UChaosCacheCollection>& InCacheAsset) 
{
	CacheAsset = InCacheAsset;
}

void FDataflowSimulationGenerator::SetBlueprintClass(const TSubclassOf<AActor>& InBlueprintClass)
{
	BlueprintClass = InBlueprintClass;
}

void FDataflowSimulationGenerator::SetBlueprintTransform(const FTransform& InBlueprintTransform)
{
	BlueprintTransform = InBlueprintTransform;
}

void FDataflowSimulationGenerator::SetBlueprintVariables(const FInstancedPropertyBag& InBlueprintVariables)
{
	BlueprintVariables = InBlueprintVariables;
}

void FDataflowSimulationGenerator::SetDataflowContent(const TObjectPtr<UDataflowBaseContent>& InDataflowContent)
{
	DataflowContent = InDataflowContent;
}

void FDataflowSimulationGenerator::SetSkeletalMeshVisibility(const bool bInSkeletalMeshVisibility)
{
	bSkeletalMeshVisibility = bInSkeletalMeshVisibility;
}

void FDataflowSimulationGenerator::RequestGeneratorAction(EDataflowGeneratorActions ActionType)
{
	if (PendingAction != EDataflowGeneratorActions::NoAction)
	{
		return;
	}
	PendingAction = ActionType;
}

void FDataflowSimulationGenerator::FreeTaskResource(bool bCancelled)
{
	TaskManager->AsyncNotification->SetProgressText(LOCTEXT("Finishing", "Finishing, please wait"));
	TaskManager->FreeSimulationResource();
	const FDateTime CurrentTime = FDateTime::UtcNow();
	UE_LOGF(LogDataflowSimulationGenerator, Log, "Simulation finished in %f seconds", (CurrentTime - TaskManager->StartTime).GetTotalSeconds());

	{
		// TODO : save the result to the chaos cache collection
	}
	if (bCancelled)
	{
		TaskManager->AsyncNotification->SetProgressText(LOCTEXT("Cancelled", "Cancelled"));
		TaskManager->AsyncNotification->SetComplete(false);
	}
	else
	{
		TaskManager->AsyncNotification->SetProgressText(LOCTEXT("Finished", "Finished"));
		TaskManager->AsyncNotification->SetComplete(true);
	}
	TaskManager.Reset();
	CacheAsset->MarkPackageDirty();
}
};

#undef LOCTEXT_NAMESPACE
