// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowSimulationGeometryCache.h"

#include "Chaos/Vector.h"
#include "GeometryCache.h"
#include "Logging/LogMacros.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "Rendering/SkeletalMeshModel.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/StaticMesh.h"

#if WITH_EDITOR 
#include "FileHelpers.h"
#include "GeometryCacheConstantTopologyWriter.h"
#endif

DEFINE_LOG_CATEGORY(LogDataflowSimulationGeometryCache);

#define LOCTEXT_NAMESPACE "DataflowSimulationGeometryCache"
	
namespace UE::DataflowSimulationGeometryCache
{
	int32 GetNumVertices(const FSkeletalMeshLODRenderData& LODData)
	{
		int32 NumVertices = 0;
		for (const FSkelMeshRenderSection& Section : LODData.RenderSections)
		{
			NumVertices += Section.NumVertices;
		}
		return NumVertices;
	}

	TArrayView<TArray<FVector3f>> ShrinkToValidFrames(TArrayView<TArray<FVector3f>> Positions, int32 NumVertices)
	{
		int32 NumValidFrames = 0;
		for (const TArray<FVector3f>& Frame : Positions)
		{
			if (Frame.Num() != NumVertices)
			{
				break;
			}
			++NumValidFrames;
		}
		return TArrayView<TArray<FVector3f>>(Positions.GetData(), NumValidFrames);
	}

	void SaveGeometryCache(UGeometryCache& GeometryCache, float FrameRate, const USkinnedAsset& Asset, TConstArrayView<uint32> ImportedVertexNumbers, TArrayView<TArray<FVector3f>> PositionsToMoveFrom)
	{
#if WITH_EDITOR 
		const FSkeletalMeshRenderData* RenderData = Asset.GetResourceForRendering();
		constexpr int32 LODIndex = 0;
		if (!RenderData || !RenderData->LODRenderData.IsValidIndex(LODIndex))
		{
			return;
		}
		const FSkeletalMeshLODRenderData& LODData = RenderData->LODRenderData[LODIndex];
		const int32 NumVertices = GetNumVertices(LODData);
		PositionsToMoveFrom = ShrinkToValidFrames(PositionsToMoveFrom, NumVertices);

		using UE::GeometryCacheHelpers::FGeometryCacheConstantTopologyWriter;
		using UE::GeometryCacheHelpers::AddTrackWriterFromSkinnedAsset;
		FGeometryCacheConstantTopologyWriter::FConfig Config; Config.FPS = FrameRate;
		using FTrackWriter = FGeometryCacheConstantTopologyWriter::FTrackWriter;
		FGeometryCacheConstantTopologyWriter Writer(GeometryCache, Config);
		const int32 Index = AddTrackWriterFromSkinnedAsset(Writer, Asset);
		if (Index == INDEX_NONE)
		{
			return;
		}
		FTrackWriter& TrackWriter = Writer.GetTrackWriter(Index);
		TrackWriter.ImportedVertexNumbers = ImportedVertexNumbers;
		TrackWriter.WriteAndClose(PositionsToMoveFrom);
#endif
	}

	void SaveGeometryCache(UGeometryCache& GeometryCache, float FrameRate, const UStaticMesh& StaticMesh, TArrayView<TArray<FVector3f>> PositionsToMoveFrom)
	{
#if WITH_EDITOR 
		using UE::GeometryCacheHelpers::FGeometryCacheConstantTopologyWriter;
		using UE::GeometryCacheHelpers::AddTrackWriterFromStaticMesh;
		FGeometryCacheConstantTopologyWriter::FConfig Config; Config.FPS = FrameRate;
		using FTrackWriter = FGeometryCacheConstantTopologyWriter::FTrackWriter;
		FGeometryCacheConstantTopologyWriter Writer(GeometryCache, Config);
		const int32 Index = AddTrackWriterFromStaticMesh(Writer, StaticMesh);
		if (Index == INDEX_NONE)
		{
			return;
		}
		FTrackWriter& TrackWriter = Writer.GetTrackWriter(Index);
		TrackWriter.WriteAndClose(PositionsToMoveFrom);
#endif
	}

	void SavePackage(UObject& Object)
	{
#if WITH_EDITOR 
		TArray<UPackage*> PackagesToSave = { Object.GetOutermost() };
		constexpr bool bCheckDirty = false;
		constexpr bool bPromptToSave = false;
		FEditorFileUtils::PromptForCheckoutAndSave(PackagesToSave, bCheckDirty, bPromptToSave);
#endif
	}
};

#undef LOCTEXT_NAMESPACE
