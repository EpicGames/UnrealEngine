// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowSimulationPanel.h"
#include "Dataflow/DataflowSimulationScene.h"
#include "Widgets/SBoxPanel.h"
#include "SScrubControlPanel.h"
#include "Animation/AnimSingleNodeInstance.h"
#include "Animation/AnimSequence.h"
#include "Animation/BlendSpace.h"
#include "Chaos/CacheManagerActor.h"
#include "Dataflow/DataflowContent.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SBox.h"

#define LOCTEXT_NAMESPACE "DataflowSimulationScrubPanel"

void SDataflowSimulationPanel::Construct( const SDataflowSimulationPanel::FArguments& InArgs, TWeakPtr<FDataflowSimulationScene> InPreviewScene)
{
	SimulationScene = InPreviewScene;

	// Skip adding the the Loop button so we can add our own
	TArray<FTransportControlWidget> TransportControlWidgets;
	for (const ETransportControlWidgetType Type : TEnumRange<ETransportControlWidgetType>())
	{
		if ((Type != ETransportControlWidgetType::Custom) && (Type != ETransportControlWidgetType::Loop))
		{
			TransportControlWidgets.Add(FTransportControlWidget(Type));
		}
	}
	const FTransportControlWidget NewWidget(FOnMakeTransportWidget::CreateSP(this, &SDataflowSimulationPanel::OnCreatePreviewPlaybackModeWidget));
	TransportControlWidgets.Add(NewWidget);
	FrameIndexWidget = SNew(SEditableTextBox)
		.OnTextCommitted(this, &SDataflowSimulationPanel::SetFrameIndex)
		.IsEnabled(true);
	FrameIndexWidget->SetText(FText::AsNumber(0));
	this->ChildSlot
	[
		SNew(SHorizontalBox)
		.AddMetaData<FTagMetaData>(TEXT("DataflowSimulationScrub.Scrub"))
		+SHorizontalBox::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Center)
		.AutoWidth()
		.Padding(0,0,8,0)
		[	
			SNew(SBox)
			.WidthOverride(60.f)
			[
				FrameIndexWidget.ToSharedRef()
			]
		]
		+SHorizontalBox::Slot()
		.HAlign(HAlign_Fill) 
		.VAlign(VAlign_Center)
		.FillWidth(1)
		.Padding(0.0f)
		[
			SAssignNew(ScrubControlPanel, SScrubControlPanel)
			.IsEnabled(true)
			.Value(this, &SDataflowSimulationPanel::GetScrubValue)
			.NumOfKeys(this, &SDataflowSimulationPanel::GetNumberOfKeys)
			.SequenceLength(this, &SDataflowSimulationPanel::GetSequenceLength)
			.DisplayDrag(this, &SDataflowSimulationPanel::GetDisplayDrag)
			.OnValueChanged(this, &SDataflowSimulationPanel::OnValueChanged)
			.OnBeginSliderMovement(this, &SDataflowSimulationPanel::OnBeginSliderMovement)
			.OnClickedRecord(this, &SDataflowSimulationPanel::OnClick_Record)
			.OnClickedForwardPlay(this, &SDataflowSimulationPanel::OnClick_Forward)
			.OnClickedForwardStep(this, &SDataflowSimulationPanel::OnClick_Forward_Step)
			.OnClickedForwardEnd(this, &SDataflowSimulationPanel::OnClick_Forward_End)
			.OnClickedBackwardPlay(this, &SDataflowSimulationPanel::OnClick_Backward)
			.OnClickedBackwardStep(this, &SDataflowSimulationPanel::OnClick_Backward_Step)
			.OnClickedBackwardEnd(this, &SDataflowSimulationPanel::OnClick_Backward_End)
			.OnTickPlayback(this, &SDataflowSimulationPanel::OnTickPlayback)
			.OnGetPlaybackMode(this, &SDataflowSimulationPanel::GetPlaybackMode)
			.ViewInputMin(InArgs._ViewInputMin)
			.ViewInputMax(InArgs._ViewInputMax)
			.bDisplayAnimScrubBarEditing(false)
			.bAllowZoom(false)
			.IsRealtimeStreamingMode(false)
			.TransportControlWidgetsToCreate(TransportControlWidgets)
		]
	];
}

TSharedRef<SWidget> SDataflowSimulationPanel::OnCreatePreviewPlaybackModeWidget()
{
	PreviewPlaybackModeButton = SNew(SButton)
		.OnClicked(this, &SDataflowSimulationPanel::OnClick_PreviewPlaybackMode)
		.ButtonStyle( FAppStyle::Get(), "Animation.PlayControlsButton" )
		.IsFocusable(false)
		.ToolTipText_Lambda([&]()
		{
			if (PreviewPlaybackMode == EDataflowPlaybackMode::Default)
			{
				return LOCTEXT("PlaybackModeDefaultTooltip", "Linear playback");
			}
			else if (PreviewPlaybackMode == EDataflowPlaybackMode::Looping)
			{
				return LOCTEXT("PlaybackModeLoopingTooltip", "Looping playback");
			}
			else
			{
				return LOCTEXT("PlaybackModePingPongTooltip", "Ping pong playback");
			}
		})
		.ContentPadding(0.0f);

	TWeakPtr<SButton> WeakButton = PreviewPlaybackModeButton;

	PreviewPlaybackModeButton->SetContent(SNew(SImage)
		.ColorAndOpacity(FSlateColor::UseSubduedForeground())
		.Image_Lambda([&, WeakButton]()
		{
			if (PreviewPlaybackMode == EDataflowPlaybackMode::Default)
			{
				return FAppStyle::Get().GetBrush("Animation.Loop.Disabled");
			}
			else if (PreviewPlaybackMode == EDataflowPlaybackMode::Looping)
			{
				return FAppStyle::Get().GetBrush("Animation.Loop.Enabled");
			}
			else
			{
				return FAppStyle::Get().GetBrush("Animation.Loop.SelectionRange");		// TODO: Replace with a back and forth type icon
			}
		})
	);

	TSharedRef<SHorizontalBox> PreviewPlaybackModeBox = SNew(SHorizontalBox);
	PreviewPlaybackModeBox->AddSlot()
	.AutoWidth()
	[
		PreviewPlaybackModeButton.ToSharedRef()
	];

	return PreviewPlaybackModeBox;
}

FReply SDataflowSimulationPanel::OnClick_Forward_Step()
{
	if (const TSharedPtr<FDataflowSimulationScene> PreviewScene = SimulationScene.Pin())
	{
		UpdateSimulationTimeFromScrubValue(GetScrubValue() + PreviewScene->GetDeltaTime());
	}

	return FReply::Handled();
}

FReply SDataflowSimulationPanel::OnClick_Forward_End()
{
	UpdateSimulationTimeFromScrubValue(GetSequenceLength());

	return FReply::Handled();
}

FReply SDataflowSimulationPanel::OnClick_Backward_Step()
{
	if (const TSharedPtr<FDataflowSimulationScene> PreviewScene = SimulationScene.Pin())
	{
		UpdateSimulationTimeFromScrubValue(GetScrubValue() - PreviewScene->GetDeltaTime());
	}

	return FReply::Handled();
}

FReply SDataflowSimulationPanel::OnClick_Backward_End()
{
	if (const TSharedPtr<FDataflowSimulationScene> PreviewScene = SimulationScene.Pin())
	{
		UpdateSimulationTimeFromScrubValue(0.f);
	}

	return FReply::Handled();
}

FReply SDataflowSimulationPanel::OnClick_Record()
{
	if (const TSharedPtr<FDataflowSimulationScene> PreviewScene = SimulationScene.Pin())
	{
		PreviewScene->RecordSimulationCache();
	}
	return FReply::Handled();
}

FReply SDataflowSimulationPanel::OnClick_Forward()
{
	if(PlaybackMode == EPlaybackMode::PlayingForward)
	{
		PlaybackMode = EPlaybackMode::Stopped;
	}
	else
	{
		PlaybackMode = EPlaybackMode::PlayingForward;
	}

	return FReply::Handled();
}

FReply SDataflowSimulationPanel::OnClick_Backward()
{
	if(PlaybackMode == EPlaybackMode::PlayingReverse)
	{
		PlaybackMode = EPlaybackMode::Stopped;
	}
	else
	{
		PlaybackMode = EPlaybackMode::PlayingReverse;
	}

	return FReply::Handled();
}

FReply SDataflowSimulationPanel::OnClick_PreviewPlaybackMode()
{
	if (PreviewPlaybackMode == EDataflowPlaybackMode::Default)
	{
		PreviewPlaybackMode = EDataflowPlaybackMode::Looping;
	}
	else if (PreviewPlaybackMode == EDataflowPlaybackMode::Looping)
	{
		PreviewPlaybackMode = EDataflowPlaybackMode::PingPong;
	}
	else
	{
		PreviewPlaybackMode = EDataflowPlaybackMode::Default;
	}

	return FReply::Handled();
}

void SDataflowSimulationPanel::OnTickPlayback(double InCurrentTime, float InDeltaTime)
{
	if (const TSharedPtr<FDataflowSimulationScene> PreviewScene = SimulationScene.Pin())
	{
		const float SequenceLength = GetSequenceLength();
		const float ScrubValue = (PlaybackMode == EPlaybackMode::PlayingForward) ?
			GetScrubValue() + InDeltaTime : GetScrubValue() - InDeltaTime;
		
		if(PreviewPlaybackMode == EDataflowPlaybackMode::Looping)
		{
			UpdateSimulationTimeFromScrubValue(ScrubValue - SequenceLength * FMath::Floor(ScrubValue / SequenceLength), false);
		}
		else
		{
			if(PreviewPlaybackMode == EDataflowPlaybackMode::PingPong)
			{
				if((PlaybackMode == EPlaybackMode::PlayingForward) && (ScrubValue >= SequenceLength))
				{
					PlaybackMode = EPlaybackMode::PlayingReverse;
				}

				if((PlaybackMode == EPlaybackMode::PlayingReverse) && (ScrubValue <= 0))
				{
					PlaybackMode = EPlaybackMode::PlayingForward;
				}
			}
			UpdateSimulationTimeFromScrubValue(FMath::Clamp(ScrubValue, 0.f, SequenceLength), false);
		}
	}
}

EPlaybackMode::Type SDataflowSimulationPanel::GetPlaybackMode() const
{
	return PlaybackMode;
}

void SDataflowSimulationPanel::UpdateSimulationTimeFromScrubValue(float ScrubValue, const bool bRoundedFrame)
{
	if (const TSharedPtr<FDataflowSimulationScene> PreviewScene = SimulationScene.Pin())
	{
		const int32 TotalFrameRate = PreviewScene->GetFrameRate() * PreviewScene->GetSubframeRate();
		const float RoundedFrameTime = bRoundedFrame ? FMath::RoundToFloat(ScrubValue * TotalFrameRate) / TotalFrameRate : ScrubValue;

		PreviewScene->SimulationTime = RoundedFrameTime + PreviewScene->GetTimeRange()[0];
		FrameIndexWidget->SetText(FText::AsNumber(RoundedFrameTime * PreviewScene->GetFrameRate()));
	}
}

void SDataflowSimulationPanel::OnValueChanged(float NewValue) 
{
	UpdateSimulationTimeFromScrubValue(NewValue);
}

void SDataflowSimulationPanel::SetFrameIndex(const FText& InNewText, ETextCommit::Type InCommitType)
{
	if (InNewText.IsNumeric())
	{
		float FrameIndex = 0.f;
		LexFromString(FrameIndex, *InNewText.ToString());
		if (const TSharedPtr<FDataflowSimulationScene> PreviewScene = SimulationScene.Pin())
		{
			UpdateSimulationTimeFromScrubValue(FrameIndex / float(PreviewScene->GetFrameRate()));
		}
	}
}

void SDataflowSimulationPanel::OnBeginSliderMovement()
{}

uint32 SDataflowSimulationPanel::GetNumberOfKeys() const
{
	if (const TSharedPtr<const FDataflowSimulationScene> PreviewScene = SimulationScene.Pin())
	{
		return PreviewScene->GetNumFrames();
	}

	return 1;
}

float SDataflowSimulationPanel::GetSequenceLength() const
{
	if (const TSharedPtr<const FDataflowSimulationScene> PreviewScene = SimulationScene.Pin())
	{
		return PreviewScene->GetTimeRange()[1]-PreviewScene->GetTimeRange()[0];
	}
	return 0.0f;
}

float SDataflowSimulationPanel::GetScrubValue() const
{
	if (const TSharedPtr<const FDataflowSimulationScene> PreviewScene = SimulationScene.Pin())
	{
		return PreviewScene->SimulationTime-PreviewScene->GetTimeRange()[0];
	}

	return 0.0f;
}

bool SDataflowSimulationPanel::GetDisplayDrag() const
{
	if (const TSharedPtr<const FDataflowSimulationScene> PreviewScene = SimulationScene.Pin())
	{
		return true;
	}
	return false;
}

#undef LOCTEXT_NAMESPACE
