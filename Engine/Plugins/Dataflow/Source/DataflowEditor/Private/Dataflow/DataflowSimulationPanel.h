// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Dataflow/DataflowSimulationScene.h"
#include "Widgets/SCompoundWidget.h"
#include "ITransportControl.h"  // EPlaybackMode::Type

class SScrubControlPanel;
class SButton;
class UAnimSingleNodeInstance;
class SEditableTextBox;

/** Dataflow simulation panel to control an animation/simulation */
class SDataflowSimulationPanel : public SCompoundWidget
{
private:

	/** Data flow playback mode */
	enum class EDataflowPlaybackMode : int32
	{
		Default,
		Looping,
		PingPong
	};

	/** View min/max to run the simulation */
	SLATE_BEGIN_ARGS(SDataflowSimulationPanel)	{}
		SLATE_ATTRIBUTE( float, ViewInputMin )
		SLATE_ATTRIBUTE( float, ViewInputMax )
	SLATE_END_ARGS()

	/** Construct the simulation widget */
	void Construct( const FArguments& InArgs, TWeakPtr<FDataflowSimulationScene> InPreviewScene );

	/** Delegate when the playback mode is changed */
	TSharedRef<SWidget> OnCreatePreviewPlaybackModeWidget();

	/** Delegate when the simulation controls are pressed*/
	FReply OnClick_Forward_Step();
	FReply OnClick_Forward_End();
	FReply OnClick_Backward_Step();
	FReply OnClick_Backward_End();
	FReply OnClick_Forward();
	FReply OnClick_Backward();
	FReply OnClick_PreviewPlaybackMode();
	FReply OnClick_Record();

	/** Main simulation delegates */
	void OnTickPlayback(double InCurrentTime, float InDeltaTime);
	void UpdateSimulationTimeFromScrubValue(float ScrubValue, const bool bRoundedFrame = true);
	void OnValueChanged(float NewValue);
	void SetFrameIndex(const FText& InNewText, ETextCommit::Type InCommitType);
	void OnBeginSliderMovement();

	/** Get the playback mode used in the widget */
	EPlaybackMode::Type GetPlaybackMode() const;

	/** Get ethe current scrub value */
	float GetScrubValue() const;

	/** Get the number of keys */
	uint32 GetNumberOfKeys() const;

	/** Get the sequence length */
	float GetSequenceLength() const;

	/** Get the display drag */
	bool GetDisplayDrag() const;

	/** Simulation scene to be used for the widget */
	TWeakPtr<FDataflowSimulationScene> SimulationScene;

	/** Scrub widget defined for the timeline */
	TSharedPtr<SScrubControlPanel> ScrubControlPanel;

	/** Playback mode button */
	TSharedPtr<SButton> PreviewPlaybackModeButton;

	/** Widget showing editable frame index*/
	TSharedPtr<SEditableTextBox> FrameIndexWidget;

	/** Preview playback mode (looping...)*/
	EDataflowPlaybackMode PreviewPlaybackMode = EDataflowPlaybackMode::Looping;

	/** playback mode */
	EPlaybackMode::Type PlaybackMode = EPlaybackMode::Stopped;
};
