// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowSimulationScene.h"
#include "Dataflow/DataflowSimulationManager.h"
#include "Dataflow/DataflowSimulationControls.h"
#include "Dataflow/DataflowSimulationVisualization.h"
#include "Dataflow/DataflowEditor.h"
#include "Dataflow/DataflowInstance.h"
#include "Components/PrimitiveComponent.h"
#include "Chaos/CacheManagerActor.h"
#include "Misc/TransactionObjectEvent.h"
#include "EngineUtils.h"
#include "Animation/AnimSingleNodeInstance.h"
#include "AssetEditorModeManager.h"
#include "AssetViewerSettings.h"
#include "Engine/Selection.h"

#include "Dataflow/Interfaces/DataflowInterfaceGeometryCachable.h"
#include "Dataflow/DataflowSimulationGeometryCache.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Components/BillboardComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "GeometryCache.h"
#include "Dataflow/DataflowElement.h"

#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"


#if WITH_EDITOR
#include "Misc/FileHelper.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowSimulationScene)
#define LOCTEXT_NAMESPACE "FDataflowSimulationScene"

namespace DataflowSimulationScene::Private
{
	void ShowErrorToastMessage(const FText& Text, const FText& SubText, const float Duration)
	{
		FNotificationInfo ToastInfo(Text);
		ToastInfo.ExpireDuration = Duration;
		ToastInfo.bFireAndForget = true;
		ToastInfo.SubText = SubText;
		ToastInfo.bUseLargeFont = true;

		TSharedPtr<SNotificationItem> Notification = FSlateNotificationManager::Get().AddNotification(ToastInfo);
		if (Notification.IsValid())
		{
			Notification->SetCompletionState(SNotificationItem::ECompletionState::CS_Fail);
		}
	}

	UChaosCacheCollection* GetTransientCacheCollection()
	{
		static const FName BaseName = TEXT("TempDataflowCacheCache");

		const UChaosCacheCollection* DefaultCacheAsset = LoadObject<UChaosCacheCollection>(nullptr, TEXT("/Dataflow/CC_DefaultChaosCache.CC_DefaultChaosCache"));
		if (DefaultCacheAsset)
		{
			return Cast<UChaosCacheCollection>(StaticDuplicateObject(DefaultCacheAsset, GetTransientPackage(), BaseName));
		}
		return NewObject<UChaosCacheCollection>(GetTransientPackage(), BaseName);
	}
}
//
// Simulation Scene
//

FDataflowSimulationScene::FDataflowSimulationScene(FPreviewScene::ConstructionValues ConstructionValues, UDataflowEditor* InEditor)
	: FDataflowPreviewSceneBase(ConstructionValues, InEditor, FName("RootActor"))
{
	SceneDescription = NewObject<UDataflowSimulationSceneDescription>();
	SceneDescription->SetSimulationScene(this);

	SimulationGenerator = MakeShared<UE::Dataflow::FDataflowSimulationGenerator>();

	FActorSpawnParameters SpawnParameters;
	SpawnParameters.Name = FName("Simulation Components");
	
	AChaosCacheManager* const CacheManager = GetWorld()->SpawnActor<AChaosCacheManager>(SpawnParameters);
	CacheManager->SetEditorIconVisibility(false);
	RootSceneActor = CacheManager;

	if(GetEditorContent())
	{
#if WITH_EDITORONLY_DATA
		if(const UDataflow* DataflowAsset = GetEditorContent()->GetDataflowAsset())
		{
			SceneDescription->CacheParams = DataflowAsset->PreviewCacheParams;
			SceneDescription->CacheAsset = Cast<UChaosCacheCollection>(DataflowAsset->PreviewCacheAsset.LoadSynchronous());
			if (SceneDescription->CacheAsset == nullptr)
			{
				SceneDescription->CacheAsset = DataflowSimulationScene::Private::GetTransientCacheCollection();
			}
			SceneDescription->SetBlueprintClass(DataflowAsset->PreviewBlueprintClass);
			SceneDescription->BlueprintTransform = DataflowAsset->PreviewBlueprintTransform; 
			SceneDescription->GeometryCacheAsset = Cast<UGeometryCache>(DataflowAsset->PreviewGeometryCacheAsset.LoadSynchronous());
			SceneDescription->EmbeddedSkeletalMesh = Cast<USkeletalMesh>(DataflowAsset->PreviewEmbeddedSkeletalMesh.LoadSynchronous());
			SceneDescription->EmbeddedStaticMesh = Cast<UStaticMesh>(DataflowAsset->PreviewEmbeddedStaticMesh.LoadSynchronous());
		}
		if(SceneDescription->GetBlueprintClass() == nullptr)
		{
			SceneDescription->SetBlueprintClass(GetEditorContent()->GetPreviewClass());
		}
#endif
	}

#if WITH_EDITOR
	OnObjectsReinstancedHandle = FCoreUObjectDelegates::OnObjectsReinstanced.AddRaw(this, &FDataflowSimulationScene::OnObjectsReinstanced);
	OnObjectPropertyChangedHandle = FCoreUObjectDelegates::OnObjectPropertyChanged.AddRaw(this, &FDataflowSimulationScene::OnEditorContentPropertyChanged);
#endif

	CreateSimulationScene();

	// Sync CacheParams.TimeRange from the animation length on initial load.
	// After this, CacheParams.TimeRange is the source of truth and only updates
	// when the user edits it or the animation asset changes.
	UpdateTimelinedataFromAnimations();

	// apply settings
	if (const UDataflowSimulationSettings* Settings = InEditor->FindEditorSettings<UDataflowSimulationSettings>())
	{
		SetSimulationEnabled(Settings->bIsSimulationPlayingByDefault);

		SceneDescription->CacheParams.bEnableAsyncCaching = Settings->bIsAsyncCachingSupported;
		if (Settings->bIsAsyncCachingSupported)
		{
			SceneDescription->CacheParams.bAsyncCaching = Settings->bIsAsyncCachingEnabledByDefault;
		}
		else
		{
			SceneDescription->CacheParams.bAsyncCaching = false;
		}

		SceneDescription->bIsGeometryCacheOutputSupported = Settings->bIsGeometryCacheOutputSupported;
	}
}

void FDataflowSimulationScene::OnObjectsReinstanced(const TMap<UObject*, UObject*>& ObjectsMap)
{
	if(UObject* const* InstancedActor = ObjectsMap.Find(PreviewActor))
	{
		if(*InstancedActor)
		{
			PreviewActor = Cast<AActor>(*InstancedActor);
		}
	}
}

FDataflowSimulationScene::~FDataflowSimulationScene()
{
	ResetSimulationScene();

	// Make sure we clear the simulation scene from the scene description 
	// This may cause a crash accesing a dangling pointer when the preview blueprint is force deleted 
	// (see JIRA UE-351651)
	if (IsValid(SceneDescription))
	{
		SceneDescription->SetSimulationScene(nullptr);
	}

#if WITH_EDITOR
	FCoreUObjectDelegates::OnObjectsReinstanced.Remove(OnObjectsReinstancedHandle);
	FCoreUObjectDelegates::OnObjectPropertyChanged.Remove(OnObjectPropertyChangedHandle);
#endif
}

void FDataflowSimulationScene::OnEditorContentPropertyChanged(UObject* Object, FPropertyChangedEvent& Event)
{
	// AnimationAsset lives on UDataflowSkeletalContent (our editor content), not on the preview actor
	// or its components. Filter for our specific content.
	if (!Object || Object != GetEditorContent())
	{
		return;
	}

	// When the animation asset changes on the dataflow content, update CacheParams.TimeRange and rebuild the scene.
	// Note: Cannot use GET_MEMBER_NAME_CHECKED because AnimationAsset is a protected member of UDataflowSkeletalContent.
	static const FName AnimationAssetPropertyName(TEXT("AnimationAsset")); // Must stay in sync with the UPROPERTY name in UDataflowSkeletalContent.
	if (Event.GetPropertyName() == AnimationAssetPropertyName)
	{
		// Rebuild first so the new PreviewActor has the updated animation,
		// then read the animation length and update CacheParams.TimeRange + timeline
		ResetSimulationScene();
		CreateSimulationScene();
		UpdateTimelinedataFromAnimations();
	}
}

void FDataflowSimulationScene::UnbindSceneSelection()
{
	if(PreviewActor)
	{
		TInlineComponentArray<UPrimitiveComponent*> PrimComponents;
		PreviewActor->GetComponents(PrimComponents);

		for(UPrimitiveComponent* PrimComponent : PrimComponents)
		{
			PrimComponent->SelectionOverrideDelegate.Unbind();
		}
	}
}

void FDataflowSimulationScene::ResetSimulationScene()
{
	// Release any selected components before the PreviewActor is deleted from the scene
	if (const TSharedPtr<FAssetEditorModeManager> ModeManager = GetDataflowModeManager())
	{
		if (USelection* const SelectedComponents = ModeManager->GetSelectedComponents())
		{
			SelectedComponents->DeselectAll();
		}
	}

	// Destroy the spawned root actor
	if(PreviewActor && GetWorld())
	{
		PreviewActor->ForEachComponent<UActorComponent>(true, [this](UActorComponent* ActorComponent)
		{
			RemoveSceneObject(ActorComponent);
		});
		RemoveSceneObject(PreviewActor);
		
		GetWorld()->EditorDestroyActor(PreviewActor, true);
			
		// Since deletion can be delayed, rename to avoid future name collision
		// Call UObject::Rename directly on actor to avoid AActor::Rename which unnecessarily sunregister and re-register components
		PreviewActor->UObject::Rename(nullptr, GetTransientPackage(), REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch);
	}
	RemoveSceneObject(RootSceneActor);

	bPreviewSceneDirty = true;
	
	// Unbind the scene selection
	UnbindSceneSelection();
}

void  FDataflowSimulationScene::SetSimulationLocked(const bool bSimulationLocked, const bool bIsPlaying) 
{ 
	bIsSimulationLocked = bSimulationLocked;
}

void FDataflowSimulationScene::PauseSimulationScene() const
{
	if(!bIsSimulationLocked)
	{
		GetWorld()->GetSubsystem<UDataflowSimulationManager>()->SetSimulationEnabled(false);
		UE::Dataflow::PauseSkeletonAnimation(PreviewActor);
	}
}

void FDataflowSimulationScene::StartSimulationScene() const
{
	if(!bIsSimulationLocked)
	{
		GetWorld()->GetSubsystem<UDataflowSimulationManager>()->SetSimulationEnabled(true);
		UE::Dataflow::StartSkeletonAnimation(PreviewActor);
	}
}

void FDataflowSimulationScene::StepSimulationScene() const
{
	if(!bIsSimulationLocked)
	{
		GetWorld()->GetSubsystem<UDataflowSimulationManager>()->SetSimulationEnabled(true);
		GetWorld()->GetSubsystem<UDataflowSimulationManager>()->SetSimulationStepping(true);
		UE::Dataflow::StepSkeletonAnimation(PreviewActor);
	}
}

bool FDataflowSimulationScene::HasCachingEnabled() const
{
	return (SceneDescription && (SceneDescription->CacheAsset != nullptr));
}

bool FDataflowSimulationScene::IsSimulationEnabled() const
{
	if(!bIsSimulationLocked)
	{
		return GetWorld()->GetSubsystem<UDataflowSimulationManager>()->GetSimulationEnabled();
	}
	return false;
}

void FDataflowSimulationScene::SetSimulationEnabled(bool bEnable) const
{
	GetWorld()->GetSubsystem<UDataflowSimulationManager>()->SetSimulationEnabled(bEnable);
}

void FDataflowSimulationScene::RebuildSimulationScene()
{
	// Unregister components, cache manager, selection...
	ResetSimulationScene();

	// Register components, cache manager, selection...
	CreateSimulationScene();

	// Reset the simulation dirty flag
	if(const TObjectPtr<UDataflowBaseContent>& EditorContent = GetEditorContent())
	{
		EditorContent->SetSimulationDirty(false);
	}

	// Set the locked simulation flag to false
	bIsSimulationLocked = !IsSimulationEnabled();

	// Update simulation visualizations
	for (const TPair<FName, TUniquePtr<UE::Dataflow::IDataflowSimulationVisualization>>& Visualization : UE::Dataflow::FDataflowSimulationVisualizationRegistry::GetInstance().GetVisualizations())
	{
		if (Visualization.Value.IsValid())
		{
			Visualization.Value->SimulationSceneUpdated(this);
		}
	}
}

void FDataflowSimulationScene::BindSceneSelection()
{
	if(PreviewActor)
	{
		TInlineComponentArray<UPrimitiveComponent*> PrimComponents;
		PreviewActor->GetComponents(PrimComponents);
		
		for(UPrimitiveComponent* PrimComponent : PrimComponents)
		{
			PrimComponent->SelectionOverrideDelegate =
				UPrimitiveComponent::FSelectionOverride::CreateRaw(this, &FDataflowPreviewSceneBase::IsComponentSelected);
		}
	}
}

void FDataflowSimulationScene::UpdateTimelinedataFromAnimations()
{
	float MaxAnimLength = 0.0f;
	if (PreviewActor)
	{
		PreviewActor->ForEachComponent<USkeletalMeshComponent>(true, 
			[&MaxAnimLength](USkeletalMeshComponent* SkeletalMeshComponent)
			{
				if (UAnimSingleNodeInstance* AnimInstance = SkeletalMeshComponent->GetSingleNodeInstance())
				{
					MaxAnimLength = FMath::Max(MaxAnimLength, AnimInstance->GetLength());
				}
			});
	}

	if (MaxAnimLength > 0.0f)
	{
		// Set the time range to start at 0 and end at the animation length.
		// Enforce a minimum time range of one frame so the timeline remains usable with very short animations.
		// Use SetTimeRange to keep CacheParams.TimeRange in sync.
		SetTimeRange(FVector2f(0.0f, FMath::Max(MaxAnimLength, DeltaTime)));
	}
	else
	{
		// No animation - reset to the cache params time range
		SetTimeRange(SceneDescription->CacheParams.TimeRange);
	}

	BroadcastTimeRangeUpdated();
}

void FDataflowSimulationScene::BroadcastTimeRangeUpdated()
{
	const float ActualLength = TimeRange[1] - TimeRange[0];
	NumFrames = FMath::Max(1, (ActualLength > 0) ? FMath::Floor((ActualLength + UE_SMALL_NUMBER) * SceneDescription->CacheParams.FrameRate) + 1 : 1);
	if (OnTimeRangeUpdated.IsBound())
	{
		OnTimeRangeUpdated.Broadcast(*this);
	}
}

void FDataflowSimulationScene::CreateSimulationScene()
{
	const int32 PreviewLOD = GetPreviewLOD();

	if(SimulationGenerator && SceneDescription && GetWorld())
	{
		SimulationGenerator->SetCacheParams(SceneDescription->CacheParams);
		SimulationGenerator->SetCacheAsset(SceneDescription->CacheAsset);
		SimulationGenerator->SetBlueprintClass(SceneDescription->GetBlueprintClass());
		SimulationGenerator->SetBlueprintTransform(SceneDescription->BlueprintTransform);
		SimulationGenerator->SetBlueprintVariables(SceneDescription->BlueprintVariables);
		SimulationGenerator->SetDataflowContent(GetEditorContent());

		TimeRange = SceneDescription->CacheParams.TimeRange;
		NumFrames = (TimeRange[1] > TimeRange[0]) ? FMath::Floor((TimeRange[1] - TimeRange[0] + UE_SMALL_NUMBER) * SceneDescription->CacheParams.FrameRate) + 1 : 1;
		
		DeltaTime = (SceneDescription->CacheParams.FrameRate > 0) ? 1.f / SceneDescription->CacheParams.FrameRate: 0.f;
		PreviewActor = UE::Dataflow::SpawnSimulatedActor(SceneDescription->GetBlueprintClass(), Cast<AChaosCacheManager>(RootSceneActor),
			SceneDescription->CacheAsset, false, GetEditorContent(), SceneDescription->BlueprintTransform, SceneDescription->BlueprintVariables);
		SimulationGenerator->SetDeltaTime(DeltaTime);
		// Setup all the skelmesh animations.
		UE::Dataflow::SetupSkeletonAnimation(PreviewActor, SceneDescription->bSkeletalMeshVisibility);

		if(PreviewActor)
		{
			PreviewActor->ForEachComponent<UActorComponent>(true, [this](UActorComponent* ActorComponent)
			{
				AddSceneObject(ActorComponent, false);
			});
			AddSceneObject(PreviewActor, false);

			// Notify a freshly spawned preview actor (the simulation Blueprint) of the current simulation enabled state
			if (PreviewWorld)
			{
				if (const UDataflowSimulationManager* const SimulationManager = PreviewWorld->GetSubsystem<UDataflowSimulationManager>())
				{
					SimulationManager->NotifyActorOfCurrentSimulationEnabled(PreviewActor);
				}
			}
		}
		AddSceneObject(RootSceneActor, false);

		// Note: Do not force-enable simulation here. The simulation enabled state is managed by:
		// - The constructor (applies bIsSimulationPlayingByDefault after CreateSimulationScene)
		// - User actions (play/pause/step buttons, cache recording)
		// Force-enabling here would auto-start simulation on every property change (e.g. setting CacheAsset),
		// overriding the user's current play/pause state.

		// Pause the animation
		UE::Dataflow::PauseSkeletonAnimation(PreviewActor);
	}

	bPreviewSceneDirty = true;
	
	// update the selection binding since we are constantly editing the graph
	BindSceneSelection();

	SetPreviewLOD(PreviewLOD);
}

void FDataflowSimulationScene::RecordSimulationCache()
{
	if(AChaosCacheManager* CacheManager = Cast<AChaosCacheManager>(RootSceneActor))
	{
		if(SceneDescription->CacheParams.bAsyncCaching)
		{
			if(SimulationGenerator.IsValid())
			{
				SimulationGenerator->RequestGeneratorAction(UE::Dataflow::EDataflowGeneratorActions::StartGenerate);
				bIsSimulationLocked = true;
			}
		}
		else
		{
			bIsRecordingCache = true;
			bIsSimulationLocked = false;
		
			SimulationTime = TimeRange[0] - DeltaTime;
			GetWorld()->GetSubsystem<UDataflowSimulationManager>()->SetSimulationEnabled(true);
			
			CacheManager->CacheMode = ECacheMode::Record;
			CacheManager->SetObservedComponentProperties(CacheManager->CacheMode);
			CacheManager->BeginEvaluate();
		}
	}
}

void FDataflowSimulationScene::UpdateSimulationCache()
{
	if(AChaosCacheManager* CacheManager = Cast<AChaosCacheManager>(RootSceneActor))
	{
		if(!bIsRecordingCache)
		{
			if(bIsSimulationLocked)
			{
				// Update the cached simulation at some point in time
				// We need to add delta time because the CHaos Cache Actor incremenet by Dt before recording the first frame
				// The chaos cache manager replays are compensating with this issue and fixing it would break existing caches
				// so we nee dto fix how we replay in Dataflow
				const float ScrollBarTime = SimulationTime - TimeRange[0] + DeltaTime;
				if (ScrollBarTime != CacheManager->StartTime)
				{
					CacheManager->SetStartTime(ScrollBarTime);
				}
			}
		}
		else
		{
			SimulationTime += DeltaTime;
			if(SimulationTime >= TimeRange[1]+UE_SMALL_NUMBER)
			{
				bIsRecordingCache = false;
				bIsSimulationLocked = true;
				
				CacheManager->EndEvaluate();
				GetWorld()->GetSubsystem<UDataflowSimulationManager>()->SetSimulationEnabled(false);
						
				CacheManager->CacheMode = ECacheMode::None;
				CacheManager->SetObservedComponentProperties(CacheManager->CacheMode);
				CacheManager->BeginEvaluate();
			}
		}
	}
}

void FDataflowSimulationScene::SetTimeRange(const FVector2f& NewTimeRange)
{
	TimeRange = NewTimeRange;
	if (SceneDescription)
	{
		SceneDescription->CacheParams.TimeRange = NewTimeRange;
	}
}

void FDataflowSimulationScene::SetPreviewLOD(int32 InLOD)
{
	CurrentPreviewLOD = InLOD;

	if (PreviewActor)
	{
		PreviewActor->ForEachComponent<UActorComponent>(true, [this](UActorComponent* Component)
		{
			if (ILODSyncInterface* const LODInterface = Cast<ILODSyncInterface>(Component))
			{
				LODInterface->SetForceStreamedLOD(CurrentPreviewLOD);
				LODInterface->SetForceRenderedLOD(CurrentPreviewLOD);
			}
		});
	}
}

int32 FDataflowSimulationScene::GetPreviewLOD() const
{
	return CurrentPreviewLOD;
}

FBox FDataflowSimulationScene::GetBoundingBox() const
{
	FBox SceneBounds(ForceInitToZero);
	if (DataflowModeManager.IsValid())
	{
		USelection* const SelectedComponents = DataflowModeManager->GetSelectedComponents();

		TArray<TWeakObjectPtr<UObject>> SelectedObjects;
		const int32 NumSelected = SelectedComponents? SelectedComponents->GetSelectedObjects(SelectedObjects): 0;

		if (NumSelected > 0)
		{
			for (const TWeakObjectPtr<const UObject> SelectedObject : SelectedObjects)
			{
				if (const UPrimitiveComponent* const SelectedComponent = Cast<const UPrimitiveComponent>(SelectedObject))
				{
					// Ignore billboard ones as they have a very large bounds (-256, 256)
					const UBillboardComponent* BillboardComponent = Cast<UBillboardComponent>(SelectedComponent);
					if (BillboardComponent == nullptr)
					{
						SceneBounds += SelectedComponent->Bounds.GetBox();
					}
				}
			}
		}

		// if no selection of results in an invalid box 
		const bool bInvalidBounds = SceneBounds.GetExtent().IsNearlyZero(UE_SMALL_NUMBER);
		if (bInvalidBounds && PreviewActor)
		{
			if (PreviewActor)
			{
				PreviewActor->ForEachComponent<UPrimitiveComponent>(true, [this, &SceneBounds](UPrimitiveComponent* PrimComponent)
					{
						// Ignore billboard ones as they have a very large bounds (-256, 256)
						const UBillboardComponent* BillboardComponent = Cast<UBillboardComponent>(PrimComponent);
						if (BillboardComponent == nullptr)
						{
							SceneBounds += PrimComponent->Bounds.GetBox();
						}
					});
			}
		}
	}
	return SceneBounds;
}

void FDataflowSimulationScene::TickDataflowScene(const float DeltaSeconds)
{
	if(const TObjectPtr<UDataflowBaseContent>& EditorContent = GetEditorContent())
	{
		if (const TObjectPtr<UDataflow> DataflowGraph = EditorContent->GetDataflowAsset())
		{
			if(!bIsSimulationLocked && (UE::Dataflow::ShouldResetWorld(DataflowGraph, GetWorld(), LastTimeStamp) || EditorContent->IsSimulationDirty()))
			{
				EditorContent->OnContentDataChanged.Broadcast(PreviewActor);
			}
		}

		// Update the simulation cache at the simulation time
		UpdateSimulationCache();
		
		// Update all the skelmesh animations at the simulation time
		UE::Dataflow::UpdateSkeletonAnimation(PreviewActor, SimulationTime);

		// Compute the modified elapsed time after updating the simulation cache (loading/recording) if necessary
		const float ElapsedTime = bIsRecordingCache ? DeltaTime  : DeltaSeconds;

		// Advance the world ticking if necessary
		if (ElapsedTime > 0.0f)
		{
			GetWorld()->Tick(ELevelTick::LEVELTICK_All, ElapsedTime);
		}
	}
}

void FDataflowSimulationScene::AddReferencedObjects(FReferenceCollector& Collector)
{
	FDataflowPreviewSceneBase::AddReferencedObjects(Collector);

	Collector.AddReferencedObject(SceneDescription);
	Collector.AddReferencedObject(PreviewActor);
}

void FDataflowSimulationScene::SceneDescriptionPropertyChanged(const FName& PropertyName)
{
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UDataflowSimulationSceneDescription, CacheParams))
	{
		if(SimulationGenerator)
		{
			SimulationGenerator->SetCacheParams(SceneDescription->CacheParams);
		}
	}
	else if(PropertyName == GET_MEMBER_NAME_CHECKED(UDataflowSimulationSceneDescription, CacheAsset))
	{
		if(SimulationGenerator)
		{
			SimulationGenerator->SetCacheAsset(SceneDescription->CacheAsset);
		}
	}
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	else if(PropertyName == GET_MEMBER_NAME_CHECKED(UDataflowSimulationSceneDescription, BlueprintClass))
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		if(SimulationGenerator)
		{
			SimulationGenerator->SetBlueprintClass(SceneDescription->GetBlueprintClass());
			SimulationGenerator->SetBlueprintVariables(SceneDescription->BlueprintVariables);
		}
	}
	else if(PropertyName == GET_MEMBER_NAME_CHECKED(UDataflowSimulationSceneDescription, BlueprintTransform))
	{
		if(SimulationGenerator)
		{
			SimulationGenerator->SetBlueprintTransform(SceneDescription->BlueprintTransform);
		}
	}
	else if(PropertyName == GET_MEMBER_NAME_CHECKED(UDataflowSimulationSceneDescription, BlueprintVariables))
	{
		WritePreviewPropertiesToDataflowOwner();
		if(SimulationGenerator)
		{
			SimulationGenerator->SetBlueprintVariables(SceneDescription->BlueprintVariables);
		}
	}
	if(GetEditorContent())
	{
		if(UDataflow* DataflowAsset = GetEditorContent()->GetDataflowAsset())
		{
#if WITH_EDITORONLY_DATA
			DataflowAsset->PreviewCacheParams = SceneDescription->CacheParams;
			DataflowAsset->PreviewCacheAsset = {};
			if (SceneDescription->CacheAsset && SceneDescription->CacheAsset->IsAsset())
			{
				DataflowAsset->PreviewCacheAsset = SceneDescription->CacheAsset;
			}
			DataflowAsset->PreviewBlueprintClass = SceneDescription->GetBlueprintClass();
			DataflowAsset->PreviewBlueprintTransform = SceneDescription->BlueprintTransform;
			DataflowAsset->PreviewGeometryCacheAsset = SceneDescription->GeometryCacheAsset;
			DataflowAsset->PreviewEmbeddedSkeletalMesh = SceneDescription->EmbeddedSkeletalMesh;
			DataflowAsset->PreviewEmbeddedStaticMesh = SceneDescription->EmbeddedStaticMesh;
			DataflowAsset->MarkPackageDirty();
#endif
		}
	}
	
	// Unregister components, cache manager, selection...
	ResetSimulationScene();

	// Register components, cache manager, selection...
	CreateSimulationScene();

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UDataflowSimulationSceneDescription, CacheParams))
	{
		// The user just edited CacheParams directly (TimeRange, FrameRate, etc.).
		// CreateSimulationScene has already synced TimeRange from CacheParams.TimeRange;
		// re-deriving from the animation length here would clobber the user's edit.
		BroadcastTimeRangeUpdated();
	}
	else
	{
		// Preview BP / CacheAsset changes can affect the effective animation length;
		// refresh the timeline so the scrubber bounds track the new preview state.
		UpdateTimelinedataFromAnimations();
	}
}

void FDataflowSimulationScene::ReadPreviewPropertiesFromDataflowOwner()
{
	if (SceneDescription)
	{
		if (const UDataflowBaseContent* const EditorContent = GetEditorContent())
		{
			if (const IDataflowInstanceInterface* const DataflowInstanceInterface = UE::Dataflow::InstanceUtils::GetDataflowInterfaceFromObject(EditorContent->GetDataflowOwner()))
			{
				const FDataflowInstance& DataflowInstance = DataflowInstanceInterface->GetDataflowInstance();
				SceneDescription->BlueprintVariables.CopyMatchingValuesByName(DataflowInstance.GetPreviewBlueprintVariableOverrides());  // When reading, match the asset variables
			}
		}
	}
}

void FDataflowSimulationScene::WritePreviewPropertiesToDataflowOwner() const
{
	if (SceneDescription)
	{
		if (const UDataflowBaseContent* const EditorContent = GetEditorContent())
		{
			UObject* const DataflowOwner = EditorContent->GetDataflowOwner();
			if (IDataflowInstanceInterface* const DataflowInstanceInterface = UE::Dataflow::InstanceUtils::GetDataflowInterfaceFromObject(DataflowOwner))
			{
				FDataflowInstance& DataflowInstance = DataflowInstanceInterface->GetDataflowInstance();
				DataflowInstance.GetPreviewBlueprintVariableOverrides() = SceneDescription->BlueprintVariables;  // When writing, clobber the asset variables
				DataflowOwner->MarkPackageDirty();
			}
		}
	}
}

void UDataflowSimulationSceneDescription::GenerateGeometryCache()
{
	if (!bIsGeometryCacheOutputSupported)
	{
		const FText Title = LOCTEXT("DataflowEditor_GenerateGeometryCache_Title", "Dataflow Editor");
		const FText Message = LOCTEXT("DataflowEditor_GenerateGeometryCache_Message", "Writing to a geometry cache is not supported for this type of asset");
		constexpr float MessageDuration = 5.f;
		DataflowSimulationScene::Private::ShowErrorToastMessage(Title, Message, MessageDuration);

		return;
	}

	SimulationScene->ResetSimulationScene();
	SimulationScene->CreateSimulationScene();
	const int32 NumFrames = SimulationScene->GetNumFrames();
	float Time = 0;
	TObjectPtr<AActor> GetRootActor = SimulationScene->GetRootActor();
	TObjectPtr<AActor> PreviewActor = SimulationScene->GetPreviewActor();
	const bool UseSkeletalMesh = EmbeddedSkeletalMesh != nullptr;
	const bool UseStaticMesh = EmbeddedStaticMesh != nullptr;
	if (CacheAsset && GeometryCacheAsset && GetRootActor && (UseSkeletalMesh || UseStaticMesh))
	{
		IDataflowGeometryCachable* GeometryCachable = nullptr; //interface for ChaosDeformableTetrahedralComponent

		RenderPositions.SetNum(NumFrames);
		TInlineComponentArray<UPrimitiveComponent*> PrimComponents;
		PreviewActor->GetComponents(PrimComponents);
		for (UPrimitiveComponent* PrimComponent : PrimComponents)
		{
			GeometryCachable = Cast<IDataflowGeometryCachable>(PrimComponent);
			if (GeometryCachable)
			{
				break;
			}
		}
		if (!GeometryCachable)
		{
			UE_LOGF(LogDataflowSimulationGeometryCache, Error, "No GeometryCachable Component in the Preview Actor");
			return;
		}
		for (int32 Frame = 0; Frame < NumFrames; ++Frame)
		{
			Cast<AChaosCacheManager>(GetRootActor)->SetStartTime(Time); // cache time range is [0, (NumFrames-1)*dt]
			if (UseSkeletalMesh)
			{
				RenderPositions[Frame] = GeometryCachable->GetGeometryCachePositions(EmbeddedSkeletalMesh);
			}
			else
			{
				RenderPositions[Frame] = GeometryCachable->GetGeometryCachePositions(EmbeddedStaticMesh);
			}
			Time += SimulationScene->GetDeltaTime();
		}
		if (UseSkeletalMesh)
		{
			TOptional<TArray<int32>> OptionalMap = GeometryCachable->GetMeshImportVertexMap(*EmbeddedSkeletalMesh);
			if (!OptionalMap)
			{
				UE_LOGF(LogDataflowSimulationGeometryCache, Error, "Failed to get MeshImportVertexMap for the skeletal mesh. See the log for more info.");
				return;
			}
			const TArray<int32>& Map = OptionalMap.GetValue();
			TArray<uint32> ImportedVertexNumbers = TArray<uint32>(reinterpret_cast<const uint32*>(Map.GetData()), Map.Num());
			UE::DataflowSimulationGeometryCache::SaveGeometryCache(*GeometryCacheAsset, float(CacheParams.FrameRate), *EmbeddedSkeletalMesh, ImportedVertexNumbers, RenderPositions);
		}
		else // UseStaticMesh
		{
			UE::DataflowSimulationGeometryCache::SaveGeometryCache(*GeometryCacheAsset, float(CacheParams.FrameRate), *EmbeddedStaticMesh, RenderPositions);
		}

		UE::DataflowSimulationGeometryCache::SavePackage(*GeometryCacheAsset);
	}
}

namespace UE::Dataflow::Private
{
	template<class T>
	T* CreateOrLoad(const FString& PackageName)
	{
		const FName AssetName(FPackageName::GetLongPackageAssetName(PackageName));
		if (UPackage* const Package = CreatePackage(*PackageName))
		{
			LoadPackage(nullptr, *PackageName, LOAD_Quiet | LOAD_EditorOnly);
			T* Asset = FindObject<T>(Package, *AssetName.ToString());
			if (!Asset)
			{
				Asset = NewObject<T>(Package, *AssetName.ToString(), RF_Public | RF_Standalone | RF_Transactional);
				Asset->MarkPackageDirty();
				FAssetRegistryModule::AssetCreated(Asset);
			}
			return Asset;
		}
		return nullptr;
	}

	TObjectPtr<UGeometryCache> NewGeometryCacheDialog(const UObject* NamingAsset = nullptr)
	{
		FSaveAssetDialogConfig Config;
		{
			if (NamingAsset)
			{
				const FString PackageName = NamingAsset->GetOutermost()->GetName();
				Config.DefaultPath = FPackageName::GetLongPackagePath(PackageName);
				Config.DefaultAssetName = FString::Printf(TEXT("GeometryCache_%s"), *NamingAsset->GetName());
			}
			Config.AssetClassNames.Add(UGeometryCache::StaticClass()->GetClassPathName());
			Config.ExistingAssetPolicy = ESaveAssetDialogExistingAssetPolicy::Disallow;
			Config.DialogTitleOverride = LOCTEXT("ExportGeometryCacheDialogTitle", "Export Geometry Cache As");
		}

		FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");

#if WITH_EDITOR
		FString NewPackageName;
		FText OutError;
		for (bool bFilenameValid = false; !bFilenameValid; bFilenameValid = FFileHelper::IsFilenameValidForSaving(NewPackageName, OutError))
		{
			const FString AssetPath = ContentBrowserModule.Get().CreateModalSaveAssetDialog(Config);
			if (AssetPath.IsEmpty())
			{
				return nullptr;
			}
			NewPackageName = FPackageName::ObjectPathToPackageName(AssetPath);
		}
		return CreateOrLoad<UGeometryCache>(NewPackageName);
#else
		return nullptr;
#endif
	}
};

void UDataflowSimulationSceneDescription::NewGeometryCache()
{
	if (!bIsGeometryCacheOutputSupported)
	{
		const FText Title = LOCTEXT("DataflowEditor_NewGeometryCache_Title", "Dataflow Editor");
		const FText Message = LOCTEXT("DataflowEditor_NewGeometryCache_Message", "Writing to a geometry cache is not supported for this type of asset");
		constexpr float MessageDuration = 5.f;
		DataflowSimulationScene::Private::ShowErrorToastMessage(Title, Message, MessageDuration);

		return;
	}

	const UObject* const NamingAsset = CacheAsset? CacheAsset.Get() : nullptr;
	GeometryCacheAsset = UE::Dataflow::Private::NewGeometryCacheDialog(NamingAsset);
}

void UDataflowSimulationSceneDescription::SetSimulationScene(FDataflowSimulationScene* InSimulationScene)
{
	SimulationScene = InSimulationScene;
}

void UDataflowSimulationSceneDescription::SetBlueprintClass(TSubclassOf<AActor> InBlueprintClass)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (BlueprintClass != InBlueprintClass)
	{
		OnBlueprintClassPreChanged();
		BlueprintClass = InBlueprintClass;
		OnBlueprintClassPostChanged();
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

const TSubclassOf<AActor>& UDataflowSimulationSceneDescription::GetBlueprintClass() const
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return BlueprintClass;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UDataflowSimulationSceneDescription::PreEditChange(FProperty* PropertyAboutToChange)
{
	Super::PreEditChange(PropertyAboutToChange);

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (PropertyAboutToChange && PropertyAboutToChange->GetFName() == GET_MEMBER_NAME_CHECKED(UDataflowSimulationSceneDescription, BlueprintClass))
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		OnBlueprintClassPreChanged();
	}
}

void UDataflowSimulationSceneDescription::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (PropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(UDataflowSimulationSceneDescription, BlueprintClass))
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		OnBlueprintClassPostChanged();
	}

	if (SimulationScene)
	{
		SimulationScene->SceneDescriptionPropertyChanged(PropertyChangedEvent.GetMemberPropertyName());
	}

	DataflowSimulationSceneDescriptionChanged.Broadcast();
}

void UDataflowSimulationSceneDescription::PostTransacted(const FTransactionObjectEvent& TransactionEvent)
{
	Super::PostTransacted(TransactionEvent);

	// On Undo/Redo, PostEditChangeProperty just gets an empty FPropertyChangedEvent. However this function gets enough info to figure out which property changed
	if (TransactionEvent.GetEventType() == ETransactionObjectEventType::UndoRedo && TransactionEvent.HasPropertyChanges())
	{
		const TArray<FName>& PropertyNames = TransactionEvent.GetChangedProperties();
		for (const FName& PropertyName : PropertyNames)
		{
			SimulationScene->SceneDescriptionPropertyChanged(PropertyName);
		}
	}
}

void UDataflowSimulationSceneDescription::BeginDestroy()
{
	Super::BeginDestroy();
	UnregisterOnBlueprintCompiledDelegate();
}

void UDataflowSimulationSceneDescription::OnBlueprintClassPreChanged()
{
	UnregisterOnBlueprintCompiledDelegate();
}

void UDataflowSimulationSceneDescription::OnBlueprintClassPostChanged()
{
	const UBlueprintGeneratedClass* const BlueprintGeneratedClass = Cast<const UBlueprintGeneratedClass>(GetBlueprintClass());
	UBlueprint* const Blueprint = BlueprintGeneratedClass ? Cast<UBlueprint>(BlueprintGeneratedClass->ClassGeneratedBy) : nullptr;
	UpdateBlueprintVariables(Blueprint);  // Always call UpdateBlueprintVariables, even with a null Blueprint, so it correctly resets the variables
	RegisterOnBlueprintCompiledDelegate();
}

void UDataflowSimulationSceneDescription::UpdateBlueprintVariables(UBlueprint* Blueprint)
{
	BlueprintVariables.Reset();
	if (const UBlueprintGeneratedClass* const BlueprintGeneratedClass = Blueprint ? Cast<UBlueprintGeneratedClass>(Blueprint->GeneratedClass) : nullptr)
	{
		const UObject* const ClassDefaultObject = BlueprintGeneratedClass->GetDefaultObject();
		for (const FBPVariableDescription& BPVariableDescription : Blueprint->NewVariables)
		{
			if (BPVariableDescription.HasMetaData("BlueprintPrivate") &&
				BPVariableDescription.GetMetaData("BlueprintPrivate") == TEXT("true"))
			{
				continue;
			}

			if (const FProperty* const Property = BlueprintGeneratedClass->FindPropertyByName(BPVariableDescription.VarName))
			{
				const FName VarGuidAsName(BPVariableDescription.VarGuid.ToString());  // VarGuid instead of the VarName, because the variable name can change and FInstancedPropertyBag::SanitizePropertyName
				BlueprintVariables.AddProperty(VarGuidAsName, Property);
				// Seed the bag with the blueprint class default
				if (ClassDefaultObject)
				{
					BlueprintVariables.SetValue(VarGuidAsName, Property, ClassDefaultObject);
				}
			}
		}
	}
	// Must re-sync properties when the blueprint has changed
	if (SimulationScene)
	{
		SimulationScene->ReadPreviewPropertiesFromDataflowOwner();
	}
}

void UDataflowSimulationSceneDescription::RegisterOnBlueprintCompiledDelegate()
{
	if (const UBlueprintGeneratedClass* const BlueprintGeneratedClass = Cast<const UBlueprintGeneratedClass>(GetBlueprintClass()))
	{
		if (UBlueprint* const Blueprint = Cast<UBlueprint>(BlueprintGeneratedClass->ClassGeneratedBy))
		{
			Blueprint->OnCompiled().RemoveAll(this);
			Blueprint->OnCompiled().AddUObject(this, &UDataflowSimulationSceneDescription::UpdateBlueprintVariables);
		}
	}
}

void UDataflowSimulationSceneDescription::UnregisterOnBlueprintCompiledDelegate()
{
	if (const UBlueprintGeneratedClass* const BlueprintGeneratedClass = Cast<const UBlueprintGeneratedClass>(GetBlueprintClass()))
	{
		if (UBlueprint* const Blueprint = Cast<UBlueprint>(BlueprintGeneratedClass->ClassGeneratedBy))
		{
			Blueprint->OnCompiled().RemoveAll(this);
		}
	}
}


#undef LOCTEXT_NAMESPACE

