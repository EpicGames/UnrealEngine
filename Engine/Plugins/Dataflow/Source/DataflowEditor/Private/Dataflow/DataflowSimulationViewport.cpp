// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowSimulationViewport.h"

#include "DataflowPreviewProfileController.h"
#include "DataflowSceneProfileIndexStorage.h"
#include "Dataflow/DataflowActor.h"
#include "Dataflow/DataflowEditorMode.h"
#include "Dataflow/DataflowSimulationViewportClient.h"
#include "Dataflow/DataflowEditorToolkit.h"
#include "EditorModeManager.h"
#include "Animation/AnimSingleNodeInstance.h"
#include "Dataflow/DataflowContent.h"
#include "Dataflow/DataflowEditorCommands.h"
#include "Dataflow/DataflowSimulationScene.h"
#include "Dataflow/DataflowSimulationPanel.h"
#include "Dataflow/DataflowSimulationVisualization.h"
#include "ToolMenus.h"
#include "LODSyncInterface.h"
#include "ViewportToolbar/UnrealEdViewportToolbar.h"
#include "Widgets/Text/SRichTextBlock.h"
#include "ToolMenuSection.h"

#define LOCTEXT_NAMESPACE "SDataflowSimulationViewport"

SDataflowSimulationViewport::SDataflowSimulationViewport()
{
}

const TSharedPtr<FDataflowSimulationScene>& SDataflowSimulationViewport::GetSimulationScene() const
{
	const TSharedPtr<FDataflowSimulationViewportClient> DataflowClient = StaticCastSharedPtr<FDataflowSimulationViewportClient>(Client);
	return DataflowClient->GetDataflowEditorToolkit().Pin()->GetSimulationScene();
}

void SDataflowSimulationViewport::Construct(const FArguments& InArgs, const FAssetEditorViewportConstructionArgs& InViewportConstructionArgs)
{
	SAssetEditorViewport::FArguments ParentArgs;
	ParentArgs._EditorViewportClient = InArgs._ViewportClient;
	SAssetEditorViewport::Construct(ParentArgs, InViewportConstructionArgs);
	Client->VisibilityDelegate.BindSP(this, &SDataflowSimulationViewport::IsVisible);

	if(static_cast<FDataflowSimulationScene*>(Client->GetPreviewScene())->CanRunSimulation())
	{
		TWeakPtr<FDataflowSimulationScene> SimulationScene = GetSimulationScene();

		ViewportOverlay->AddSlot()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Top)
			.FillWidth(1)
			.Padding(10.0f, 40.0f)
			[
				// Display text 
				SNew(SRichTextBlock)
					.DecoratorStyleSet(&FAppStyle::Get())
					.Text(this, &SDataflowSimulationViewport::GetDisplayString)
					.TextStyle(&FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("AnimViewport.MessageText"))
			]
		];

	}
}

FText SDataflowSimulationViewport::GetDisplayString() const
{
	using namespace UE::Dataflow;
	
	auto ConcatenateLine = [](const FText& InText, const FText& InNewLine)->FText
	{
		if (InText.IsEmpty())
		{
			return InNewLine;
		}
		return FText::Format(LOCTEXT("ViewportTextNewlineFormatter", "{0}\n{1}"), InText, InNewLine);
	};

	FText DisplayText;

	const TMap<FName, TUniquePtr<IDataflowSimulationVisualization>>& Visualizations = FDataflowSimulationVisualizationRegistry::GetInstance().GetVisualizations();
	for (const TPair<FName, TUniquePtr<IDataflowSimulationVisualization>>& Visualization : Visualizations)
	{
		FText Text = Visualization.Value->GetDisplayString(GetSimulationScene().Get());
		DisplayText = ConcatenateLine(DisplayText, Text);
	}
	
	return DisplayText;
}

TSharedPtr<SWidget> SDataflowSimulationViewport::BuildViewportToolbar()
{
	const FName ToolbarName = "Dataflow.SimulationViewportToolbar";
	
	if (!UToolMenus::Get()->IsMenuRegistered(ToolbarName))
	{
		UToolMenu* Menu = UToolMenus::Get()->RegisterMenu(ToolbarName, NAME_None, EMultiBoxType::SlimHorizontalToolBar);
		Menu->StyleName = "ViewportToolbar";
		
		Menu->AddSection("Left");
		
		{
			FToolMenuSection& RightSection = Menu->AddSection("Right");
			RightSection.Alignment = EToolMenuSectionAlign::Last;
			
			RightSection.AddEntry(UE::UnrealEd::CreateCameraSubmenu(
				UE::UnrealEd::FViewportCameraMenuOptions()
					.ShowCameraMovement()
					.ShowLensControls()
			));
			RightSection.AddEntry(UE::UnrealEd::CreateViewModesSubmenu());
			RightSection.AddEntry(UE::UnrealEd::CreateDefaultShowSubmenu());
			RightSection.AddEntry(UE::UnrealEd::CreateAssetViewerProfileSubmenu());
			{
				RightSection.AddDynamicEntry("ViewAndVisualization", FNewToolMenuSectionDelegate::CreateLambda([this](FToolMenuSection& Section)
				{
					BuildVisualizationMenu(Section);	
				}));
			}
			
			{
				// LOD
				RightSection.AddDynamicEntry("DynamicLOD", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& Section)
				{
					if (UUnrealEdViewportToolbarContext* Context = Section.FindContext<UUnrealEdViewportToolbarContext>())
					{
						TWeakPtr<SDataflowSimulationViewport> SimulationViewport = StaticCastWeakPtr<SDataflowSimulationViewport>(Context->Viewport);
						Section.AddEntry(UE::UnrealEd::CreatePreviewLODSelectionSubmenu(SimulationViewport));
					}
				}));
			}
		}
	}
	
	FToolMenuContext Context;
	{
		Context.AppendCommandList(GetCommandList());
		Context.AddExtender(GetExtenders());
		
		UUnrealEdViewportToolbarContext* ContextObject = UE::UnrealEd::CreateViewportToolbarDefaultContext(SharedThis(this));
		Context.AddObject(ContextObject);
	}
	
	return UToolMenus::Get()->GenerateWidget(ToolbarName, Context);
}

void SDataflowSimulationViewport::BuildVisualizationMenu(FToolMenuSection& MenuSection)
{
	if (UUnrealEdViewportToolbarContext* Context = MenuSection.FindContext<UUnrealEdViewportToolbarContext>())
	{
		MenuSection.AddSubMenu(
			"Visualization",
			LOCTEXT("VisualizationSubmenuLabel", "Visualization"),
			LOCTEXT("VisualizationSubmenuTooltip", "Controls visualization options"),
			FNewMenuDelegate::CreateLambda([WeakViewport = Context->Viewport](FMenuBuilder& Menu)
				{
					if (const TSharedPtr<SDataflowSimulationViewport> Viewport = StaticCastSharedPtr<SDataflowSimulationViewport>(WeakViewport.Pin()))
					{
						if (const TSharedPtr<FEditorViewportClient> ViewportClient = Viewport->GetViewportClient())
						{
							for (const TPair<FName, TUniquePtr<UE::Dataflow::IDataflowSimulationVisualization>>& Visualization : UE::Dataflow::FDataflowSimulationVisualizationRegistry::GetInstance().GetVisualizations())
							{
								Visualization.Value->ExtendSimulationVisualizationMenu(StaticCastSharedPtr<FDataflowSimulationViewportClient>(ViewportClient), Menu);
							}
						}
					}
				}),
			/*bInopenSubMenuOnClick*/ false,
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "DetailsView.ViewOptions"),
			/*bShouldCloseWindowAfterMenuSelection*/ false
		);
	}
}


TSharedPtr<IPreviewProfileController> SDataflowSimulationViewport::CreatePreviewProfileController()
{
	TSharedPtr<FDataflowSimulationSceneProfileIndexStorage> ProfileIndexStorage;

	if (const TSharedPtr<FDataflowSimulationViewportClient> ViewportClient = StaticCastSharedPtr<FDataflowSimulationViewportClient>(GetViewportClient()))
	{
		if (ViewportClient->GetDataflowEditorToolkit().IsValid())
		{
			if (const TSharedPtr<FDataflowEditorToolkit> Toolkit = ViewportClient->GetDataflowEditorToolkit().Pin())
			{
				ProfileIndexStorage = Toolkit->GetSimulationSceneProfileIndexStorage();
			}
		}
	}
	
	if (!ProfileIndexStorage)
	{
		ProfileIndexStorage = MakeShared<FDataflowSimulationSceneProfileIndexStorage>(GetSimulationScene());
	}
	
	return MakeShared<FDataflowPreviewProfileController>(ProfileIndexStorage);
}

void SDataflowSimulationViewport::OnFocusViewportToSelection()
{
	if(const FDataflowPreviewSceneBase* PreviewScene = static_cast<FDataflowPreviewSceneBase*>(Client->GetPreviewScene()))
	{
		const FBox SceneBoundingBox = PreviewScene->GetBoundingBox();
		Client->FocusViewportOnBox(SceneBoundingBox);
	}
}

UDataflowEditorMode* SDataflowSimulationViewport::GetEdMode() const
{
	if (const FEditorModeTools* const EditorModeTools = Client->GetModeTools())
	{
		if (UDataflowEditorMode* const DataflowEdMode = Cast<UDataflowEditorMode>(EditorModeTools->GetActiveScriptableMode(UDataflowEditorMode::EM_DataflowEditorModeId)))
		{
			return DataflowEdMode;
		}
	}
	return nullptr;
}

void SDataflowSimulationViewport::BindCommands()
{
	SAssetEditorViewport::BindCommands();
}

bool SDataflowSimulationViewport::IsVisible() const
{
	// Intentionally not calling SEditorViewport::IsVisible because it will return false if our simulation is more than 250ms.
	return ViewportWidget.IsValid();
}

TSharedRef<class SEditorViewport> SDataflowSimulationViewport::GetViewportWidget()
{
	return SharedThis(this);
}

TSharedPtr<FExtender> SDataflowSimulationViewport::GetExtenders() const
{
	TSharedPtr<FExtender> Result(MakeShareable(new FExtender));
	return Result;
}

void SDataflowSimulationViewport::OnFloatingButtonClicked()
{
}

int32 SDataflowSimulationViewport::GetCurrentLOD() const
{
	if (TSharedPtr<FDataflowSimulationScene> SimScene = GetSimulationScene())
	{
		return SimScene->GetPreviewLOD();
	}
	return -1;
}

int32 SDataflowSimulationViewport::GetLODCount() const
{
	int32 MaxNumLODs = 0;
	if (const TSharedPtr<const FDataflowSimulationScene>& SimScene = GetSimulationScene())
	{
		if (const AActor* const PreviewActor = SimScene->GetPreviewActor())
		{
			PreviewActor->ForEachComponent<UActorComponent>(true, [&MaxNumLODs](UActorComponent* Component)
			{
				if (const ILODSyncInterface* const LODInterface = Cast<ILODSyncInterface>(Component))
				{
					MaxNumLODs = FMath::Max(MaxNumLODs, LODInterface->GetNumSyncLODs());
				}
			});
		}
	}
	return MaxNumLODs;
}

bool SDataflowSimulationViewport::IsLODSelected(int32 LODIndex) const
{
	if (TSharedPtr<FDataflowSimulationScene> SimScene = GetSimulationScene())
	{
		return SimScene->GetPreviewLOD() == LODIndex;
	}
	return false;
}

void SDataflowSimulationViewport::SetLODLevel(int32 LODIndex)
{
	if (TSharedPtr<FDataflowSimulationScene> SimScene = GetSimulationScene())
	{
		return SimScene->SetPreviewLOD(LODIndex);
	}
}

float SDataflowSimulationViewport::GetViewMinInput() const
{
	return 0.0f;
}

float SDataflowSimulationViewport::GetViewMaxInput() const
{
	return GetSimulationScene()->GetTimeRange()[1]-GetSimulationScene()->GetTimeRange()[0];
}


#undef LOCTEXT_NAMESPACE
