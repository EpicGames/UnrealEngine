// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "IPreviewLODController.h"
#include "SBaseCharacterFXEditorViewport.h"
#include "SCommonEditorViewportToolbarBase.h"

class ADataflowActor;
class FAdvancedPreviewScene;
class UDataflowEditorMode;
class FDataflowConstructionViewportClient;
struct FToolMenuSection;

// ----------------------------------------------------------------------------------

class SDataflowSimulationViewport : public SBaseCharacterFXEditorViewport, public ICommonEditorViewportToolbarInfoProvider, public IPreviewLODController
{
public:
	SLATE_BEGIN_ARGS(SDataflowSimulationViewport) {}
		SLATE_ARGUMENT(TSharedPtr<FEditorViewportClient>, ViewportClient)
		SLATE_ARGUMENT(TSharedPtr<FUICommandList>, CommandList)
	SLATE_END_ARGS()

	SDataflowSimulationViewport();

	void Construct(const FArguments& InArgs, const FAssetEditorViewportConstructionArgs& InViewportConstructionArgs);

	// SEditorViewport
	virtual void BindCommands() override;
	virtual bool IsVisible() const override;
	virtual TSharedPtr<SWidget> BuildViewportToolbar() override;
	virtual TSharedPtr<IPreviewProfileController> CreatePreviewProfileController() override;
	virtual void OnFocusViewportToSelection() override;
	
	// ICommonEditorViewportToolbarInfoProvider
	virtual TSharedRef<class SEditorViewport> GetViewportWidget() override;
	virtual TSharedPtr<FExtender> GetExtenders() const override;
	virtual void OnFloatingButtonClicked() override;
	
	// IPreviewLODController
	virtual int32 GetCurrentLOD() const override;
	virtual int32 GetLODCount() const override;
	virtual bool IsLODSelected(int32 LODIndex) const override;
	virtual void SetLODLevel(int32 LODIndex) override;

	/** Get the simulation scene */
	const TSharedPtr<class FDataflowSimulationScene>& GetSimulationScene() const;

	/** Build visualisation menu */
	static void BuildVisualizationMenu(FToolMenuSection& MenuSection);

private:
	float GetViewMinInput() const;
	float GetViewMaxInput() const;

	FText GetDisplayString() const;

	UDataflowEditorMode* GetEdMode() const;
};

