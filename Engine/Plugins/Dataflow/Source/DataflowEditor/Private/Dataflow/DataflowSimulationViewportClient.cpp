// Copyright Epic Games, Inc. All Rights Reserved.
#include "Dataflow/DataflowSimulationViewportClient.h"

#include "AssetEditorModeManager.h"
#include "Dataflow/DataflowObject.h"
#include "Dataflow/DataflowEditorMode.h"
#include "Dataflow/DataflowEditorOptions.h"
#include "Dataflow/DataflowEditorToolkit.h"
#include "Dataflow/DataflowEditorCollectionComponent.h"
#include "Dataflow/DataflowEngineSceneHitProxies.h"
#include "Dataflow/DataflowGraphEditor.h"
#include "Dataflow/DataflowEditorPreviewSceneBase.h"
#include "Dataflow/DataflowSimulationScene.h"
#include "Dataflow/DataflowSimulationVisualization.h"
#include "EditorModeManager.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "GraphEditor.h"
#include "PreviewScene.h"
#include "Selection.h"
#include "SGraphPanel.h"
#include "SNodePanel.h"
#include "Widgets/Docking/SDockTab.h"

FDataflowSimulationViewportClient::FDataflowSimulationViewportClient(FEditorModeTools* InModeTools,
                                                             TWeakPtr<FDataflowSimulationScene> InDataflowSimulationScene,  const bool bCouldTickScene,
                                                             const TWeakPtr<SEditorViewport> InEditorViewportWidget)
	: FDataflowEditorViewportClientBase(InModeTools, InDataflowSimulationScene, bCouldTickScene, InEditorViewportWidget)
	, DataflowSimulationScenePtr(InDataflowSimulationScene)
{
	// We want our near clip plane to be quite close so that we can zoom in further.
	OverrideNearClipPlane(KINDA_SMALL_NUMBER);

	EngineShowFlags.SetSelectionOutline(true);
	EngineShowFlags.EnableAdvancedFeatures();

	bEnableSceneTicking = bCouldTickScene;

	if (UDataflowEditorOptions* const Options = UDataflowEditorOptions::StaticClass()->GetDefaultObject<UDataflowEditorOptions>())
	{
		FOVAngle = Options->SimulationViewFOV;
		ViewFOV = FOVAngle;
		ExposureSettings.bFixed = Options->bSimulationViewFixedExposure;
		SetCameraSpeedSettings(Options->SimulationCameraSpeedSettings);
	}
}

FDataflowSimulationViewportClient::~FDataflowSimulationViewportClient()
{
	if (UDataflowEditorOptions* const Options = UDataflowEditorOptions::StaticClass()->GetDefaultObject<UDataflowEditorOptions>())
	{
		Options->SimulationViewFOV = FOVAngle;
		Options->bSimulationViewFixedExposure = ExposureSettings.bFixed;
		Options->SimulationCameraSpeedSettings = GetCameraSpeedSettings();
		Options->SaveConfig();
	}
}

void FDataflowSimulationViewportClient::SetDataflowEditorToolkit(TWeakPtr<FDataflowEditorToolkit> InDataflowEditorToolkitPtr)
{
	DataflowEditorToolkitPtr = InDataflowEditorToolkitPtr;
}

void FDataflowSimulationViewportClient::SetToolCommandList(TWeakPtr<FUICommandList> InToolCommandList)
{
	ToolCommandList = InToolCommandList;
}

bool FDataflowSimulationViewportClient::IsDataflowEditorTabActive() const
{
	if (TSharedPtr<FDataflowEditorToolkit> DataflowEditorToolkit = DataflowEditorToolkitPtr.Pin())
	{
		if (TSharedPtr<FTabManager> TabManager = DataflowEditorToolkit->GetTabManager())
		{
			if (TSharedPtr<SDockTab> DataflowEditorTab = FGlobalTabmanager::Get()->GetMajorTabForTabManager(TabManager.ToSharedRef()))
			{
				bool bIsTabInSamePanelAsActiveOne = false;
				if (TSharedPtr<class SDockTab> ActiveTab = FGlobalTabmanager::Get()->GetActiveTab())
				{
					TSharedPtr<FTabManager> ActiveTabManager = ActiveTab->GetTabManagerPtr();
					bIsTabInSamePanelAsActiveOne = (ActiveTabManager == TabManager);
				}
				const bool bIsTabInForecround = DataflowEditorTab->IsForeground();
				return (bIsTabInSamePanelAsActiveOne && bIsTabInForecround);
			}
		}
	}
	return false;
}

void FDataflowSimulationViewportClient::Tick(float DeltaSeconds)
{
	Super::Tick(DeltaSeconds);
	if (const TSharedPtr<FDataflowSimulationScene> DataflowSimulationScene = DataflowSimulationScenePtr.Pin())
	{
		bool bPauseInPIE = true;
		bool bPauseWhenNotFocused = true;
		if (UDataflowSimulationSceneDescription* SceneDescription = DataflowSimulationScene->GetPreviewSceneDescription())
		{
			bPauseInPIE = SceneDescription->bPauseSimulationViewportWhenPlayingInEditor;
			bPauseWhenNotFocused = SceneDescription->bPauseSimulationViewportWhenNotFocused;
		}

		const bool bIsTabActive = IsDataflowEditorTabActive();
		const bool bIsInPIEOrSIE = GEditor->PlayWorld != NULL || GEditor->bIsSimulatingInEditor;
		const bool bShouldPause = (bPauseInPIE && bIsInPIEOrSIE) || (bPauseWhenNotFocused && !bIsTabActive);
		if (bShouldPause)
		{
			return;
		}

		DataflowSimulationScene->TickDataflowScene(DeltaSeconds);
	}
}

void FDataflowSimulationViewportClient::ProcessClick(FSceneView& View, HHitProxy* HitProxy, FKey Key, EInputEvent Event, uint32 HitX, uint32 HitY)
{
	Super::ProcessClick(View, HitProxy, Key, Event, HitX, HitY);
	OnViewportClicked(HitProxy);
}

void FDataflowSimulationViewportClient::OnViewportClicked(HHitProxy* HitProxy)
{
	USelection* SelectedComponents = ModeTools->GetSelectedComponents();
	
	TArray<UPrimitiveComponent*> PreviouslySelectedComponents;
	SelectedComponents->GetSelectedObjects<UPrimitiveComponent>(PreviouslySelectedComponents);

	SelectedComponents->Modify();
	SelectedComponents->BeginBatchSelectOperation();

	SelectedComponents->DeselectAll();

	if (HitProxy && HitProxy->IsA(HActor::StaticGetType()))
	{
		const HActor* ActorProxy = static_cast<HActor*>(HitProxy);
		if (ActorProxy && ActorProxy->PrimComponent && ActorProxy->Actor)
		{
			UPrimitiveComponent* Component = const_cast<UPrimitiveComponent*>(ActorProxy->PrimComponent.Get());
			SelectedComponents->Select(Component);
			Component->PushSelectionToProxy();
		}
	}
	SelectedComponents->EndBatchSelectOperation();

	for (UPrimitiveComponent* const Component : PreviouslySelectedComponents)
	{
		Component->PushSelectionToProxy();
	}

	TArray<UPrimitiveComponent*> CurrentlySelectedComponents;
	SelectedComponents->GetSelectedObjects<UPrimitiveComponent>(CurrentlySelectedComponents);

	// Get all the scene selected elements 
	TArray<FDataflowBaseElement*> DataflowElements;
	GetSelectedElements(HitProxy, DataflowElements);
	
	OnSelectionChangedMulticast.Broadcast(CurrentlySelectedComponents, DataflowElements);
}


void FDataflowSimulationViewportClient::Draw(const FSceneView* View, FPrimitiveDrawInterface* PDI)
{
	FDataflowEditorViewportClientBase::Draw(View, PDI);

	using namespace UE::Dataflow;
	const TMap<FName, TUniquePtr<IDataflowSimulationVisualization>>& Visualizations = FDataflowSimulationVisualizationRegistry::GetInstance().GetVisualizations();
	for (const TPair<FName, TUniquePtr<IDataflowSimulationVisualization>>& Visualization : Visualizations)
	{
		if (PreviewScene)
		{
			Visualization.Value->Draw(static_cast<FDataflowSimulationScene*>(PreviewScene), PDI);
		}
	}
}

void FDataflowSimulationViewportClient::DrawCanvas(FViewport& InViewport, FSceneView& View, FCanvas& Canvas)
{
	FEditorViewportClient::DrawCanvas(InViewport, View, Canvas);

	using namespace UE::Dataflow;
	const TMap<FName, TUniquePtr<IDataflowSimulationVisualization>>& Visualizations = FDataflowSimulationVisualizationRegistry::GetInstance().GetVisualizations();
	for (const TPair<FName, TUniquePtr<IDataflowSimulationVisualization>>& Visualization : Visualizations)
	{
		if (PreviewScene)
		{
			Visualization.Value->DrawCanvas(static_cast<FDataflowSimulationScene*>(PreviewScene), &Canvas, &View);
		}
	}
}
