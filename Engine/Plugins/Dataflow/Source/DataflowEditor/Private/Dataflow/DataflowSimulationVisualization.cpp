// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowSimulationVisualization.h"
#include "Misc/LazySingleton.h"
#include "ChaosLog.h"

#define LOCTEXT_NAMESPACE "DataflowSimulationVisualization"

namespace UE::Dataflow
{
	FDataflowSimulationVisualizationRegistry& FDataflowSimulationVisualizationRegistry::GetInstance()
	{
		return TLazySingleton<FDataflowSimulationVisualizationRegistry>::Get();
	}

	void FDataflowSimulationVisualizationRegistry::TearDown()
	{
		return TLazySingleton<FDataflowSimulationVisualizationRegistry>::TearDown();
	}

	void FDataflowSimulationVisualizationRegistry::RegisterVisualization(TUniquePtr<IDataflowSimulationVisualization>&& Visualization)
	{
		const FName NewVisualizationName = Visualization->GetName();
		if (VisualizationMap.Contains(NewVisualizationName))
		{
			UE_LOGF(LogChaos, Warning, "Dataflow simulation visualization registration conflicts with existing visualization: %ls", *NewVisualizationName.ToString());
		}
		else
		{
			VisualizationMap.Add(NewVisualizationName, MoveTemp(Visualization));
		}
	}

	void FDataflowSimulationVisualizationRegistry::DeregisterVisualization(const FName& VisualizationName)
	{
		if (!VisualizationMap.Contains(VisualizationName))
		{
			UE_LOGF(LogChaos, Warning, "Dataflow visualization deregistration -- visualization not registered : %ls", *VisualizationName.ToString());
		}
		else
		{
			VisualizationMap.Remove(VisualizationName);
		}
	}

	const TMap<FName, TUniquePtr<IDataflowSimulationVisualization>>& FDataflowSimulationVisualizationRegistry::GetVisualizations() const
	{
		return VisualizationMap;
	}

	const IDataflowSimulationVisualization* FDataflowSimulationVisualizationRegistry::GetVisualization(const FName& Name) const
	{
		if (VisualizationMap.Contains(Name))
		{
			return VisualizationMap[Name].Get();
		}
		return nullptr;
	}

}

#undef LOCTEXT_NAMESPACE 

