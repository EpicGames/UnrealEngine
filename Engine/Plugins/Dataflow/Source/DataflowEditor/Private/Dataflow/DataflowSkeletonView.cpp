// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowSkeletonView.h"

#include "Animation/Skeleton.h"
#include "Dataflow/DataflowEditor.h"
#include "Dataflow/DataflowEdNode.h"
#include "Dataflow/DataflowObjectInterface.h"
#include "GeometryCollection/GeometryCollectionEngineConversion.h"
#include "IEditableSkeleton.h"
#include "ISkeletonTree.h"
#include "ISkeletonTreeItem.h"
#include "ISkeletonEditorModule.h"
#include "Engine/SkeletalMesh.h"

const FName FDataflowSkeletonView::SkeletonName("DataflowSkeleton");
FDataflowSkeletonView::FDataflowSkeletonView(TObjectPtr<UDataflowBaseContent> InContent)
	: FDataflowNodeView(InContent)
	, SkeletonEditor(nullptr)
	, SkeletalMesh(NewObject<USkeletalMesh>())
	, CollectionIndexRemap(TArray<int32>())
{
	check(InContent);
	UpdateSkeleton();
}

FDataflowSkeletonView::~FDataflowSkeletonView()
{
	if (SkeletonEditor)
	{
		// remove widget delegates (see FDataflowCollectionSpreadSheet)
	}
}

TSharedPtr<ISkeletonTree> FDataflowSkeletonView::CreateEditor(FSkeletonTreeArgs& InSkeletonTreeArgs)
{
	ISkeletonEditorModule& SkeletonEditorModule = FModuleManager::LoadModuleChecked<ISkeletonEditorModule>("SkeletonEditor");
	SkeletonEditor = SkeletonEditorModule.CreateSkeletonTree(Skeleton, InSkeletonTreeArgs);
	SkeletonEditor->Refresh();
	// add widget delegates (see FDataflowCollectionSpreadSheet)
	return SkeletonEditor;
}

void FDataflowSkeletonView::SetSupportedOutputTypes()
{
	GetSupportedOutputTypes().Empty();

	GetSupportedOutputTypes().Add("FManagedArrayCollection");
}

void FDataflowSkeletonView::UpdateSkeleton()
{
	if (!Skeleton)
	{
		Skeleton = NewObject<USkeleton>(SkeletalMesh, SkeletonName);
	}
	SkeletalMesh->SetSkeleton(Skeleton);
	SkeletalMesh->SetRefSkeleton(Skeleton->GetReferenceSkeleton());

	if (SkeletonEditor)
	{
		SkeletonEditor->GetEditableSkeleton()->RecreateBoneTree(SkeletalMesh);
		SkeletonEditor->SetSkeletalMesh(SkeletalMesh);
		SkeletonEditor->Refresh();
	}
}

USkeleton* FDataflowSkeletonView::GetSkeleton()
{
	if (SkeletalMesh)
	{
		return SkeletalMesh->GetSkeleton();
	}
	return nullptr;
}

void FDataflowSkeletonView::UpdateViewData()
{
	bool bClearSkeleton = true;
	Skeleton = NewObject<USkeleton>(SkeletalMesh, SkeletonName);
	if (TObjectPtr<UDataflowEdNode> EdNode = GetSelectedNode())
	{
		if (EdNode->IsBound())
		{
			if (TSharedPtr<FDataflowNode> Node = EdNode->DataflowGraph->FindBaseNode(EdNode->DataflowNodeGuid))
			{
				if (FDataflowOutput* Output = Node->FindOutput(FName("Collection")))
				{
					if (const TObjectPtr<UDataflowSkeletalContent> SkeletalContent = Cast<UDataflowSkeletalContent>(GetEditorContent()))
					{
						if (TSharedPtr<UE::Dataflow::FEngineContext> Context = SkeletalContent->GetDataflowContext())
						{
							const FManagedArrayCollection DefaultCollection;
							const FManagedArrayCollection& Result = Output->ReadValue(*Context, DefaultCollection);

							FGeometryCollectionEngineConversion::ConvertCollectionToSkeleton(Result, Skeleton, CollectionIndexRemap);
						}
					}
				}
			}
		}
	}

	UpdateSkeleton();
}

void FDataflowSkeletonView::ConstructionViewSelectionChanged(const TArray<UPrimitiveComponent*>& InSelectedComponents, const TArray<FDataflowBaseElement*>& SelectedElements)
{
	if (ensure(SkeletonEditor))
	{
		SkeletonEditor->DeselectAll();
		for (UPrimitiveComponent* Component : InSelectedComponents)
		{
			SkeletonEditor->SetSelectedBone(FName(Component->GetName()), ESelectInfo::Type::Direct);
		}
		SkeletonEditor->Refresh();
	}
}


void FDataflowSkeletonView::SkeletonViewSelectionChanged(const TArrayView<TSharedPtr<ISkeletonTreeItem>>& InSelectedItems, ESelectInfo::Type InSelectInfo)
{
	if (SkeletonEditor)
	{
		//TArray<UObject*> Objects;
		//Algo::TransformIf(InSelectedItems, Objects,
		//	[](const TSharedPtr<ISkeletonTreeItem>& InItem) { return InItem->GetObject() != nullptr; },
		//	[](const TSharedPtr<ISkeletonTreeItem>& InItem) { return InItem->GetObject(); });
		//DetailsView->SetObjects(Objects);
	}
}

void FDataflowSkeletonView::AddReferencedObjects(FReferenceCollector& Collector)
{
	FDataflowNodeView::AddReferencedObjects(Collector);
	Collector.AddReferencedObject(Skeleton);
	Collector.AddReferencedObject(SkeletalMesh);
}
