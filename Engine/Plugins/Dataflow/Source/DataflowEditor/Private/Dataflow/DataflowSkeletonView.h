// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Dataflow/DataflowView.h"

class UDataflowEditor;
class ISkeletonTree;
class ISkeletonTreeItem;
class USkeleton;
class USkeletalMesh;
class UDataflowEdNode;
struct FSkeletonTreeArgs;

/**
*
* Class to handle the SelectionView widget
*
*/
class FDataflowSkeletonView : public FDataflowNodeView
{
public:
	FDataflowSkeletonView(TObjectPtr<UDataflowBaseContent> InContent = nullptr);
	~FDataflowSkeletonView();

	/** Create the Skeleton Tree Editor*/
	TSharedPtr<ISkeletonTree> CreateEditor(FSkeletonTreeArgs& InSkeletonTreeArgs);

	/** Set selection types*/
	virtual void SetSupportedOutputTypes() override;

	/** Skeleton Access */
	USkeleton* GetSkeleton();

	/** Update Data*/
	void UpdateSkeleton();

	/** Update the view */
	virtual void UpdateViewData() override;

	/** Update the view based on changes in the construction view */
	virtual void ConstructionViewSelectionChanged(const TArray<UPrimitiveComponent*>& SelectedComponents, const TArray<FDataflowBaseElement*>& SelectedElements) override;
	
	/** Update the view based on changes in the simulation view */
	virtual void SimulationViewSelectionChanged(const TArray<UPrimitiveComponent*>& SelectedComponents, const TArray<FDataflowBaseElement*>& SelectedElements) override {}

	/** Add GC managed objects*/
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;

	/** Selection View Callbacks */
	void SkeletonViewSelectionChanged(const TArrayView<TSharedPtr<ISkeletonTreeItem>>& InSelectedItems, ESelectInfo::Type InSelectInfo);

	static const FName SkeletonName;

private:
	TSharedPtr<ISkeletonTree> SkeletonEditor;

	/* Skeletal Mesh in the SkeletalViewer*/
	TObjectPtr<USkeletalMesh> SkeletalMesh;

	/* Visualized Skeleton*/
	TObjectPtr<USkeleton> Skeleton;

	/* Rempping from the selected node to the SkeletalMesh*/
	TArray<int32> CollectionIndexRemap;
};

