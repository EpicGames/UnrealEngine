// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowSubGraphPaletteItem.h"

#include "Dataflow/DataflowGraphSchemaAction.h"
#include "DetailLayoutBuilder.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"

void SDataflowSubGraphPaletteItem::Construct(const FArguments& InArgs, FCreateWidgetForActionData* const InCreateData, TSharedPtr<SDataflowGraphEditor> InEditor)
{
	if (InCreateData)
	{
		if (InCreateData->Action || InCreateData->Action->GetTypeId() == FEdGraphSchemaAction_DataflowSubGraph::StaticGetTypeId())
		{
			SubGraphAction = StaticCastSharedPtr<FEdGraphSchemaAction_DataflowSubGraph>(InCreateData->Action);
		}
	}
	SGraphPaletteItem::Construct(SGraphPaletteItem::FArguments(), InCreateData);
}

TSharedRef<SWidget> SDataflowSubGraphPaletteItem::CreateTextSlotWidget(FCreateWidgetForActionData* const InCreateData, TAttribute<bool> bIsReadOnlyIn)
{
	if (SubGraphAction == nullptr)
	{
		return SNullWidget::NullWidget;
	}

	const FText SubGraphName = FText::FromString(SubGraphAction->GetSubGraphName().ToString());

	TSharedPtr<SInlineEditableTextBlock> EditableTextElement;

	// create a type selection widget
	TSharedPtr<SWidget> Widget =
		SNew(SHorizontalBox)

		+SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(FMargin(4,0))
		[
			SNew(SImage)
				.Image(this,&SDataflowSubGraphPaletteItem::GetSubGraphIcon)
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Left)
		[
			SAssignNew(EditableTextElement, SInlineEditableTextBlock)
				.Text(SubGraphName)
				.OnTextCommitted(this, &SDataflowSubGraphPaletteItem::OnNameTextCommitted)
				.OnVerifyTextChanged(this, &SDataflowSubGraphPaletteItem::OnNameTextVerifyChanged)
		]
		;

	InlineRenameWidget = EditableTextElement.ToSharedRef();
	InCreateData->OnRenameRequest->BindSP(InlineRenameWidget.Get(), &SInlineEditableTextBlock::EnterEditingMode);

	return Widget.ToSharedRef();
}

bool SDataflowSubGraphPaletteItem::OnNameTextVerifyChanged(const FText& InNewText, FText& OutErrorMessage)
{
	if (SubGraphAction)
	{
		const bool bCanRename = SubGraphAction->CanRenameItem(InNewText);
		if (!bCanRename)
		{
			OutErrorMessage = FText::FromString(TEXT("Invalid or existing subgraph name"));
		}
		return bCanRename;
	}
	return false;
}

void SDataflowSubGraphPaletteItem::OnNameTextCommitted(const FText& InNewText, ETextCommit::Type InTextCommit)
{
	if (SubGraphAction)
	{
		SubGraphAction->RenameItem(InNewText);
	}
}

const FSlateBrush* SDataflowSubGraphPaletteItem::GetSubGraphIcon() const
{
	if (SubGraphAction && SubGraphAction->IsForEachSubGraph())
	{
		return FAppStyle::GetBrush(TEXT("GraphEditor.Macro.Loop_16x"));
	}
	return FAppStyle::GetBrush(TEXT("GraphEditor.Function_16x")); // default
}