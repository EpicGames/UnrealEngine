// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SGraphPalette.h"

class SDataflowGraphEditor;
struct FEdGraphSchemaAction_DataflowSubGraph;

/**
* Widget for displaying a subgraph entry in SDataflowMembersWidget
*/
class SDataflowSubGraphPaletteItem : public SGraphPaletteItem
{
public:
	SLATE_BEGIN_ARGS(SDataflowSubGraphPaletteItem) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, FCreateWidgetForActionData* const InCreateData, TSharedPtr<SDataflowGraphEditor> InEditor);

private:
	// SGraphPaletteItem Interface
	virtual TSharedRef<SWidget> CreateTextSlotWidget(FCreateWidgetForActionData* const InCreateData, TAttribute<bool> bIsReadOnly) override;
	virtual bool OnNameTextVerifyChanged(const FText& InNewText, FText& OutErrorMessage) override;
	virtual void OnNameTextCommitted(const FText& NewText, ETextCommit::Type InTextCommit) override;
	// End of SGraphPaletteItem Interface

	const FSlateBrush* GetSubGraphIcon() const;

private:
	TSharedPtr<FEdGraphSchemaAction_DataflowSubGraph> SubGraphAction;

	// held for OnRequestRename calls
	TSharedPtr<SInlineEditableTextBlock> InlineRenameWidget;
};
