// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowTemplateRegistry.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Dataflow/DataflowObject.h"
#include "Styling/SlateIconFinder.h"

#define LOCTEXT_NAMESPACE "DataflowTemplateRegistry"

FDataflowTemplateRegistry& FDataflowTemplateRegistry::Get()
{
	static FDataflowTemplateRegistry Instance;
	return Instance;
}

void FDataflowTemplateRegistry::RegisterTemplateAsset(
	const UClass* AssetClass, FDataflowTemplateAssetRegistration Registration)
{
	if (!ensureMsgf(AssetClass, TEXT("FDataflowTemplateRegistry::RegisterTemplateAsset: null AssetClass")) ||
	    !ensureMsgf(Registration.AssetPath.IsValid(), TEXT("FDataflowTemplateRegistry::RegisterTemplateAsset: invalid AssetPath")))
	{
		return;
	}
	AssetRegistrations.FindOrAdd(AssetClass->GetClassPathName()).Add(MoveTemp(Registration));
}

void FDataflowTemplateRegistry::RegisterTemplateFolder(
	const UClass* AssetClass, FDataflowTemplateFolderRegistration Registration)
{
	if (!ensureMsgf(AssetClass, TEXT("FDataflowTemplateRegistry::RegisterTemplateFolder: null AssetClass")))
	{
		return;
	}
	Registrations.FindOrAdd(AssetClass->GetClassPathName()).Add(MoveTemp(Registration));
}

TArray<FDataflowTemplateOption> FDataflowTemplateRegistry::GetTemplateOptions(
	const UClass* AssetClass, bool bAddDefaultBlankOption) const
{
	TArray<FDataflowTemplateOption> Result;

	const FSlateIcon DataflowDefaultIcon(FName("DataflowEditorStyle"), "ClassIcon.Dataflow");

	// "Blank" is always the first option.
	if (bAddDefaultBlankOption)
	{
		FDataflowTemplateOption& Blank = Result.AddDefaulted_GetRef();
		Blank.DisplayName = LOCTEXT("Blank_Name",    "Blank");
		Blank.Tooltip     = LOCTEXT("Blank_Tip",     "Start from an empty Dataflow graph");
		Blank.Icon = DataflowDefaultIcon.GetIcon();
		Blank.TemplateId  = NAME_None;   // NAME_None > caller creates a blank UDataflow
	}

	// Collect individually registered assets (most-derived first), deduplicating by object path.
	TSet<FName> AddedTemplateIds;  // tracks ids already emitted; also used below for folder dedup
	for (const UClass* Class = AssetClass; Class; Class = Class->GetSuperClass())
	{
		const TArray<FDataflowTemplateAssetRegistration>* AssetRegs =
			AssetRegistrations.Find(Class->GetClassPathName());
		if (!AssetRegs)
		{
			continue;
		}
		for (const FDataflowTemplateAssetRegistration& Reg : *AssetRegs)
		{
			const FName TemplateId = FName(Reg.AssetPath.ToString());
			if (AddedTemplateIds.Contains(TemplateId))
			{
				continue;
			}
			AddedTemplateIds.Add(TemplateId);

			// DisplayName: explicit or derived from asset name.
			FText DisplayName = Reg.DisplayName;
			if (DisplayName.IsEmpty())
			{
				FString Derived = Reg.AssetPath.GetAssetName();
				Derived.RemoveFromStart(TEXT("DF_"));
				Derived = Derived.Replace(TEXT("_"), TEXT(" "));
				DisplayName = FText::FromString(Derived);
			}

			// Tooltip: explicit or asset object path.
			FText Tooltip = Reg.Tooltip;
			if (Tooltip.IsEmpty())
			{
				Tooltip = FText::FromString(Reg.AssetPath.ToString());
			}

			// Icon: explicit registration or fallback.
			const FSlateBrush* Icon = (Reg.Icon.IsSet())
				? Reg.Icon.GetIcon()
				: DataflowDefaultIcon.GetIcon();

			FDataflowTemplateOption& Option = Result.AddDefaulted_GetRef();
			Option.DisplayName = DisplayName;
			Option.Tooltip     = Tooltip;
			Option.Icon        = Icon;
			Option.TemplateId  = TemplateId;
		}
	}

	// Walk the class hierarchy (most-derived first) to collect registered folders,
	// preserving insertion order and deduplicating by path.
	TArray<FString>           OrderedFolders;
	TMap<FString, FSlateIcon> FolderToIcon;

	for (const UClass* Class = AssetClass; Class; Class = Class->GetSuperClass())
	{
		const TArray<FDataflowTemplateFolderRegistration>* Regs =
			Registrations.Find(Class->GetClassPathName());
		if (!Regs)
		{
			continue;
		}
		for (const FDataflowTemplateFolderRegistration& Reg : *Regs)
		{
			if (!FolderToIcon.Contains(Reg.ContentFolder))
			{
				// First registration for this folder wins (most-derived class).
				FolderToIcon.Add(Reg.ContentFolder) = Reg.Icon;
				OrderedFolders.Add(Reg.ContentFolder);
			}
		}
	}

	if (OrderedFolders.IsEmpty())
	{
		return Result;
	}

	// Query the asset registry for UDataflow assets in the registered folders.
	IAssetRegistry& AssetRegistry =
		FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();

	FARFilter Filter;
	Filter.ClassPaths.Add(UDataflow::StaticClass()->GetClassPathName());
	Filter.bRecursiveClasses = false;
	Filter.bRecursivePaths   = true;
	for (const FString& Folder : OrderedFolders)
	{
		Filter.PackagePaths.Add(FName(*Folder));
	}

	TArray<FAssetData> FoundAssets;
	AssetRegistry.GetAssets(Filter, FoundAssets);

	// Sort by folder order first, then by asset name, so results are stable
	// regardless of asset registry enumeration order.
	FoundAssets.Sort([&OrderedFolders](const FAssetData& A, const FAssetData& B)
	{
		const FString PathA = A.PackagePath.ToString();
		const FString PathB = B.PackagePath.ToString();
		const int32 FolderA = OrderedFolders.IndexOfByPredicate(
			[&PathA](const FString& F){ return PathA.StartsWith(F); });
		const int32 FolderB = OrderedFolders.IndexOfByPredicate(
			[&PathB](const FString& F){ return PathB.StartsWith(F); });
		if (FolderA != FolderB)
		{
			return FolderA < FolderB;
		}
		return A.AssetName.LexicalLess(B.AssetName);
	});

	for (const FAssetData& AssetData : FoundAssets)
	{
		// Skip assets already emitted as individually registered templates.
		const FName TemplateId = FName(AssetData.GetObjectPathString());
		if (AddedTemplateIds.Contains(TemplateId))
		{
			continue;
		}
		AddedTemplateIds.Add(TemplateId);

		// Resolve icon from the folder this asset lives under.
		const FString AssetPackagePath = AssetData.PackagePath.ToString();
		const FSlateIcon* FoundIcon = nullptr;
		for (const FString& Folder : OrderedFolders)
		{
			if (AssetPackagePath.StartsWith(Folder))
			{
				FoundIcon = FolderToIcon.Find(Folder);
				break;
			}
		}

		const FSlateBrush* Icon = (FoundIcon && FoundIcon->IsSet())
			? FoundIcon->GetIcon()
			: DataflowDefaultIcon.GetIcon();

		// Build a human-readable display name: strip leading "DF_", replace underscores.
		FString DisplayName = AssetData.AssetName.ToString();
		DisplayName.RemoveFromStart(TEXT("DF_"));
		DisplayName = DisplayName.Replace(TEXT("_"), TEXT(" "));

		FDataflowTemplateOption& Option = Result.AddDefaulted_GetRef();
		Option.DisplayName = FText::FromString(DisplayName);
		Option.Tooltip     = FText::FromString(AssetData.GetObjectPathString());
		Option.Icon        = Icon;
		// Full object path stored in TemplateId so callers can LoadObject<UDataflow> directly.
		Option.TemplateId  = TemplateId;
	}

	return Result;
}

#undef LOCTEXT_NAMESPACE
