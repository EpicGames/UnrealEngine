// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowToolRegistry.h"
#include "Misc/LazySingleton.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Dataflow/DataflowEditorCommands.h"

namespace UE::Dataflow
{

	FDataflowToolRegistry& FDataflowToolRegistry::Get()
	{
		return TLazySingleton<FDataflowToolRegistry>::Get();
	}

	void FDataflowToolRegistry::TearDown()
	{
		TLazySingleton<FDataflowToolRegistry>::TearDown();
	}

	void FDataflowToolRegistry::AddNodeToToolMapping(const FName& NodeName, TObjectPtr<UInteractiveToolBuilder> ToolBuilder, const TSharedRef<const IDataflowToolActionCommands>& ToolActionCommands,
		const FSlateIcon& AddNodeButtonIcon, const FText& AddNodeButtonText, const FName& ToolCategory, const FName& AddNodeConnectionType, const FName& AddNodeConnectionName)
	{
//#if WITH_EDITOR
//		ensureMsgf(!FDataflowEditorCommands::IsRegistered(), TEXT("FDataflowToolRegistry: DataflowEditorCommands have already been registered. \
//				Newly registered Tools may not be available in the Editor. \
//				Ensure that AddNodeToToolMapping is called before the DataflowEditor module is loaded."));
//#endif
		// FUICommandInfo is uninitialized, it will be created later in FDataflowEditorCommandsImpl::RegisterCommands
		NodeTypeToToolMap.Add(NodeName, FToolInfo{ ToolBuilder, ToolActionCommands, AddNodeButtonIcon, AddNodeButtonText, AddNodeConnectionType, AddNodeConnectionName, nullptr, nullptr, ToolCategory });
	}

	void FDataflowToolRegistry::RemoveNodeToToolMapping(const FName& NodeName)
	{
		NodeTypeToToolMap.Remove(NodeName);
	}

	TArray<FName> FDataflowToolRegistry::GetNodeNames() const
	{
		TArray<FName> NodeNames;
		NodeTypeToToolMap.GetKeys(NodeNames);
		return NodeNames;
	}
	
	TSharedPtr<FUICommandInfo>& FDataflowToolRegistry::GetAddNodeCommandForNode(const FName& NodeType)
	{
		check(NodeTypeToToolMap.Contains(NodeType));
		return NodeTypeToToolMap[NodeType].AddNodeCommand;
	}

	const FSlateIcon& FDataflowToolRegistry::GetAddNodeButtonIcon(const FName& NodeType) const
	{
		check(NodeTypeToToolMap.Contains(NodeType));
		return NodeTypeToToolMap[NodeType].AddNodeButtonIcon;
	}

	const FText& FDataflowToolRegistry::GetAddNodeButtonText(const FName& NodeType) const
	{
		check(NodeTypeToToolMap.Contains(NodeType));
		return NodeTypeToToolMap[NodeType].AddNodeButtonText;
	}

	TSharedPtr<FUICommandInfo>& FDataflowToolRegistry::GetToolCommandForNode(const FName& NodeName)
	{
		check(NodeTypeToToolMap.Contains(NodeName));
		return NodeTypeToToolMap[NodeName].ToolCommand;
	}

	const FName& FDataflowToolRegistry::GetToolCategoryForNode(const FName& NodeName) const
	{
		check(NodeTypeToToolMap.Contains(NodeName));
		return NodeTypeToToolMap[NodeName].ToolCategory;
	}

	const FName& FDataflowToolRegistry::GetAddNodeConnectionType(const FName& NodeName) const
	{
		check(NodeTypeToToolMap.Contains(NodeName));
		return NodeTypeToToolMap[NodeName].AddNodeConnectionType;
	}

	const FName& FDataflowToolRegistry::GetAddNodeConnectionName(const FName& NodeName) const
	{
		check(NodeTypeToToolMap.Contains(NodeName));
		return NodeTypeToToolMap[NodeName].AddNodeConnectionName;
	}

	UInteractiveToolBuilder* FDataflowToolRegistry::GetToolBuilderForNode(const FName& NodeName)
	{
		check(NodeTypeToToolMap.Contains(NodeName));
		return NodeTypeToToolMap[NodeName].ToolBuilder;
	}

	const UInteractiveToolBuilder* FDataflowToolRegistry::GetToolBuilderForNode(const FName& NodeName) const
	{
		check(NodeTypeToToolMap.Contains(NodeName));
		return NodeTypeToToolMap[NodeName].ToolBuilder;
	}

	void FDataflowToolRegistry::UnbindActiveCommands(const TSharedPtr<FUICommandList>& UICommandList) const
	{
		for (const TPair<FName, FToolInfo>& Entry : NodeTypeToToolMap)
		{
			Entry.Value.ToolActionCommands->UnbindActiveCommands(UICommandList);
		}
	}

	void FDataflowToolRegistry::BindCommandsForCurrentTool(const TSharedPtr<FUICommandList>& UICommandList, UInteractiveTool* Tool) const
	{
		for (const TPair<FName, FToolInfo>& Entry : NodeTypeToToolMap)
		{
			Entry.Value.ToolActionCommands->BindCommandsForCurrentTool(UICommandList, Tool);
		}
	}

}
