// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowToolTarget.h"

#include "ConversionUtils/DynamicMeshViaMeshDescriptionUtil.h"
#include "Dataflow/CollectionRenderingPatternUtility.h"
#include "Dataflow/DataflowEditor.h"
#include "Dataflow/DataflowContent.h"
#include "Dataflow/DataflowEditorToolkit.h"
#include "Dataflow/DataflowEditorUtil.h"
#include "Dataflow/DataflowEdNode.h"
#include "Dataflow/DataflowObject.h"
#include "Dataflow/DataflowObjectInterface.h"
#include "Dataflow/DataflowNodeParameters.h"
#include "DynamicMeshToMeshDescription.h"
#include "GeometryCollection/Facades/CollectionRenderingFacade.h"
#include "Materials/Material.h"
#include "MeshDescriptionToDynamicMesh.h"
#include "MaterialDomain.h"
#include "StaticMeshAttributes.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowToolTarget)



//
// UDataflowReadOnlyToolTarget
//

bool UDataflowReadOnlyToolTarget::IsValid() const
{
	return Dataflow != nullptr && Asset != nullptr;
}

int32 UDataflowReadOnlyToolTarget::GetNumMaterials() const
{
	return (ensure(IsValid()) && Dataflow->Material) ? 1 : 0;
}

UMaterialInterface* UDataflowReadOnlyToolTarget::GetMaterial(int32 MaterialIndex) const
{
	return (ensure(IsValid()) &&  MaterialIndex == 0)? (UMaterialInterface*)Dataflow->Material : nullptr;
}

void UDataflowReadOnlyToolTarget::GetMaterialSet(FComponentMaterialSet& MaterialSetOut, bool bPreferAssetMaterials) const
{
	if (!ensure(IsValid())) return;
	if (Dataflow->Material)
	{
		MaterialSetOut.Materials.SetNum(1);
		MaterialSetOut.Materials[0] = (UMaterialInterface*)Dataflow->Material;
	}
}

bool UDataflowReadOnlyToolTarget::CommitMaterialSetUpdate(const FComponentMaterialSet& MaterialSet, bool bApplyToAsset)
{
	// filter out any Engine materials that we don't want to be permanently assigning
	TArray<UMaterialInterface*> FilteredMaterials = MaterialSet.Materials;
	for (int32 k = 0; k < FilteredMaterials.Num(); ++k)
	{
		FString AssetPath = FilteredMaterials[k]->GetPathName();
		if (AssetPath.StartsWith(TEXT("/MeshModelingToolsetExp/")))
		{
			FilteredMaterials[k] = UMaterial::GetDefaultMaterial(MD_Surface);
		}
	}

	if(IsValid() && FilteredMaterials.Num() > 0)
	{
		if (UMaterial* Material = Cast<UMaterial>(FilteredMaterials[0]))
		{
			Dataflow->Material = Material;
		}
		return true;
	}
	return false;
}

const FMeshDescription* UDataflowReadOnlyToolTarget::GetMeshDescription(const FGetMeshParameters& GetMeshParams)
{
	if (ensure(IsValid()) && !DataflowMeshDescription.IsValid())
	{
		DataflowMeshDescription = MakeUnique<FMeshDescription>();
		FStaticMeshAttributes Attributes(*DataflowMeshDescription);
		Attributes.Register();

		const FDynamicMesh3 DynamicMesh = GetDynamicMesh();
		
		FDynamicMeshToMeshDescription Converter;
		Converter.Convert(&DynamicMesh, *DataflowMeshDescription, true);
	}
	return DataflowMeshDescription.Get();
}

FMeshDescription UDataflowReadOnlyToolTarget::GetEmptyMeshDescription()
{
	FMeshDescription EmptyMeshDescription;
	FStaticMeshAttributes MeshAttributes(EmptyMeshDescription);
	MeshAttributes.Register();

	return EmptyMeshDescription;
}

FDynamicMesh3 UDataflowReadOnlyToolTarget::GetDynamicMesh()
{
	FDynamicMesh3 DynamicMesh;
	if(IsValid())
	{
		// @todo(dataflow)
	}
	return DynamicMesh;
}

//
// UDataflowToolTarget
//

void UDataflowToolTarget::CommitMeshDescription(const FCommitter& Committer, const FCommitMeshParameters& CommitMeshParams)
{
	if (ensure(IsValid()) == false) return;

	// Let the user fill our mesh description with the Committer
	if (!DataflowMeshDescription.IsValid())
	{
		DataflowMeshDescription = MakeUnique<FMeshDescription>();
		FStaticMeshAttributes Attributes(*DataflowMeshDescription);
		Attributes.Register();
	}
	FCommitterParams CommitParams;
	CommitParams.MeshDescriptionOut = DataflowMeshDescription.Get();
	Committer(CommitParams);

	// The conversion we have right now is from dynamic mesh to volume, so we convert
	// to dynamic mesh first.
	FDynamicMesh3 DynamicMesh;
	FMeshDescriptionToDynamicMesh Converter;
	Converter.Convert(DataflowMeshDescription.Get(), DynamicMesh);

	CommitDynamicMesh(DynamicMesh);
}

void UDataflowToolTarget::CommitDynamicMesh(const FDynamicMesh3& DynamicMesh, const FDynamicMeshCommitInfo& CommitInfo)
{
	if(IsValid())
	{
		// @todo(dataflow)
	}
}

// Factory

bool UDataflowReadOnlyToolTargetFactory::CanBuildTarget(UObject* SourceObject, const FToolTargetTypeRequirements& Requirements) const
{
	// We are using an exact cast here to prevent subclasses, which might not meet all
	// requirements for functionality such as the deprecated DestructibleMeshComponent, from 
	// being caught up as valid targets.
	// If you want to make the tool target work with some subclass of UDataflow,
	// just add another factory that allows that class specifically(but make sure that
	// GetMeshDescription and such work properly)
	const TObjectPtr<UDataflowBaseContent> BaseContent = Cast<UDataflowBaseContent>(SourceObject);
	if(BaseContent)
	{
		const UDataflow* Dataflow = BaseContent->GetDataflowAsset();
	
		return Dataflow &&
			ExactCast<UDataflow>(Cast<UDataflow>(Dataflow)) &&
			!ExactCast<UDataflow>(Cast<UDataflow>(Dataflow))->GetOutermost()->bIsCookedForEditor &&
			Requirements.AreSatisfiedBy(UDataflowReadOnlyToolTarget::StaticClass());
	}
	return false;
}

UToolTarget* UDataflowReadOnlyToolTargetFactory::BuildTarget(UObject* SourceObject, const FToolTargetTypeRequirements& Requirements)
{
	const TObjectPtr<UDataflowBaseContent> BaseContent = CastChecked<UDataflowBaseContent>(SourceObject);

	if(BaseContent)
	{
		UDataflowReadOnlyToolTarget* Target = NewObject<UDataflowReadOnlyToolTarget>();
		Target->Asset = BaseContent->GetDataflowOwner();
		Target->Dataflow = BaseContent->GetDataflowAsset();
		Target->Context = UE::Dataflow::GetContext(BaseContent);

		// @todo(brice) : I needed to comment this out?
		//checkSlow(Target->Component.IsValid() && Requirements.AreSatisfiedBy(Target));

		return Target;
	}

	return nullptr;
}

bool UDataflowToolTargetFactory::CanBuildTarget(UObject* SourceObject, const FToolTargetTypeRequirements& Requirements) const
{
	// We are using an exact cast here to prevent subclasses, which might not meet all
	// requirements for functionality such as the deprecated DestructibleMeshComponent, from 
	// being caught up as valid targets.
	// If you want to make the tool target work with some subclass of UDataflow,
	// just add another factory that allows that class specifically(but make sure that
	// GetMeshDescription and such work properly)

	const TObjectPtr<UDataflowBaseContent> BaseContent = Cast<UDataflowBaseContent>(SourceObject);
	if(BaseContent)
	{
		const UDataflow* Dataflow = BaseContent->GetDataflowAsset();
		
		return Dataflow &&
			ExactCast<UDataflow>(Cast<UDataflow>(Dataflow)) &&
			!ExactCast<UDataflow>(Cast<UDataflow>(Dataflow))->GetOutermost()->bIsCookedForEditor &&
			Requirements.AreSatisfiedBy(UDataflowToolTarget::StaticClass());
	}
	return false;
}

UToolTarget* UDataflowToolTargetFactory::BuildTarget(UObject* SourceObject, const FToolTargetTypeRequirements& Requirements)
{
	const TObjectPtr<UDataflowBaseContent> BaseContent = CastChecked<UDataflowBaseContent>(SourceObject);

	if(BaseContent)
	{
		UDataflowToolTarget* Target = NewObject<UDataflowToolTarget>();
		Target->Asset = BaseContent->GetDataflowOwner();
		Target->Dataflow = BaseContent->GetDataflowAsset();
		Target->Context = UE::Dataflow::GetContext(BaseContent);
		//checkSlow(Target->Component.IsValid() && Requirements.AreSatisfiedBy(Target));

		return Target;
	}
	return nullptr;
}

