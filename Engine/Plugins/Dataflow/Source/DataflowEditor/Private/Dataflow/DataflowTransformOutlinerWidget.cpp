// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowTransformOutlinerWidget.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Dataflow/DataflowSettings.h"
#include "GeometryCollection/GeometryCollection.h"
#include "Dataflow/DataflowCollectionSpreadSheetHelpers.h"
#include "ToolMenus.h"

#define LOCTEXT_NAMESPACE "TransformOutliner"

// --- FDataflowOutlinerTreeItem ---

void FDataflowOutlinerTreeItem::GenerateContextMenu(UToolMenu* Menu, STransformOutliner& Outliner)
{
	auto SharedOutliner = StaticCastSharedRef<STransformOutliner>(Outliner.AsShared());

	FToolMenuSection& Section = Menu->AddSection("Section");
	Section.AddMenuEntry("ExpandAll", LOCTEXT("ExpandAll", "Expand All Levels"), FText(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Visible"), FUIAction(FExecuteAction::CreateRaw(&Outliner, &STransformOutliner::ContextMenuExpandAll)));
	Section.AddMenuEntry("PreviousLevel", LOCTEXT("PreviousLevel", "Decrement Display Level"), FText(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Minus"), FUIAction(FExecuteAction::CreateRaw(&Outliner, &STransformOutliner::ContextMenuPreviousLevel)));
	Section.AddMenuEntry("NextLevel", LOCTEXT("NextLevel", "Increment Display Level"), FText(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Plus"), FUIAction(FExecuteAction::CreateRaw(&Outliner, &STransformOutliner::ContextMenuNextLevel)));
}

// for now use the transform group as this simplify the attribute copy 
static const FName DataCollectionGroup = FGeometryCollection::TransformGroup;

template <typename T>
static T GetAttributeValue(const TManagedArrayAccessor<T>& Attribute, int32 Index, T Default)
{
	return (Attribute.IsValid()) ? Attribute.Get()[Index] : Default;
}

// --- FDataflowOutlinerTreeItemCollection ---

TSharedRef<ITableRow> FDataflowOutlinerTreeItemCollection::MakeTreeRowWidget(const TSharedRef<STableViewBase>& InOwnerTable, bool bNoExtraColumn)
{
	FString Output = OutputName.ToString();
	FString NumVertsString = "";
	FString NumFacesString = "";

	if (Collection.IsValid())
	{
		const int32 NumVerts = Collection->NumElements(FGeometryCollection::VerticesGroup);
		NumVertsString = FString::FormatAsNumber(NumVerts);
		const int32 NumFaces = Collection->NumElements(FGeometryCollection::FacesGroup);
		NumFacesString = FString::FormatAsNumber(NumFaces);
	}

	return SNew(STableRow<FDataflowOutlinerTreeItemPtr>, InOwnerTable)
		.Content()
		[
			SNew(STextBlock).Text(FText::FromString(Output + "      Faces: " + NumFacesString + "    Verts: " + NumVertsString))
		];
}

void FDataflowOutlinerTreeItemCollection::GetChildren(FDataflowOutlinerTreeItemList& OutChildren)
{
	OutChildren = ChildItems;
}

FDataflowOutlinerTreeItemPtr FDataflowOutlinerTreeItemCollection::GetItemFromBoneIndex(int32 BoneIndex) const
{
	return ItemsByBoneIndex.FindRef(BoneIndex);
}

void FDataflowOutlinerTreeItemCollection::GetChildrenForBone(FDataflowOutlinerTreeItemBone& BoneItem, FDataflowOutlinerTreeItemList& OutChildren) const
{
	if (!Collection.IsValid())
	{
		return;
	}

	if (Collection->HasAttribute("Children", FGeometryCollection::TransformGroup))
	{
		const int32 BoneIndex = BoneItem.GetBoneIndex();
		if (ensure(BoneIndex >= 0 && BoneIndex < Collection->NumElements(FGeometryCollection::TransformGroup)))
		{
			const TManagedArray<TSet<int32>>& Children = Collection->GetAttribute<TSet<int32>>("Children", FGeometryCollection::TransformGroup);
			for (const int32 ChildIndex : Children[BoneIndex])
			{
				FDataflowOutlinerTreeItemPtr ChildPtr = ItemsByBoneIndex.FindRef(ChildIndex);
				if (ChildPtr.IsValid())
				{
					OutChildren.Add(ChildPtr);
				}
			}
		}
	}
}

bool FDataflowOutlinerTreeItemCollection::HasChildrenForBone(const FDataflowOutlinerTreeItemBone& BoneItem) const
{
	if (!Collection.IsValid())
	{
		return false;
	}

	if (Collection->HasAttribute("Children", FGeometryCollection::TransformGroup))
	{
		const int32 BoneIndex = BoneItem.GetBoneIndex();
		if (ensure(BoneIndex >= 0 && BoneIndex < Collection->NumElements(FGeometryCollection::TransformGroup)))
		{
			const TManagedArray<TSet<int32>>& Children = Collection->GetAttribute<TSet<int32>>("Children", FGeometryCollection::TransformGroup);
			return Children[(BoneIndex)].Num() > 0;
		}
	}

	return false;
}

void FDataflowOutlinerTreeItemCollection::ExpandAll()
{
	TreeView->SetItemExpansion(AsShared(), true);

	for (const auto& Elem : ItemsByBoneIndex)
	{
		TreeView->SetItemExpansion(Elem.Value, true);
	}
}

void FDataflowOutlinerTreeItemCollection::ExpandByLevel(const int32 InLevel)
{
	if (Collection->HasAttribute("Level", DataCollectionGroup))
	{
		const TManagedArray<int32>& Levels = Collection->GetAttribute<int32>("Level", DataCollectionGroup);

		TreeView->SetItemExpansion(AsShared(), true);

		for (const auto& Elem : ItemsByBoneIndex)
		{
			const int32 BoneIndex = Elem.Key;

			if (Levels[BoneIndex] < InLevel)
			{
				TreeView->SetItemExpansion(Elem.Value, true);
			}
			else
			{
				TreeView->SetItemExpansion(Elem.Value, false);
			}
		}
	}
}

void FDataflowOutlinerTreeItemCollection::RegenerateChildren()
{
	if (Collection.IsValid())
	{
		ItemsByBoneIndex.Empty();
		ChildItems.Empty();

		if (Collection)
		{
			if (Collection->HasAttribute("Parent", FGeometryCollection::TransformGroup))
			{
				const int32 NumElements = Collection->NumElements(FGeometryCollection::TransformGroup);
				const TManagedArray<int32>& Parents = Collection->GetAttribute<int32>("Parent", FGeometryCollection::TransformGroup);

				RootIndex = FGeometryCollection::Invalid;

				// Add a sub item to the outliner tree for each of the bones/chunks in this GeometryCollection
					for (int32 BoneIndex = 0; BoneIndex < NumElements; BoneIndex++)
					{
						if (FilterBoneIndex(BoneIndex))
						{
							TSharedRef<FDataflowOutlinerTreeItemBone> NewItem = MakeShared<FDataflowOutlinerTreeItemBone>(BoneIndex, this);
							if (Parents[BoneIndex] == RootIndex)
							{
								// The actual children directly beneath this node are the ones without a parent.  The rest are children of children
								ChildItems.Add(NewItem);
							}

							ItemsByBoneIndex.Add(BoneIndex, NewItem);
						}
					}
			}
		}
	}
}

void FDataflowOutlinerTreeItemCollection::RequestTreeRefresh()
{
	if (TreeView.IsValid())
	{
		TreeView->RequestTreeRefresh();
	}
}

bool FDataflowOutlinerTreeItemCollection::FilterBoneIndex(int32 BoneIndex) const
{
	if (Collection.IsValid())
	{
		if (Collection->HasAttribute("SimulationType", FGeometryCollection::TransformGroup) &&
			Collection->HasAttribute("Children", FGeometryCollection::TransformGroup))
		{
			const TManagedArray<int32>& SimTypes = Collection->GetAttribute<int32>("SimulationType", FGeometryCollection::TransformGroup);
			const TManagedArray<TSet<int32>>& Children = Collection->GetAttribute<TSet<int32>>("Children", FGeometryCollection::TransformGroup);

			bool bHasChildren = Children[BoneIndex].Num() > 0;

			if (SimTypes[BoneIndex] != FGeometryCollection::ESimulationTypes::FST_Clustered)
			{
				// We only display cluster nodes deeper than the view level.
				const int32 FractureLevel = -1; // Make all the levels visible

				if (FractureLevel >= 0 && Collection->HasAttribute("Level", FTransformCollection::TransformGroup))
				{
					const TManagedArray<int32>& Level = Collection->GetAttribute<int32>("Level", FTransformCollection::TransformGroup);
					int32 BoneLevel = Level[BoneIndex];
					// bone is not at the right level itself and doesn't have child(ren) at the right level
					if (BoneLevel != FractureLevel && (!bHasChildren || BoneLevel + 1 != FractureLevel))
					{
						return false;
					}
				}
			}
		}
	}

	return true;
}

bool FDataflowOutlinerTreeItemCollection::IsValid() const
{
	if (bInvalidated)
	{
		return false;
	}

	if (Collection)
	{
		return true;
	}

	return false;
}

// --- FDataflowOutlinerTreeItemBone ---


TSharedPtr<const FManagedArrayCollection> FDataflowOutlinerTreeItemBone::GetCollection() const
{
	ensure(ParentCollectionItem);
	return ParentCollectionItem->GetCollection();
}

void FDataflowOutlinerTreeItemBone::UpdateItemColorFromCollection()
{
	ItemColor = UE::Dataflow::CollectionSpreadSheetHelpers::UpdateItemColorFromCollection(GetCollection(), FGeometryCollection::TransformGroup, GetBoneIndex());
}

TSharedRef<ITableRow> FDataflowOutlinerTreeItemBone::MakeTreeRowWidget(const TSharedRef<STableViewBase>& InOwnerTable, bool bIsPinned)
{
	UpdateItemColorFromCollection();

	TSharedPtr<const FManagedArrayCollection> Collection = GetCollection();

	// This gets displayed in the pinned rows
	if (bIsPinned)
	{
		return SNew(STableRow<FDataflowOutlinerTreeItemPtr>, InOwnerTable)
			.Content()
			[
				SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.Padding(2.0f, 4.0f)
					.AutoWidth()
					[
						SNew(STextBlock)
						.Text(FText::AsNumber(BoneIndex))
						.ColorAndOpacity(ItemColor)
					]
					+ SHorizontalBox::Slot()
					.Padding(2.0f, 4.0f)
					.AutoWidth()
					[
						SNew(STextBlock)
						.Text(FText::FromString(Collection->GetAttribute<FString>("BoneName", DataCollectionGroup)[BoneIndex]))
						.ColorAndOpacity(ItemColor)
					]
			];
	}

	return SNew(STransformOutlinerRow, InOwnerTable, SharedThis(this));
}

bool FDataflowOutlinerTreeItemBone::IsValidBone() const
{
	TSharedPtr<const FManagedArrayCollection> Collection = GetCollection();

	if (ParentCollectionItem && ParentCollectionItem->IsValid())
	{
		return BoneIndex >= 0 && BoneIndex < Collection->NumElements(DataCollectionGroup);
	}
	return false;
}

TSharedRef<SWidget> FDataflowOutlinerTreeItemBone::MakeIndexColumnWidget() const
{
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(STextBlock)
			.Text(FText::AsNumber(BoneIndex))
			.ColorAndOpacity(ItemColor)
		];
}

TSharedRef<SWidget> FDataflowOutlinerTreeItemBone::MakeEmptyColumnWidget() const
{
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.Padding(12.f, 0.f)
		[
			SNew(STextBlock)
			.ColorAndOpacity(ItemColor)
		];
}

void FDataflowOutlinerTreeItemBone::GetChildren(FDataflowOutlinerTreeItemList& OutChildren)
{
	ParentCollectionItem->GetChildrenForBone(*this, OutChildren);
}

bool FDataflowOutlinerTreeItemBone::HasChildren() const
{
	return ParentCollectionItem->HasChildrenForBone(*this);
}

// --- STransformOutlinerRow ---

TSharedRef<SWidget> STransformOutlinerRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	TSharedPtr<const FManagedArrayCollection> Collection = Item->GetCollection();
	FName AttrType = UE::Dataflow::CollectionSpreadSheetHelpers::GetArrayTypeString(Collection->GetAttributeType(ColumnName, DataCollectionGroup));

	// This can happen because sometimes slate retains old items until the next tick, and keeps calling callbacks on them until then
	if (!Item->IsValidBone())
	{
		return Item->MakeEmptyColumnWidget();
	}

	if (ColumnName == "Index")
	{
		const TSharedPtr<SWidget> NameWidget = Item->MakeIndexColumnWidget();
		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(SExpanderArrow, SharedThis(this))
					.ShouldDrawWires(true)
			]
			+ SHorizontalBox::Slot()
			[
				NameWidget.ToSharedRef()
			];
	}

	return UE::Dataflow::CollectionSpreadSheetHelpers::MakeColumnWidget(Collection,
		FGeometryCollection::TransformGroup,
		ColumnName,
		Item->GetBoneIndex(),
		Item->GetItemColor());
}

// --- STransformOutliner ---

void STransformOutliner::Construct(const FArguments& InArgs)
{
	bPerformingSelection = false;

	HeaderRowWidget =
		SNew(SHeaderRow)
		.Visibility(EVisibility::Visible);

	RegenerateHeader();

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.Padding(FMargin(0.0f, 3.f))
		[
			SNew(SOverlay)
			+ SOverlay::Slot()
			[
				SAssignNew(TreeView, STreeView<FDataflowOutlinerTreeItemPtr>)
				.TreeItemsSource(reinterpret_cast<FDataflowOutlinerTreeItemList*>(&RootNodes))
				.OnSelectionChanged(this, &STransformOutliner::OnSelectionChanged)
				.OnGenerateRow(this, &STransformOutliner::MakeTreeRowWidget)
				.OnGetChildren(this, &STransformOutliner::OnGetChildren)
				.OnContextMenuOpening(this, &STransformOutliner::OnOpenContextMenu)
				.AllowInvisibleItemSelection(true)
				.ShouldStackHierarchyHeaders(true)
				.OnGeneratePinnedRow(this, &STransformOutliner::OnGeneratePinnedRowWidget, true)
				.HighlightParentNodesForSelection(true)
				.OnSetExpansionRecursive(this, &STransformOutliner::ExpandRecursive)
				.HeaderRow(HeaderRowWidget)
				.ExternalScrollbar(InArgs._ExternalVerticalScrollBar)
			]
		]
	];
}

void STransformOutliner::RegenerateHeader()
{
	constexpr int32 CustomColumnWidth = 120;
	constexpr int32 CWidthMult = 9;

	HeaderRowWidget->ClearColumns();

	if (!Collection.IsValid() || Collection->NumElements(DataCollectionGroup) == 0)
	{
		return;
	}

	TArray<UE::Dataflow::CollectionSpreadSheetHelpers::FAttrInfo> AttrInfo;

	// Add "Index" manually
	AttrInfo.Add({ "Index", "int32" });
	// Add all the other attrs
	for (FName Attr : Collection->AttributeNames(DataCollectionGroup))
	{
		AttrInfo.Add({ Attr, UE::Dataflow::CollectionSpreadSheetHelpers::GetArrayTypeString(Collection->GetAttributeType(Attr, DataCollectionGroup)).ToString()});
	}

	for (int32 Idx = 0; Idx < AttrInfo.Num(); ++Idx)
	{
		int32 ColumnWidth = CustomColumnWidth;
		const FString ColumnNameStr = AttrInfo[Idx].Name.ToString();

		if (ColumnNameStr == "Index")
		{
			ColumnWidth = CustomColumnWidth;

			HeaderRowWidget->AddColumn(
				SHeaderRow::Column(AttrInfo[Idx].Name)
				.DefaultLabel(FText::FromName(AttrInfo[Idx].Name))
				.ManualWidth(ColumnWidth)
				.HAlignCell(HAlign_Left)
				.HAlignHeader(HAlign_Center)
				.VAlignCell(VAlign_Center));
		}
		else
		{
			int32 ColumnNameStrLen = ColumnNameStr.Len() * CWidthMult;

			if (UE::Dataflow::CollectionSpreadSheetHelpers::AttrTypeWidthMap.Contains(AttrInfo[Idx].Type))
			{
				ColumnWidth = UE::Dataflow::CollectionSpreadSheetHelpers::AttrTypeWidthMap[AttrInfo[Idx].Type];
			}

			ColumnWidth = ColumnNameStrLen > ColumnWidth ? ColumnNameStrLen : ColumnWidth;

			HeaderRowWidget->AddColumn(
				SHeaderRow::Column(AttrInfo[Idx].Name)
				.DefaultLabel(FText::FromName(AttrInfo[Idx].Name))
				.ManualWidth(ColumnWidth)
				.HAlignCell(HAlign_Center)
				.HAlignHeader(HAlign_Center)
				.VAlignCell(VAlign_Center));
		}
	}
}

void STransformOutliner::RegenerateItems()
{
	TreeView->RebuildList();
}

TSharedRef<ITableRow> STransformOutliner::MakeTreeRowWidget(FDataflowOutlinerTreeItemPtr InItem, const TSharedRef<STableViewBase>& InOwnerTable)
{
	return InItem->MakeTreeRowWidget(InOwnerTable);
}

TSharedRef<ITableRow> STransformOutliner::OnGeneratePinnedRowWidget(FDataflowOutlinerTreeItemPtr InItem, const TSharedRef<STableViewBase>& InOwnerTable, bool bPinned)
{
	return InItem->MakeTreeRowWidget(InOwnerTable, true);
}

void STransformOutliner::OnGetChildren(FDataflowOutlinerTreeItemPtr InItem, TArray<FDataflowOutlinerTreeItemPtr>& OutChildren)
{
	InItem->GetChildren(OutChildren);
}

TSharedPtr<SWidget> STransformOutliner::OnOpenContextMenu()
{
	FDataflowOutlinerTreeItemList SelectedItems;
	TreeView->GetSelectedItems(SelectedItems);

	if (SelectedItems.Num())
	{
		UToolMenus* ToolMenus = UToolMenus::Get();
		static const FName MenuName = "STransformOutliner.TransformOutlinerContextMenu";
		if (!ToolMenus->IsMenuRegistered(MenuName))
		{
			ToolMenus->RegisterMenu(MenuName);
		}

		// Build up the menu for a selection
		FToolMenuContext Context;
		UToolMenu* Menu = ToolMenus->GenerateMenu(MenuName, Context);
		SelectedItems[0]->GenerateContextMenu(Menu, *this);
		return ToolMenus->GenerateWidget(Menu);
	}

	return TSharedPtr<SWidget>();
}

void STransformOutliner::UpdateGeometryCollection()
{
	TreeView->RequestTreeRefresh();
	ExpandAll();
}

void STransformOutliner::SetCollection(const TSharedPtr<const FManagedArrayCollection>& InCollection, const FName InOutputName)
{
	if (InCollection == Collection)
	{
		return;
	}

	// Clear the cached Tree ItemSelection without affecting the SelectedBones as 
	// we want to refresh the tree selection using selected bones
	TGuardValue<bool> ExternalSelectionGuard(bPerformingSelection, true);
	TreeView->ClearSelection();

	// explicitly mark the root nodes as invalid before emptying, so we know we can safely ignore them in case slate still triggers callbacks for them (they will not be deleted until the tree view refresh, on tick)
	for (TSharedPtr<FDataflowOutlinerTreeItemCollection>& RootNode : RootNodes)
	{
		if (RootNode)
		{
			RootNode->Invalidate();
		}
	}
	RootNodes.Empty();

	RootNodes.Add(MakeShared<FDataflowOutlinerTreeItemCollection>(InCollection, InOutputName, TreeView));

	TreeView->RequestTreeRefresh();
	ExpandAll();

	Collection = InCollection;
}

void STransformOutliner::ExpandAll()
{
	for (TSharedPtr<FDataflowOutlinerTreeItemCollection> ItemPtr : RootNodes)
	{
		ItemPtr->ExpandAll();
	}
}

void STransformOutliner::ExpandByLevel(const int32 InLevel)
{
	for (TSharedPtr<FDataflowOutlinerTreeItemCollection> ItemPtr : RootNodes)
	{
		ItemPtr->ExpandByLevel(InLevel);
	}
}

void STransformOutliner::ExpandRecursive(TSharedPtr<FDataflowOutlinerTreeItem> ItemPtr, bool bInExpansionState) const
{
	TreeView->SetItemExpansion(ItemPtr, bInExpansionState);

	FDataflowOutlinerTreeItemList ItemChildren;
	ItemPtr->GetChildren(ItemChildren);
	for (auto& Child : ItemChildren)
	{
		ExpandRecursive(Child, bInExpansionState);
	}
}

int32 STransformOutliner::GetBoneSelectionCount() const
{
	return TreeView->GetSelectedItems().Num();
}

void STransformOutliner::OnSelectionChanged(FDataflowOutlinerTreeItemPtr Item, ESelectInfo::Type SelectInfo)
{
}

void STransformOutliner::ContextMenuExpandAll()
{
	DisplayLevel = -1;

	ExpandAll();
}

void STransformOutliner::ContextMenuPreviousLevel()
{
	if (DisplayLevel == -1)
	{
		return;
	}

	DisplayLevel--;

	if (DisplayLevel == -1)
	{
		ExpandAll();
	}
	else
	{
		ExpandByLevel(DisplayLevel);
	}
}

void STransformOutliner::ContextMenuNextLevel()
{
	if (Collection->HasAttribute("Level", DataCollectionGroup))
	{
		const TManagedArray<int32>& Levels = Collection->GetAttribute<int32>("Level", DataCollectionGroup);
		int32 MaxLevel = -1000;
		for (int32 Idx = 0; Idx < Levels.Num(); ++Idx)
		{
			if (Levels[Idx] > MaxLevel)
			{
				MaxLevel = Levels[Idx];
			}
		}

		if (DisplayLevel < MaxLevel)
		{
			ExpandByLevel(++DisplayLevel);
		}
	}
}

#undef LOCTEXT_NAMESPACE // "CollectionSpreadSheet"
