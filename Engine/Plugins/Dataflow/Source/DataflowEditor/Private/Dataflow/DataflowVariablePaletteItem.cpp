// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowVariablePaletteItem.h"

#include "Dataflow/DataflowGraphSchemaAction.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "DetailLayoutBuilder.h"
#include "PropertyBagDetails.h"
#include "SPinTypeSelector.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"

#define LOCTEXT_NAMESPACE "DataflowVariablePaletteItem"

void SDataflowVariablePaletteItem::Construct(const FArguments& InArgs, FCreateWidgetForActionData* const InCreateData, TSharedPtr<SDataflowGraphEditor> InEditor)
{
	if (InCreateData)
	{
		if (InCreateData->Action || InCreateData->Action->GetTypeId() == FEdGraphSchemaAction_DataflowVariable::StaticGetTypeId())
		{
			VariableAction = StaticCastSharedPtr<FEdGraphSchemaAction_DataflowVariable>(InCreateData->Action);
		}
	}
	SGraphPaletteItem::Construct(SGraphPaletteItem::FArguments(), InCreateData);
}

static bool CanHaveMemberVariableOfType(const FEdGraphPinType& PinType)
{
	if (PinType.PinCategory == UEdGraphSchema_K2::PC_Exec
		|| PinType.PinCategory == UEdGraphSchema_K2::PC_Wildcard
		|| PinType.PinCategory == UEdGraphSchema_K2::PC_MCDelegate
		|| PinType.PinCategory == UEdGraphSchema_K2::PC_Delegate
		|| PinType.PinCategory == UEdGraphSchema_K2::PC_Interface)
	{
		return false;
	}

	return true;
}

namespace UE::Dataflow::Private
{
	static bool IsSupportedDataflowVariableType(const FEdGraphPinType& PinType)
	{
		const bool bContainerIsAllowed = (false
			|| PinType.ContainerType == EPinContainerType::None 
			|| PinType.ContainerType == EPinContainerType::Array
			);

		const bool bCategoryIsAllowed = (false
			|| PinType.PinCategory == UEdGraphSchema_K2::PC_Boolean
			|| PinType.PinCategory == UEdGraphSchema_K2::PC_Class
			|| PinType.PinCategory == UEdGraphSchema_K2::PC_Int
			|| PinType.PinCategory == UEdGraphSchema_K2::PC_Int64
			|| PinType.PinCategory == UEdGraphSchema_K2::PC_Float
			|| PinType.PinCategory == UEdGraphSchema_K2::PC_Double
			|| PinType.PinCategory == UEdGraphSchema_K2::PC_Real
			|| PinType.PinCategory == UEdGraphSchema_K2::PC_Name
			|| PinType.PinCategory == UEdGraphSchema_K2::PC_Object
			|| PinType.PinCategory == UEdGraphSchema_K2::PC_String
			|| PinType.PinCategory == UEdGraphSchema_K2::PC_Struct
			|| PinType.PinCategory == UEdGraphSchema_K2::AllObjectTypes
			//|| PinType.PinCategory == UEdGraphSchema_K2::PC_Byte	// some enums are using this category  (see below )
			//|| PinType.PinCategory == UEdGraphSchema_K2::PC_Enum			// not yet as we do not support enum properly in dataflow graphs
			//|| PinType.PinCategory == UEdGraphSchema_K2::PC_SoftObject	// Graph do not really use this type
			//|| PinType.PinCategory == UEdGraphSchema_K2::PC_SoftClass		// Graph do not really use this type
			);

		return (bContainerIsAllowed && bCategoryIsAllowed);
	}

	static void GetFilteredVariableTypeTree(TArray<TSharedPtr<UEdGraphSchema_K2::FPinTypeTreeInfo>>& TypeTree, ETypeTreeFilter TypeTreeFilter)
	{
		// Collect all the supported types from the default property bag schema 
		check(GetDefault<UEdGraphSchema_K2>());
		TArray<TSharedPtr<UEdGraphSchema_K2::FPinTypeTreeInfo>> TempTypeTree;
		GetDefault<UPropertyBagSchema>()->GetVariableTypeTree(TempTypeTree, TypeTreeFilter);

		// make sure we add managed array collection type ( before the struct category )
		int32 StructIndex = INDEX_NONE;
		for (int32 Index = 0; Index < TempTypeTree.Num(); ++Index)
		{
			if (TempTypeTree[Index])
			{
				if (TempTypeTree[Index]->GetPinTypeNoResolve().PinCategory == UEdGraphSchema_K2::PC_Struct && !TempTypeTree[Index]->Children.IsEmpty())
				{
					StructIndex = Index;
					break;
				}
			}
		}
		TSharedPtr<UEdGraphSchema_K2::FPinTypeTreeInfo> CollectionType = MakeShareable(new UEdGraphSchema_K2::FPinTypeTreeInfo(UEdGraphSchema_K2::PC_Struct, TBaseStructure<FManagedArrayCollection>::Get(), LOCTEXT("ManagedArrayCollection", "A collection fo groups and attributes")));
		if (StructIndex != INDEX_NONE)
		{
			TempTypeTree.Insert(CollectionType, StructIndex);
		}
		else
		{
			TempTypeTree.Add(CollectionType);
		}

		// Filter the results
		for (TSharedPtr<UEdGraphSchema_K2::FPinTypeTreeInfo>& PinType : TempTypeTree)
		{
			if (PinType.IsValid())
			{
				if (IsSupportedDataflowVariableType(PinType->GetPinType(false)))
				{
					// remove children types that are not supported 
					for (int32 ChildIndex = 0; ChildIndex < PinType->Children.Num(); )
					{
						TSharedPtr<UEdGraphSchema_K2::FPinTypeTreeInfo> Child = PinType->Children[ChildIndex];
						if (Child.IsValid())
						{
							const FEdGraphPinType& ChildPinType = Child->GetPinType(/*bForceLoadSubCategoryObject*/false);

							if (!CanHaveMemberVariableOfType(ChildPinType))
							{
								if (!IsSupportedDataflowVariableType(ChildPinType))
								{
									PinType->Children.RemoveAt(ChildIndex);
									continue;
								}
							}
						}
						++ChildIndex;
					}
					TypeTree.Add(PinType);
				}
			}
		}
	}
}
TSharedRef<SWidget> SDataflowVariablePaletteItem::CreateTextSlotWidget(FCreateWidgetForActionData* const InCreateData, TAttribute<bool> bIsReadOnlyIn)
{
	if (VariableAction == nullptr)
	{
		return SNullWidget::NullWidget;
	}

	const FText VariableName = FText::FromString(VariableAction->GetVariableName());

	TSharedPtr<SInlineEditableTextBlock> EditableTextElement;

	// create a type selection widget
	TSharedPtr<SWidget> Widget =
		SNew(SHorizontalBox)

		// search bar 
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Left)
		[
			SAssignNew(EditableTextElement, SInlineEditableTextBlock)
				.Text(VariableName)
				.OnTextCommitted(this, &SDataflowVariablePaletteItem::OnNameTextCommitted)
				.OnVerifyTextChanged(this, &SDataflowVariablePaletteItem::OnNameTextVerifyChanged)
		]

		+ SHorizontalBox::Slot()
		.FillWidth(1.0f)
		.HAlign(HAlign_Right)
		[
			SNew(SBox)
				.Padding(FMargin(12, 0, 12, 0))
				[
					SNew(SPinTypeSelector, FGetPinTypeTree::CreateStatic(UE::Dataflow::Private::GetFilteredVariableTypeTree))
						.TargetPinType(VariableAction->GetVariableType())
						.OnPinTypeChanged(this, &SDataflowVariablePaletteItem::OnPinTypeChanged)
						.Schema(GetDefault<UPropertyBagSchema>())
						.bAllowArrays(/*bAllowArrays*/true)
						.TypeTreeFilter(ETypeTreeFilter::None)
						.Font(IDetailLayoutBuilder::GetDetailFont())
				]
		];

	InlineRenameWidget = EditableTextElement.ToSharedRef();
	InCreateData->OnRenameRequest->BindSP(InlineRenameWidget.Get(), &SInlineEditableTextBlock::EnterEditingMode);

	return Widget.ToSharedRef();
}

bool SDataflowVariablePaletteItem::OnNameTextVerifyChanged(const FText& InNewText, FText& OutErrorMessage)
{
	if (VariableAction)
	{
		const bool bCanRename = VariableAction->CanRenameItem(InNewText);
		if (!bCanRename)
		{
			OutErrorMessage = FText::FromString(TEXT("Invalid or existing variable name"));
		}
		return bCanRename;
	}
	return false;
}

void SDataflowVariablePaletteItem::OnNameTextCommitted(const FText& InNewText, ETextCommit::Type InTextCommit)
{
	if (VariableAction)
	{
		VariableAction->RenameItem(InNewText);
	}
}

void SDataflowVariablePaletteItem::OnPinTypeChanged(const FEdGraphPinType& PinType)
{
	if (VariableAction)
	{
		VariableAction->SetVariableType(PinType);
	}
}

#undef LOCTEXT_NAMESPACE 