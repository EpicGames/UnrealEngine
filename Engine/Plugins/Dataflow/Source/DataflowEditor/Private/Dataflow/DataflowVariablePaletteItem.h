// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SGraphPalette.h"

class SDataflowGraphEditor;
struct FEdGraphSchemaAction_DataflowVariable;
struct FEdGraphPinType;

/**
* Widget for displaying a single variable item in SDataflowMembersWidget
*/
class SDataflowVariablePaletteItem : public SGraphPaletteItem
{
public:
	SLATE_BEGIN_ARGS(SDataflowVariablePaletteItem) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, FCreateWidgetForActionData* const InCreateData, TSharedPtr<SDataflowGraphEditor> InEditor);

private:
	// SGraphPaletteItem Interface
	virtual TSharedRef<SWidget> CreateTextSlotWidget(FCreateWidgetForActionData* const InCreateData, TAttribute<bool> bIsReadOnly) override;
	virtual bool OnNameTextVerifyChanged(const FText& InNewText, FText& OutErrorMessage) override;
	virtual void OnNameTextCommitted(const FText& NewText, ETextCommit::Type InTextCommit) override;
	// End of SGraphPaletteItem Interface

	void OnPinTypeChanged(const FEdGraphPinType& PinType);

private:
	TSharedPtr<FEdGraphSchemaAction_DataflowVariable> VariableAction;

	// held for OnRequestRename calls
	TSharedPtr<SInlineEditableTextBlock> InlineRenameWidget;
};
