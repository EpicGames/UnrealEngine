// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowVerticesOutlinerWidget.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Dataflow/DataflowSettings.h"
#include "GeometryCollection/GeometryCollection.h"
#include "Dataflow/DataflowCollectionSpreadSheetHelpers.h"

#define LOCTEXT_NAMESPACE "VerticesOutliner"

// --- FGeometryCollectionVerticesTreeItem ---

void FGeometryCollectionVerticesTreeItem::GenerateContextMenu(UToolMenu* Menu, SVerticesOutliner& Outliner)
{
}

// --- FGeometryCollectionVerticesTreeItemGeometry ---

TSharedRef<ITableRow> FGeometryCollectionVerticesTreeItemGeometry::MakeTreeRowWidget(const TSharedRef<STableViewBase>& InOwnerTable, bool bNoExtraColumn)
{
	FString NumVertsString = "";
	FString NumFacesString = "";
	FString TransformIdxStr = "";
	FString GeometryIdxStr = "";
	FString BoneNameStr = "";

	if (Collection.IsValid() &&
		Collection->HasAttribute("TransformIndex", FGeometryCollection::GeometryGroup) &&
		Collection->HasAttribute("TransformToGeometryIndex", FGeometryCollection::TransformGroup) &&
		Collection->HasAttribute("VertexCount", FGeometryCollection::GeometryGroup) &&
		Collection->HasAttribute("FaceCount", FGeometryCollection::GeometryGroup) &&
		Collection->HasAttribute("BoneName", FGeometryCollection::TransformGroup))
	{
		const TManagedArray<int32>& TransformIndices = Collection->GetAttribute<int32>("TransformIndex", FGeometryCollection::GeometryGroup);
		const TManagedArray<int32>& TransformToGeometryIndices = Collection->GetAttribute<int32>("TransformToGeometryIndex", FGeometryCollection::TransformGroup);
		const TManagedArray<int32>& VertexCounts = Collection->GetAttribute<int32>("VertexCount", FGeometryCollection::GeometryGroup);
		const TManagedArray<int32>& FaceCounts = Collection->GetAttribute<int32>("FaceCount", FGeometryCollection::GeometryGroup);
		const TManagedArray<FString>& BoneNames = Collection->GetAttribute<FString>("BoneName", FGeometryCollection::TransformGroup);

		const int32 TransformIdx = TransformIndices[GeometryIdx];
		TransformIdxStr = FString::FormatAsNumber(TransformIdx);

		GeometryIdxStr = FString::FormatAsNumber(GeometryIdx);

		const int32 NumVerts = VertexCounts[GeometryIdx];
		NumVertsString = FString::FormatAsNumber(NumVerts);

		const int32 NumFaces = FaceCounts[GeometryIdx];
		NumFacesString = FString::FormatAsNumber(NumFaces);

		BoneNameStr = BoneNames[TransformIdx];
	}

	return SNew(STableRow<FGeometryCollectionVerticesTreeItemPtr>, InOwnerTable)
		.Content()
		[
			SNew(STextBlock).Text(FText::FromString("Geometry Idx: " + GeometryIdxStr + "     BoneName: " + BoneNameStr + "     Transform Idx: " + TransformIdxStr + "     Faces: " + NumFacesString + "     Verts: " + NumVertsString))
		];
}

void FGeometryCollectionVerticesTreeItemGeometry::GetChildren(FGeometryCollectionVerticesTreeItemList& OutChildren)
{
	OutChildren = ChildItems;
}

FGeometryCollectionVerticesTreeItemPtr FGeometryCollectionVerticesTreeItemGeometry::GetItemFromVertexIndex(int32 InVertexIndex) const
{
	return ItemsByVertexIndex.FindRef(InVertexIndex);
}

void FGeometryCollectionVerticesTreeItemGeometry::GetChildrenForVertex(FGeometryCollectionVerticesTreeItemVertex& VertexItem, FGeometryCollectionVerticesTreeItemList& OutChildren) const
{
	if (!Collection.IsValid())
	{
		return;
	}
}

bool FGeometryCollectionVerticesTreeItemGeometry::HasChildrenForVertex(const FGeometryCollectionVerticesTreeItemVertex& VertexItem) const
{
	if (!Collection.IsValid())
	{
		return false;
	}

	return false;
}

void FGeometryCollectionVerticesTreeItemGeometry::ExpandAll()
{
	TreeView->SetItemExpansion(AsShared(), true);

	for (const auto& Elem : ItemsByVertexIndex)
	{
		TreeView->SetItemExpansion(Elem.Value, true);
	}
}

void FGeometryCollectionVerticesTreeItemGeometry::RegenerateChildren()
{
	if (Collection.IsValid())
	{
		ItemsByVertexIndex.Empty();
		ChildItems.Empty();

		if (Collection &&
			Collection->HasAttribute("VertexStart", FGeometryCollection::GeometryGroup) &&
			Collection->HasAttribute("VertexCount", FGeometryCollection::GeometryGroup))
		{
			const TManagedArray<int32>& VertexStarts = Collection->GetAttribute<int32>("VertexStart", FGeometryCollection::GeometryGroup);
			const TManagedArray<int32>& VertexCounts = Collection->GetAttribute<int32>("VertexCount", FGeometryCollection::GeometryGroup);

			const int32 VertexStart = VertexStarts[GeometryIdx];
			const int32 VertexCount = VertexCounts[GeometryIdx];

			RootIndex = FGeometryCollection::Invalid;

			ItemsByVertexIndex.Reserve(VertexCount);
			ChildItems.Reserve(VertexCount);

			// Add a sub item to the outliner tree for each of the bones/chunks in this GeometryCollection
			for (int32 Index = 0; Index < VertexCount; ++Index)
			{
				const int32 VertexIndex = VertexStart + Index;

				TSharedRef<FGeometryCollectionVerticesTreeItemVertex> NewItem = MakeShared<FGeometryCollectionVerticesTreeItemVertex>(VertexIndex, this);

				ChildItems.Add(NewItem);
				ItemsByVertexIndex.Add(VertexIndex, NewItem);
			}
		}
	}
}

void FGeometryCollectionVerticesTreeItemGeometry::RequestTreeRefresh()
{
	if (TreeView.IsValid())
	{
		TreeView->RequestTreeRefresh();
	}
}

bool FGeometryCollectionVerticesTreeItemGeometry::FilterBoneIndex(int32 BoneIndex) const
{
	return true;
}

bool FGeometryCollectionVerticesTreeItemGeometry::IsValid() const
{
	if (bInvalidated)
	{
		return false;
	}

	if (Collection)
	{
		return true;
	}

	return false;
}

// --- FGeometryCollectionVerticesTreeItemGeometry ---


TSharedPtr<const FManagedArrayCollection> FGeometryCollectionVerticesTreeItemVertex::GetCollection() const
{
	ensure(ParentCollectionItem);
	return ParentCollectionItem->GetCollection();
}

void FGeometryCollectionVerticesTreeItemVertex::UpdateItemColorFromCollection()
{
	ItemColor = UE::Dataflow::CollectionSpreadSheetHelpers::UpdateItemColorFromCollection(GetCollection(), FGeometryCollection::VerticesGroup, GetVertexIndex());
}

TSharedRef<ITableRow> FGeometryCollectionVerticesTreeItemVertex::MakeTreeRowWidget(const TSharedRef<STableViewBase>& InOwnerTable, bool bIsPinned)
{
	UpdateItemColorFromCollection();

	TSharedPtr<const FManagedArrayCollection> Collection = GetCollection();

	return SNew(SVerticesOutlinerRow, InOwnerTable, SharedThis(this));
}

bool FGeometryCollectionVerticesTreeItemVertex::IsValidVertex() const
{
	TSharedPtr<const FManagedArrayCollection> Collection = GetCollection();

	if (ParentCollectionItem && ParentCollectionItem->IsValid())
	{
		return VertexIndex >= 0 && VertexIndex < Collection->NumElements(FGeometryCollection::VerticesGroup);
	}

	return false;
}

TSharedRef<SWidget> FGeometryCollectionVerticesTreeItemVertex::MakeIndexColumnWidget() const
{
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(STextBlock)
			.Text(FText::AsNumber(VertexIndex))
			.ColorAndOpacity(ItemColor)
		];
}

TSharedRef<SWidget> FGeometryCollectionVerticesTreeItemVertex::MakeEmptyColumnWidget() const
{
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.Padding(12.f, 0.f)
		[
			SNew(STextBlock)
			.ColorAndOpacity(ItemColor)
		];
}

void FGeometryCollectionVerticesTreeItemVertex::GetChildren(FGeometryCollectionVerticesTreeItemList& OutChildren)
{
	ParentCollectionItem->GetChildrenForVertex(*this, OutChildren);
}

bool FGeometryCollectionVerticesTreeItemVertex::HasChildren() const
{
	return ParentCollectionItem->HasChildrenForVertex(*this);
}

// --- SVerticesOutlinerRow ---

TSharedRef<SWidget> SVerticesOutlinerRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	TSharedPtr<const FManagedArrayCollection> Collection = Item->GetCollection();
	FName AttrType = UE::Dataflow::CollectionSpreadSheetHelpers::GetArrayTypeString(Collection->GetAttributeType(ColumnName, FGeometryCollection::VerticesGroup));

	// This can happen because sometimes slate retains old items until the next tick, and keeps calling callbacks on them until then
	if (!Item->IsValidVertex())
	{
		return Item->MakeEmptyColumnWidget();
	}

	if (ColumnName == "Index")
	{
		const TSharedPtr<SWidget> NameWidget = Item->MakeIndexColumnWidget();
		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(SExpanderArrow, SharedThis(this))
				.ShouldDrawWires(true)
			]
			+ SHorizontalBox::Slot()
			[
				NameWidget.ToSharedRef()
			];
	}

	return UE::Dataflow::CollectionSpreadSheetHelpers::MakeColumnWidget(Collection, 
		FGeometryCollection::VerticesGroup, 
		ColumnName, 
		Item->GetVertexIndex(), 
		Item->GetItemColor());
}

// --- SVerticesOutliner ---

void SVerticesOutliner::Construct(const FArguments& InArgs)
{
	bPerformingSelection = false;

	HeaderRowWidget =
		SNew(SHeaderRow)
		.Visibility(EVisibility::Visible);

	RegenerateHeader();

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.Padding(FMargin(0.0f, 3.f))
		[
			SNew(SOverlay)
			+ SOverlay::Slot()
			[
				SAssignNew(TreeView, STreeView<FGeometryCollectionVerticesTreeItemPtr>)
				.TreeItemsSource(reinterpret_cast<FGeometryCollectionVerticesTreeItemList*>(&RootNodes))
				.OnSelectionChanged(this, &SVerticesOutliner::OnSelectionChanged)
				.OnGenerateRow(this, &SVerticesOutliner::MakeTreeRowWidget)
				.OnGetChildren(this, &SVerticesOutliner::OnGetChildren)
				.OnContextMenuOpening(this, &SVerticesOutliner::OnOpenContextMenu)
				.AllowInvisibleItemSelection(true)
				.ShouldStackHierarchyHeaders(true)
				.OnGeneratePinnedRow(this, &SVerticesOutliner::OnGeneratePinnedRowWidget, true)
				.HighlightParentNodesForSelection(true)
				.OnSetExpansionRecursive(this, &SVerticesOutliner::ExpandRecursive)
				.HeaderRow(HeaderRowWidget)
				.ExternalScrollbar(InArgs._ExternalVerticalScrollBar)
			]
		]
	];
}

void SVerticesOutliner::RegenerateHeader()
{
	constexpr int32 CustomColumnWidth = 120;
	constexpr int32 CWidthMult = 9;

	HeaderRowWidget->ClearColumns();

	if (!Collection.IsValid() || Collection->NumElements(FGeometryCollection::VerticesGroup) == 0)
	{
		return;
	}

	TArray<UE::Dataflow::CollectionSpreadSheetHelpers::FAttrInfo> AttrInfo;

	// Add "Index" manually
	AttrInfo.Add({ "Index", "int32" });
	// Add all the other attrs
	for (FName Attr : Collection->AttributeNames(FGeometryCollection::VerticesGroup))
	{
		AttrInfo.Add({ Attr, UE::Dataflow::CollectionSpreadSheetHelpers::GetArrayTypeString(Collection->GetAttributeType(Attr, FGeometryCollection::VerticesGroup)).ToString()});
	}

	for (int32 Idx = 0; Idx < AttrInfo.Num(); ++Idx)
	{
		int32 ColumnWidth = CustomColumnWidth;
		const FString ColumnNameStr = AttrInfo[Idx].Name.ToString();

		if (ColumnNameStr == "Index")
		{
			ColumnWidth = CustomColumnWidth;

			HeaderRowWidget->AddColumn(
				SHeaderRow::Column(AttrInfo[Idx].Name)
				.DefaultLabel(FText::FromName(AttrInfo[Idx].Name))
				.ManualWidth(ColumnWidth)
				.HAlignCell(HAlign_Left)
				.HAlignHeader(HAlign_Center)
				.VAlignCell(VAlign_Center));
		}
		else
		{
			int32 ColumnNameStrLen = ColumnNameStr.Len() * CWidthMult;

			if (UE::Dataflow::CollectionSpreadSheetHelpers::AttrTypeWidthMap.Contains(AttrInfo[Idx].Type))
			{
				ColumnWidth = UE::Dataflow::CollectionSpreadSheetHelpers::AttrTypeWidthMap[AttrInfo[Idx].Type];
			}

			ColumnWidth = ColumnNameStrLen > ColumnWidth ? ColumnNameStrLen : ColumnWidth;

			HeaderRowWidget->AddColumn(
				SHeaderRow::Column(AttrInfo[Idx].Name)
				.DefaultLabel(FText::FromName(AttrInfo[Idx].Name))
				.ManualWidth(ColumnWidth)
				.HAlignCell(HAlign_Center)
				.HAlignHeader(HAlign_Center)
				.VAlignCell(VAlign_Center));
		}
	}
}

void SVerticesOutliner::RegenerateItems()
{
	TreeView->RebuildList();
}

TSharedRef<ITableRow> SVerticesOutliner::MakeTreeRowWidget(FGeometryCollectionVerticesTreeItemPtr InItem, const TSharedRef<STableViewBase>& InOwnerTable)
{
	return InItem->MakeTreeRowWidget(InOwnerTable);
}

TSharedRef<ITableRow> SVerticesOutliner::OnGeneratePinnedRowWidget(FGeometryCollectionVerticesTreeItemPtr InItem, const TSharedRef<STableViewBase>& InOwnerTable, bool bPinned)
{
	return InItem->MakeTreeRowWidget(InOwnerTable, true);
}

void SVerticesOutliner::OnGetChildren(FGeometryCollectionVerticesTreeItemPtr InItem, TArray<FGeometryCollectionVerticesTreeItemPtr>& OutChildren)
{
	InItem->GetChildren(OutChildren);
}

TSharedPtr<SWidget> SVerticesOutliner::OnOpenContextMenu()
{
	return TSharedPtr<SWidget>();
}

void SVerticesOutliner::UpdateGeometryCollection()
{
	TreeView->RequestTreeRefresh();
	//ExpandAll();
}

void SVerticesOutliner::SetCollection(const TSharedPtr<const FManagedArrayCollection>& InCollection)
{
	if (Collection == InCollection)
	{
		return;
	}

	// Clear the cached Tree ItemSelection without affecting the SelectedBones as 
	// we want to refresh the tree selection using selected bones
	TGuardValue<bool> ExternalSelectionGuard(bPerformingSelection, true);
	TreeView->ClearSelection();

	// explicitly mark the root nodes as invalid before emptying, so we know we can safely ignore them in case slate still triggers callbacks for them (they will not be deleted until the tree view refresh, on tick)
	for (TSharedPtr<FGeometryCollectionVerticesTreeItemGeometry>& RootNode : RootNodes)
	{
		if (RootNode)
		{
			RootNode->Invalidate();
		}
	}
	RootNodes.Empty();

	const int32 NumGeoms = InCollection->NumElements(FGeometryCollection::GeometryGroup);
	
	for (int32 Idx = 0; Idx < NumGeoms; ++Idx)
	{
		RootNodes.Add(MakeShared<FGeometryCollectionVerticesTreeItemGeometry>(InCollection, Idx, TreeView));
	}

	TreeView->RequestTreeRefresh();
	//ExpandAll();

	Collection = InCollection;
}

void SVerticesOutliner::ExpandAll()
{
	for (TSharedPtr<FGeometryCollectionVerticesTreeItemGeometry> ItemPtr : RootNodes)
	{
		ItemPtr->ExpandAll();
	}
}

void SVerticesOutliner::ExpandRecursive(TSharedPtr<FGeometryCollectionVerticesTreeItem> ItemPtr, bool bInExpansionState) const
{
	TreeView->SetItemExpansion(ItemPtr, bInExpansionState);

	FGeometryCollectionVerticesTreeItemList ItemChildren;
	ItemPtr->GetChildren(ItemChildren);
	for (auto& Child : ItemChildren)
	{
		ExpandRecursive(Child, bInExpansionState);
	}
}

int32 SVerticesOutliner::GetBoneSelectionCount() const
{
	return TreeView->GetSelectedItems().Num();
}

void SVerticesOutliner::OnSelectionChanged(FGeometryCollectionVerticesTreeItemPtr Item, ESelectInfo::Type SelectInfo)
{
}

#undef LOCTEXT_NAMESPACE // "VerticesOutliner"
