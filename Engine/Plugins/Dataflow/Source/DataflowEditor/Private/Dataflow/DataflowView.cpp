// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowView.h"

#include "Dataflow/DataflowEditor.h"
#include "Dataflow/DataflowSelection.h"
#include "Templates/EnableIf.h"

#define LOCTEXT_NAMESPACE "DataflowView"

FDataflowNodeView::FDataflowNodeView(TObjectPtr<UDataflowBaseContent> InContent)
	: FGCObject()
	, EditorContent(InContent)
{
}

TObjectPtr<UDataflowBaseContent> FDataflowNodeView::GetEditorContent()
{
	if (ensure(EditorContent))
	{
		return EditorContent;
	}
	return nullptr;
}

bool FDataflowNodeView::SelectedNodeHaveSupportedOutputTypes(UDataflowEdNode* InNode)
{
	SetSupportedOutputTypes();

	if (InNode && InNode->IsBound())
	{
		if (TSharedPtr<FDataflowNode> DataflowNode = InNode->DataflowGraph->FindBaseNode(InNode->DataflowNodeGuid))
		{
			TArray<FDataflowOutput*> Outputs = DataflowNode->GetOutputs();

			for (FDataflowOutput* Output : Outputs)
			{
				for (const FString& OutputType : SupportedOutputTypes)
				{
					if (Output->GetType() == FName(*OutputType))
					{
						return true;
					}
				}
			}
		}
	}

	return false;
}

void FDataflowNodeView::OnConstructionViewSelectionChanged(const TArray<UPrimitiveComponent*>& SelectedComponents, const TArray<FDataflowBaseElement*>& SelectedElements)
{
	ConstructionViewSelectionChanged(SelectedComponents, SelectedElements);
}

void FDataflowNodeView::OnSimulationViewSelectionChanged(const TArray<UPrimitiveComponent*>& SelectedComponents, const TArray<FDataflowBaseElement*>& SelectedElements)
{
	SimulationViewSelectionChanged(SelectedComponents, SelectedElements);
}

UE::Dataflow::FTimestamp  FDataflowNodeView::GetNodeLastModifiedTimeStamp(UDataflowEdNode* InNode) const
{
	if (InNode)
	{
		return GetNodeLastModifiedTimeStamp(InNode->GetDataflowNode().Get());
	}
	return UE::Dataflow::FTimestamp::Invalid;
}
UE::Dataflow::FTimestamp  FDataflowNodeView::GetNodeLastModifiedTimeStamp(FDataflowNode* InNode) const
{
	if (InNode)
	{
		return InNode->GetTimestamp();
	}
	return UE::Dataflow::FTimestamp::Invalid;
}

void FDataflowNodeView::OnSelectedNodeChanged(UDataflowEdNode* InNode)
{
	if (!bIsPinnedDown)
	{
		UDataflowEdNode* ValidNodeToSelect = nullptr;
		if (SelectedNodeHaveSupportedOutputTypes(InNode))
		{
			ValidNodeToSelect = InNode;
		}

		const UE::Dataflow::FTimestamp NodeTimestamp = GetNodeLastModifiedTimeStamp(ValidNodeToSelect);
		const bool bIsMoreRecentThanRefresh = (LastRefreshTimestamp < NodeTimestamp);
		const bool bIsDifferentNode = (ValidNodeToSelect != SelectedNode);
		
		if (bIsDifferentNode)
		{
			SelectedNode = ValidNodeToSelect;
		}

		if (bIsMoreRecentThanRefresh || bIsDifferentNode)
		{
			LastRefreshTimestamp = NodeTimestamp;
			UpdateViewData();
		}
	}
}

void FDataflowNodeView::RefreshView()
{
	const UE::Dataflow::FTimestamp NodeTimestamp = GetNodeLastModifiedTimeStamp(SelectedNode);
	const bool bIsMoreRecentThanRefresh = (LastRefreshTimestamp < NodeTimestamp);

	if (!bIsRefreshLocked && SelectedNode && bIsMoreRecentThanRefresh)
	{
		UpdateViewData();
	}
}

void FDataflowNodeView::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(SelectedNode);
	if (EditorContent)
	{
		Collector.AddReferencedObject(EditorContent);
	}
}


#undef LOCTEXT_NAMESPACE
