// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"
#include "Dataflow/DataflowEdNode.h"
#include "Dataflow/SelectionViewWidget.h"
#include "Dataflow/DataflowSelection.h"

struct FDataflowBaseElement;
class UPrimitiveComponent;
class UDataflowEditor;
class UDataflowBaseContent;
/**
*
* Base listener class to interface between the DataflowToolkit and Dataflow views
*
*/
class IDataflowViewListener
{
public:
	virtual void OnConstructionViewSelectionChanged(const TArray<UPrimitiveComponent*>& SelectedComponents, const TArray<FDataflowBaseElement*>& SelectedElements) = 0;
	virtual void OnSimulationViewSelectionChanged(const TArray<UPrimitiveComponent*>& SelectedComponents, const TArray<FDataflowBaseElement*>& SelectedElements) = 0;
	virtual void OnSelectedNodeChanged(UDataflowEdNode* InNode) = 0;  // nullptr is valid
	virtual void OnNodeInvalidated(FDataflowNode* InvalidatedNode) = 0;
	virtual void RefreshView() = 0;
};


/**
*
* FDataflowNodeView class implements common functions for single node based Dataflow views
*
*/
class FDataflowNodeView : public FGCObject, public IDataflowViewListener
{
public:
	FDataflowNodeView(TObjectPtr<UDataflowBaseContent> InContent = nullptr);

	UDataflowEdNode* GetSelectedNode() const { return SelectedNode; }
	bool SelectedNodeHaveSupportedOutputTypes(UDataflowEdNode* InNode);

	TArray<FString>& GetSupportedOutputTypes() { return SupportedOutputTypes; }

	TObjectPtr<UDataflowBaseContent> GetEditorContent();

	/**
	* Virtual functions to overwrite in view widget classes
	*/
	virtual void UpdateViewData() = 0;
	virtual void SetSupportedOutputTypes() = 0;
	virtual void ConstructionViewSelectionChanged(const TArray<UPrimitiveComponent*>& SelectedComponents, const TArray<FDataflowBaseElement*>& SelectedElements) = 0;
	virtual void SimulationViewSelectionChanged(const TArray<UPrimitiveComponent*>& SelectedComponents, const TArray<FDataflowBaseElement*>& SelectedElements) = 0;
	/**
	* Callback for PinnedDown change
	*/
	void OnPinnedDownChanged(const bool State) { bIsPinnedDown = State; }

	/**
	* Callback for RefreshLock change
	*/
	void OnRefreshLockedChanged(const bool State) { bIsRefreshLocked = State; }

	/**
	* Virtual function overrides from IDataflowViewListener base class
	*/
	virtual void OnConstructionViewSelectionChanged(const TArray<UPrimitiveComponent*>& SelectedComponents, const TArray<FDataflowBaseElement*>& SelectedElements) override;
	virtual void OnSimulationViewSelectionChanged(const TArray<UPrimitiveComponent*>& SelectedComponents, const TArray<FDataflowBaseElement*>& SelectedElements) override;
	virtual void OnSelectedNodeChanged(UDataflowEdNode* InNode) override;  // nullptr is valid
	virtual void OnNodeInvalidated(FDataflowNode* InvalidatedNode) override {}
	virtual void RefreshView() override;

	/** FGCObject interface */
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override { return TEXT("FDataflowNodeView"); }

public:
	UE::Dataflow::FTimestamp GetNodeLastModifiedTimeStamp(UDataflowEdNode* InNode) const;
	UE::Dataflow::FTimestamp GetNodeLastModifiedTimeStamp(FDataflowNode* InNode) const;

private:
	TObjectPtr<UDataflowBaseContent> EditorContent = nullptr;

	TObjectPtr<UDataflowEdNode> SelectedNode = nullptr;
	UE::Dataflow::FTimestamp LastRefreshTimestamp = UE::Dataflow::FTimestamp::Invalid;

	bool bIsPinnedDown = false;

	bool bIsRefreshLocked = false;

	TArray<FString> SupportedOutputTypes;

	FDelegateHandle OnNodeInvalidatedDelegateHandle;
};


