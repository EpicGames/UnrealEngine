// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DebugDrawText3dVisualization.h"
#include "Dataflow/DataflowConstructionScene.h"
#include "Dataflow/DataflowDebugDrawComponent.h"
#include "Engine/Canvas.h"
#include "Engine/Engine.h"

namespace UE::Dataflow
{
	namespace Private
	{
		float DataflowConstructionFontScale = 1.5;
		FAutoConsoleVariableRef CVARDataflowConstructionFontScale(TEXT("p.Dataflow.Editor.ConstructionFontScale"), DataflowConstructionFontScale,
			TEXT("Scale text in the Dataflow Construction View."));

		static void DrawText(FCanvas* Canvas, const FSceneView* SceneView, const FVector& Pos, const FString& Text, const FLinearColor& Color, const float Scale = 1.f)
		{
#if WITH_EDITOR
			if (Canvas && SceneView)
			{
				FVector2D PixelLocation;
				if (SceneView->WorldToPixel(Pos, PixelLocation))
				{
					// WorldToPixel doesn't account for DPIScale
					const float DPIScale = Canvas->GetDPIScale();
					FCanvasTextItem TextItem(PixelLocation / DPIScale, FText::FromString(Text), GEngine->GetSmallFont(), Color);

					TextItem.Scale = FVector2D::UnitVector * Scale * DataflowConstructionFontScale;
					TextItem.EnableShadow(FLinearColor::Black);
					TextItem.Draw(Canvas);
				}
			}
#endif
		}
	}

	FName FDebugDrawText3dVisualization::Name = FName("DebugDrawText3dVisualization");

	void FDebugDrawText3dVisualization::DrawCanvas(const FDataflowConstructionScene* ConstructionScene, FCanvas* Canvas, const FSceneView* SceneView)
	{
		if (ConstructionScene)
		{
			const TArray<TObjectPtr<UDebugDrawComponent>> DebugDrawComponents = ConstructionScene->GetDebugDrawComponents();
			for (const TObjectPtr<UDebugDrawComponent>& DebugDrawComponent : DebugDrawComponents)
			{
				if (DebugDrawComponent)
				{
					if (const FDebugRenderSceneProxy* SceneProxy = static_cast<FDebugRenderSceneProxy*>(DebugDrawComponent->GetSceneProxy()))
					{
						for (const FDebugRenderSceneProxy::FText3d& Text : SceneProxy->Texts)
						{
							Private::DrawText(Canvas, SceneView, Text.Location, Text.Text, Text.Color);
						}
					}
				}
			}
		}
	}
}

