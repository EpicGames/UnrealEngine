// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Dataflow/DataflowConstructionVisualization.h"

namespace UE::Dataflow
{
	/** Display Text3d from FDataflowDebugDraw on the editor's Canvas */
	class FDebugDrawText3dVisualization final : public IDataflowConstructionVisualization
	{
	public:
		static FName Name;

	private:
		virtual FName GetName() const override
		{
			return Name;
		}
		virtual void DrawCanvas(const FDataflowConstructionScene* ConstructionScene, FCanvas* Canvas, const FSceneView* SceneView) override;
	};
}

