// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/MeshStatsConstructionVisualization.h"
#include "Components/DynamicMeshComponent.h"
#include "Dataflow/DataflowConstructionViewportClient.h"
#include "Dataflow/DataflowConstructionScene.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "CanvasItem.h"
#include "CanvasTypes.h"

#define LOCTEXT_NAMESPACE "MeshStatsConstructionVisualization"

namespace UE::Dataflow
{
	FName FMeshStatsConstructionVisualization::Name = FName("MeshStatsConstructionVisualization");

	void FMeshStatsConstructionVisualization::ExtendViewportShowMenu(const TSharedPtr<FDataflowConstructionViewportClient>& ViewportClient, FMenuBuilder& MenuBuilder)
	{
		MenuBuilder.BeginSection(TEXT("MeshStatsVisualization"), LOCTEXT("MeshStatsVisualizationSectionName", "Mesh Stats"));
		{
			const FUIAction MeshStatsToggleAction(
				FExecuteAction::CreateLambda([this, ViewportClient]()
					{
						bMeshStatsVisualizationEnabled = !bMeshStatsVisualizationEnabled;
						if (ViewportClient)
						{
							ViewportClient->Invalidate();
						}
					}),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this]()
					{
						return bMeshStatsVisualizationEnabled;
					}));

			MenuBuilder.AddMenuEntry(LOCTEXT("MeshStatsVisualization_MeshStatsEnabled", "Mesh Stats"),
				LOCTEXT("MeshStatsVisualization_MeshStatsEnabled_TooltipText", "Display mesh stats"),
				FSlateIcon(),
				MeshStatsToggleAction,
				NAME_None,
				EUserInterfaceActionType::ToggleButton);
		}
		MenuBuilder.EndSection();
	}

	void FMeshStatsConstructionVisualization::DrawCanvas(const FDataflowConstructionScene* ConstructionScene, FCanvas* Canvas, const FSceneView* SceneView)
	{
		if (bMeshStatsVisualizationEnabled && ConstructionScene)
		{
			int32 TotalTris = 0;
			int32 TotalVertices = 0;

			for (const UDynamicMeshComponent* const MeshComponent : ConstructionScene->GetDynamicMeshComponents())
			{
				if (const UE::Geometry::FDynamicMesh3* const Mesh = MeshComponent->GetMesh())
				{
					TotalTris += Mesh->TriangleCount();
					TotalVertices += Mesh->VertexCount();
				}
			}

			FCanvasTextItem MessageTextItem(FVector2D(10, 40), FText::GetEmpty(), GEngine->GetSmallFont(), FLinearColor::White);
			MessageTextItem.Text = FText::Format(LOCTEXT("MeshStatsVisualization_DisplayMessage", "Triangles: {0}   Vertices: {1}"), TotalTris, TotalVertices);
			MessageTextItem.EnableShadow(FLinearColor::Black);
			Canvas->DrawItem(MessageTextItem);
		}
	}

}

#undef LOCTEXT_NAMESPACE 

