// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Dataflow/DataflowConstructionVisualization.h"

namespace UE::Dataflow
{
	/** A simple visualization that displays the number of triangles in the Construction View Scene */
	class FMeshStatsConstructionVisualization final : public IDataflowConstructionVisualization
	{
	public:
		static FName Name;

	private:
		virtual FName GetName() const override
		{
			return Name;
		}
		virtual void ExtendViewportShowMenu(const TSharedPtr<FDataflowConstructionViewportClient>& ViewportClient, FMenuBuilder& MenuBuilder) override;
		virtual void DrawCanvas(const FDataflowConstructionScene* ConstructionScene, FCanvas* Canvas, const FSceneView* SceneView) override;

		bool bMeshStatsVisualizationEnabled = false;
	};
}

