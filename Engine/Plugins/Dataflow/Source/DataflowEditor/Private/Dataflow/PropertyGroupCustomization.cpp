// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/PropertyGroupCustomization.h"
#include "Dataflow/DataflowGraphEditor.h"
#include "Dataflow/DataflowNode.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"
#include "DetailWidgetRow.h"

#define LOCTEXT_NAMESPACE "PropertyGroupCustomization"

namespace UE::Dataflow
{
	bool FPropertyGroupCustomization::MakeGroupName(FString& InOutString)
	{
		const FString SourceString = InOutString;
		InOutString = SlugStringForValidName(InOutString, TEXT("_")).Replace(TEXT("\\"), TEXT("_"));
		bool bCharsWereRemoved;
		do { InOutString.TrimCharInline(TEXT('_'), &bCharsWereRemoved); } while (bCharsWereRemoved);
		return InOutString.Equals(SourceString);
	}

	TSharedRef<IPropertyTypeCustomization> FPropertyGroupCustomization::MakeInstance()
	{
		return MakeShareable(new FPropertyGroupCustomization);
	}

	bool FPropertyGroupCustomization::MakeValidListName(FString& InOutString, FText& OutErrorMessage) const
	{
		const bool bIsValidGroupName = MakeGroupName(InOutString);
		if (!bIsValidGroupName)
		{
			OutErrorMessage =
				LOCTEXT("NotAValidGroupName",
					"To be a valid group name, this text string musn't start by an underscore,\n"
					"contain whitespaces, or any of the following character: \"',/.:|&!~@#(){}[]=;^%$`");
		}
		return bIsValidGroupName;
	}


	TArray<FName> FPropertyGroupCustomization::GetListNames(const FManagedArrayCollection& Collection) const
	{
		// Find all group names in the parent node's collection
		const TArray<FName> TargetGroupNames = GetTargetGroupNames(Collection);
		if (Collection.IsEmpty())
		{
			// if the collection is empty, no filtering 
			// this may happen if the graph is in manual evaluation and will return empty collection
			return TargetGroupNames;
		}

		// Find all group names in the parent selection node's collection
		TArray<FName> CollectionGroupNames;
		CollectionGroupNames.Reserve(TargetGroupNames.Num());
		for (const FName& GroupName : Collection.GroupNames())
		{
			if (TargetGroupNames.Contains(GroupName))
			{
				CollectionGroupNames.Add(GroupName);
			}
		}
		return CollectionGroupNames;
	}
}  // End namespace UE::Chaos::ClothAsset

#undef LOCTEXT_NAMESPACE
