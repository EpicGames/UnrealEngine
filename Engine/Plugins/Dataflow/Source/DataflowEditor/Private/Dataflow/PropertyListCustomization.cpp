// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/PropertyListCustomization.h"
#include "Dataflow/DataflowGraphEditor.h"
#include "Dataflow/DataflowNode.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"

namespace UE::Dataflow
{
	const FManagedArrayCollection& FPropertyListCustomization::GetPropertyCollection(
		const TSharedPtr<UE::Dataflow::FContext>& Context,
		const TSharedPtr<IPropertyHandle>& ChildPropertyHandle,
		const FName CollectionPropertyName = FName(TEXT("Collection")))
	{
		static const FManagedArrayCollection EmptyCollection;

		TSharedPtr<IPropertyHandle> OwnerHandle = ChildPropertyHandle;
		while (TSharedPtr<IPropertyHandle> ParentHandle = OwnerHandle->GetParentHandle())
		{
			OwnerHandle = MoveTemp(ParentHandle);
		}
		if (const TSharedPtr<IPropertyHandleStruct> OwnerHandleStruct = OwnerHandle->AsStruct())
		{
			if (const TSharedPtr<FStructOnScope> StructOnScope = OwnerHandleStruct->GetStructData())
			{
				if (const UStruct* const Struct = StructOnScope->GetStruct())
				{
					if (Struct->IsChildOf<FDataflowNode>())
					{
						const FDataflowNode* const DataflowNode = reinterpret_cast<FDataflowNode*>(StructOnScope->GetStructMemory());

						if (const FProperty* const Property = Struct->FindPropertyByName(CollectionPropertyName))
						{
							if (const FStructProperty* const StructProperty = CastField<FStructProperty>(Property))
							{
								if (StructProperty->GetCPPType(nullptr, CPPF_None) == TEXT("FManagedArrayCollection"))
								{
									if (const FDataflowInput* const DataflowInput = DataflowNode->FindInput(StructProperty->ContainerPtrToValuePtr<FManagedArrayCollection*>(DataflowNode)))
									{
										// Read-only: do not force evaluation here. Evaluation is the toolkit's responsibility (on selection / on invalidation).
										UE::Dataflow::FContextThreaded EmptyContext;
										UE::Dataflow::FContext& EffectiveContext = Context.IsValid() ? *Context : EmptyContext;
										if (const FDataflowOutput* const ConnectedOutput = DataflowInput->GetConnection())
										{
											return ConnectedOutput->ReadValue(EffectiveContext, EmptyCollection);
										}
										return EmptyCollection;
									}
								}
							}
						}
					}
				}
			}
		}
		return EmptyCollection;
	}

	void FPropertyListCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
	{
		DataflowGraphEditor = SDataflowGraphEditor::GetSelectedGraphEditor();

		uint32 NumChildren;
		const FPropertyAccess::Result Result = PropertyHandle->GetNumChildren(NumChildren);

		ChildPropertyHandle = (Result == FPropertyAccess::Success && NumChildren) ? PropertyHandle->GetChildHandle(0) : nullptr;

		ListNames.Reset();

		HeaderRow
			.NameContent()
			[
				PropertyHandle->CreatePropertyNameWidget(PropertyHandle->GetPropertyDisplayName())
			]
			.ValueContent()
			.MinDesiredWidth(250)
			.MaxDesiredWidth(350.f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.MaxWidth(145.f)
				[
					SAssignNew(ComboButton, SComboButton)
					.ButtonStyle(FAppStyle::Get(), "NoBorder")
					.ContentPadding(0)
					.OnGetMenuContent(this, &FPropertyListCustomization::OnGetMenuContent)
					.ButtonContent()
					[
						SNew(SEditableTextBox)
						.Text(this, &FPropertyListCustomization::GetText)
						.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
						.OnTextCommitted(this, &FPropertyListCustomization::OnTextCommitted)
						.OnVerifyTextChanged(this, &FPropertyListCustomization::OnVerifyTextChanged)
					]
				]
			];
	}

	FText FPropertyListCustomization::GetText() const
	{
		FText Text;
		if (ChildPropertyHandle)
		{
			ChildPropertyHandle->GetValueAsFormattedText(Text);
		}
		return Text;
	}

	void FPropertyListCustomization::OnTextCommitted(const FText& NewText, ETextCommit::Type CommitInfo)
	{
		if (ChildPropertyHandle)
		{
			FText CurrentText;
			ChildPropertyHandle->GetValueAsFormattedText(CurrentText);

			if (!NewText.ToString().Equals(CurrentText.ToString(), ESearchCase::CaseSensitive))
			{
				FString String = NewText.ToString();
				FText UnusedErrorString;
				MakeValidListName(String, UnusedErrorString);
				ChildPropertyHandle->SetValueFromFormattedString(String);
			}
		}
	}

	void FPropertyListCustomization::OnSelectionChanged(TSharedPtr<FText> ItemSelected, ESelectInfo::Type /*SelectInfo*/)
	{
		if (ChildPropertyHandle)
		{
			// Set the child property's value
			if (ItemSelected)
			{
				FText CurrentText;
				ChildPropertyHandle->GetValueAsFormattedText(CurrentText);

				if (!ItemSelected->EqualTo(CurrentText))
				{
					ChildPropertyHandle->SetValueFromFormattedString(ItemSelected->ToString());
				}

				if (TSharedPtr<SComboButton> PinnedComboButton = ComboButton.Pin())
				{
					PinnedComboButton->SetIsOpen(false);
				}
			}
		}
	}

	bool FPropertyListCustomization::OnVerifyTextChanged(const FText& Text, FText& OutErrorMessage)
	{
		FString TextString = Text.ToString();
		return MakeValidListName(TextString, OutErrorMessage);
	}

	TSharedRef<ITableRow> FPropertyListCustomization::MakeCategoryViewWidget(TSharedPtr<FText> Item, const TSharedRef< STableViewBase >& OwnerTable)
	{
		if (Item)
		{
			return SNew(STableRow<TSharedPtr<FString>>, OwnerTable)
				.Padding(4)
			[
				SNew(STextBlock)
				.Text(*Item)
				.Font(IDetailLayoutBuilder::GetDetailFont())
			];
		}
		else
		{
			return SNew(STableRow<TSharedPtr<FString>>, OwnerTable);
		}
	}

	TSharedRef<SWidget> FPropertyListCustomization::OnGetMenuContent()
	{
		const TSharedPtr<const SDataflowGraphEditor> DataflowGraphEditorPtr = DataflowGraphEditor.Pin();
		const TSharedPtr<UE::Dataflow::FContext> Context = DataflowGraphEditorPtr ? DataflowGraphEditorPtr->GetDataflowContext() : TSharedPtr<UE::Dataflow::FContext>();

		ListNames.Reset();

		// Retrieve collection
		const FManagedArrayCollection& Collection = GetPropertyCollection(Context, ChildPropertyHandle, GetCollectionPropertyName());

		const TArray<FName> TargetListNames = OnGetListNames.IsBound() ? OnGetListNames.Execute(Collection) : TArray<FName>();
		for (const FName& ListName : TargetListNames)
		{
			ListNames.Add(MakeShareable(new FText(FText::FromName(ListName))));
		}

		return SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.MaxHeight(400.0f)
			[
				SNew(SListView<TSharedPtr<FText>>)
					.ListItemsSource(&ListNames)
					.SelectionMode(ESelectionMode::Single)
					.OnGenerateRow(this, &FPropertyListCustomization::MakeCategoryViewWidget)
					.OnSelectionChanged(this, &FPropertyListCustomization::OnSelectionChanged)
			];
	}
}  // End namespace UE::Chaos::ClothAsset
