// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Dataflow/DataflowGraphSchemaAction.h"

#include "SGraphActionMenu.h"
#include "EdGraph/EdGraphSchema.h"
#include "Templates/SharedPointer.h"
#include "Widgets/Images/SImage.h"

class SDataflowGraphActionWidget : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SDataflowGraphActionWidget) {}
		SLATE_ARGUMENT(TSharedPtr<SWidget>, NameWidget)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const FCreateWidgetForActionData* const InCreateData)
	{
		check(InCreateData->Action.IsValid());
		ActionPtr = InCreateData->Action;
		MouseButtonDownDelegate = InCreateData->MouseButtonDownDelegate;

		// Icon widget
		TSharedRef<SWidget> IconWidget = SNullWidget::NullWidget;
		if (const FSlateBrush* const PrimaryIcon = InCreateData->Action->GetPaletteIcon())
		{
			SAssignNew(IconWidget, SImage)
			.DesiredSizeOverride(FVector2D(16))
			.Image(PrimaryIcon);
		}

		// Name
		const TSharedRef<SWidget> NameWidget =
			SNew(STextBlock)
			.Font(FCoreStyle::GetDefaultFontStyle("Regular", 9))
			.Text(InCreateData->Action->GetMenuDescription())
			;

		this->ChildSlot
		[
			SNew(SHorizontalBox)
			.ToolTipText(InCreateData->Action->GetTooltipDescription())
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				IconWidget
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(4, 0, 0, 0)
			[
				NameWidget
			]
		];
	}

	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		if (MouseButtonDownDelegate.Execute(ActionPtr))
		{
			return FReply::Handled();
		}

		return FReply::Unhandled();
	}

	/** The action that we want to display with this widget */
	TWeakPtr<FEdGraphSchemaAction> ActionPtr;
	/** Delegate executed when mouse button goes down */
	FCreateWidgetMouseButtonDown MouseButtonDownDelegate;
};
