// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/SDataflowTemplatePicker.h"

#include "ContentBrowserModule.h"
#include "IContentBrowserSingleton.h"
#include "Styling/AppStyle.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/Text/STextBlock.h"

#include "Dataflow/DataflowObject.h"

#define LOCTEXT_NAMESPACE "DataflowTemplatePicker"

namespace DataflowTemplatePickerPrivate
{
	static constexpr float TileSize     = 128.f;
	static constexpr float TileIconSize =  64.f;
}

FDataflowPickerResult SDataflowTemplatePicker::ShowModal(
	const TArray<FDataflowTemplateOption>& Templates,
	TSharedPtr<SWindow>                    ParentWindow)
{
	TSharedRef<SWindow> Window = SNew(SWindow)
		.Title(LOCTEXT("DialogTitle", "Choose Dataflow Setup"))
		.SizingRule(ESizingRule::UserSized)
		.ClientSize(FVector2D(640.f, 560.f))
		.AutoCenter(EAutoCenter::PreferredWorkArea)
		.SupportsMaximize(false)
		.SupportsMinimize(false)
		.HasCloseButton(true)
		.MinWidth(480.f)
		.MinHeight(400.f);

	TSharedRef<SDataflowTemplatePicker> Picker =
		SNew(SDataflowTemplatePicker)
		.Templates(Templates)
		.ParentWindow(Window);

	Window->SetContent(Picker);

	FSlateApplication::Get().AddModalWindow(Window, ParentWindow);

	return Picker->GetResult();
}

void SDataflowTemplatePicker::Construct(const FArguments& InArgs)
{
	Templates   = InArgs._Templates;
	OwnerWindow = InArgs._ParentWindow;

	ChildSlot
	[
		SNew(SBox)
		[
			SNew(SVerticalBox)

			// Header
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(16.f, 14.f, 16.f, 10.f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("HeaderLabel", "Choose a Dataflow Graph template"))
				.TextStyle(FAppStyle::Get(), "LargeText")
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SSeparator)
			]

			// Template tiles — fills all space not claimed by the fixed sections below.
			+ SVerticalBox::Slot()
			.FillHeight(1.f)
			.Padding(16.f, 12.f, 16.f, 0.f)
			[
				SNew(SScrollBox)
				.Orientation(Orient_Vertical)
				+ SScrollBox::Slot()
				[
					BuildTemplateGrid()
				]
			]

			// "or pick existing" divider
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(16.f, 14.f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("OrExisting", "Or pick an existing Dataflow Graph"))
				.TextStyle(FAppStyle::Get(), "LargeText")
			]

			// Embedded asset picker
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(16.f, 0.f, 16.f, 4.f)
			[
				BuildAssetPickerSection()
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(FMargin(0.f, 10.f, 0.f, 0.f))
			[
				SNew(SSeparator)
			]

			// Bottom bar
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				BuildBottomBar()
			]
		]
	];
}

///////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> SDataflowTemplatePicker::BuildTemplateGrid()
{
	// SWrapBox automatically wraps tiles to a new row when the dialog is resized.
	TSharedRef<SWrapBox> Grid = SNew(SWrapBox)
		.UseAllottedSize(true)
		.InnerSlotPadding(FVector2D(8.f, 8.f));

	for (int32 i = 0; i < Templates.Num(); ++i)
	{
		Grid->AddSlot()
		[
			MakeTemplateTile(Templates[i], i)
		];
	}

	return Grid;
}

TSharedRef<SWidget> SDataflowTemplatePicker::MakeTemplateTile(
	const FDataflowTemplateOption& Option, int32 Index)
{
	using namespace DataflowTemplatePickerPrivate;

	const FSlateBrush* IconBrush = Option.Icon
		? Option.Icon
		: FAppStyle::GetBrush("ClassIcon.DataTable");   // generic fallback

	// SCheckBox with ToggleButtonCheckbox style gives us free selected-state visuals
	// and radio-button behaviour when IsChecked returns Checked for only one index.
	return SNew(SBox)
		.WidthOverride(TileSize)
		.HeightOverride(TileSize)
		.ToolTipText(Option.Tooltip)
		[
			SNew(SCheckBox)
			.Style(FAppStyle::Get(), "ToggleButtonCheckbox")
			.IsChecked_Lambda([this, Index]()
			{
				return GetTileCheckState(Index);
			})
			.OnCheckStateChanged(this, &SDataflowTemplatePicker::OnTemplateTileChecked, Index)
			.HAlign(HAlign_Center)
			.Padding(6.f)
			[
				SNew(SVerticalBox)

				+ SVerticalBox::Slot()
				.FillHeight(1.f)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(SImage)
					.Image(IconBrush)
					.DesiredSizeOverride(FVector2D(TileIconSize, TileIconSize))
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Center)
				.Padding(0.f, 4.f, 0.f, 0.f)
				[
					SNew(STextBlock)
					.Text(Option.DisplayName)
					.Justification(ETextJustify::Center)
					.WrapTextAt(TileSize - 12.f)
				]
			]
		];
}

TSharedRef<SWidget> SDataflowTemplatePicker::BuildAssetPickerSection()
{
	return SNew(SHorizontalBox)

		// Combo button — shows current selection, opens the inline picker as a dropdown.
		+ SHorizontalBox::Slot()
		.FillWidth(1.f)
		.VAlign(VAlign_Center)
		[
			SAssignNew(AssetComboButton, SComboButton)
			.ButtonStyle(FAppStyle::Get(), "PropertyEditor.AssetComboStyle")
			.ForegroundColor(FAppStyle::GetColor("PropertyEditor.AssetName.ColorAndOpacity"))
			.ContentPadding(FMargin(4.f, 2.f))
			.CollapseMenuOnParentFocus(true)
			.OnGetMenuContent(this, &SDataflowTemplatePicker::MakeAssetPickerMenu)
			.ButtonContent()
			[
				SNew(STextBlock)
				.Text(this, &SDataflowTemplatePicker::GetSelectedAssetLabel)
				.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
			]
		]

		// Clear button — visible only when an asset is selected.
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(2.f, 0.f, 0.f, 0.f)
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "NoBorder")
			.Visibility_Lambda([this]() { return HasAssetSelected() ? EVisibility::Visible : EVisibility::Collapsed; })
			.OnClicked_Lambda([this]() { OnClearAssetSelection(); return FReply::Handled(); })
			.ToolTipText(LOCTEXT("ClearAssetTip", "Clear selected asset"))
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("Icons.X"))
				.DesiredSizeOverride(FVector2D(12.f, 12.f))
			]
		];
}

TSharedRef<SWidget> SDataflowTemplatePicker::MakeAssetPickerMenu()
{
	FAssetPickerConfig Config;
	Config.Filter.ClassPaths.Add(UDataflow::StaticClass()->GetClassPathName());
	Config.bAllowNullSelection  = false;
	Config.bShowBottomToolbar   = false;
	Config.bAutohideSearchBar   = false;
	Config.InitialAssetViewType = EAssetViewType::List;
	Config.OnAssetSelected      = FOnAssetSelected::CreateSP(
	                                  this, &SDataflowTemplatePicker::OnAssetSelected);
	Config.OnAssetDoubleClicked = FOnAssetDoubleClicked::CreateSP(
	                                  this, &SDataflowTemplatePicker::OnAssetDoubleClicked);

	IContentBrowserSingleton& CB =
		FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser").Get();

	return SNew(SBox)
		.WidthOverride(350.f)
		.HeightOverride(300.f)
		[
			CB.CreateAssetPicker(Config)
		];
}

TSharedRef<SWidget> SDataflowTemplatePicker::BuildBottomBar()
{
	return SNew(SHorizontalBox)
		.Clipping(EWidgetClipping::ClipToBounds)

		// Spacer
		+ SHorizontalBox::Slot()
		.FillWidth(1.f)

		// Right: No Dataflow + OK buttons
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		.AutoWidth()
		.Padding(8.f, 10.f, 16.f, 10.f)
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0.f, 0.f, 8.f, 0.f)
			[
				SNew(SButton)
				.Text(LOCTEXT("NoneBtn", "No Dataflow"))
				.ToolTipText(LOCTEXT("NoneTip", "Continue without attaching a Dataflow graph"))
				.OnClicked(this, &SDataflowTemplatePicker::OnNoDataflowClicked)
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SButton)
				.Text(LOCTEXT("OKBtn", "OK"))
				.ButtonStyle(FAppStyle::Get(), "PrimaryButton")
				.IsEnabled(this, &SDataflowTemplatePicker::IsOKEnabled)
				.OnClicked(this, &SDataflowTemplatePicker::OnOKClicked)
			]
		];
}

void SDataflowTemplatePicker::OnTemplateTileChecked(ECheckBoxState NewState, int32 Index)
{
	if (NewState == ECheckBoxState::Checked)
	{
		CandidateResult.Type          = EDataflowPickerResult::Template;
		CandidateResult.TemplateIndex = Index;
		CandidateResult.TemplateId    = Templates[Index].TemplateId;
		CandidateResult.SelectedAsset = FAssetData();
	}
	else
	{
		// Unchecking the active tile clears the template selection.
		if (CandidateResult.TemplateIndex == Index)
		{
			CandidateResult = FDataflowPickerResult{};
		}
	}
}

void SDataflowTemplatePicker::OnAssetSelected(const FAssetData& AssetData)
{
	CandidateResult.Type          = EDataflowPickerResult::ExistingAsset;
	CandidateResult.SelectedAsset = AssetData;
	CandidateResult.TemplateIndex = INDEX_NONE;
	CandidateResult.TemplateId    = NAME_None;

	if (TSharedPtr<SComboButton> Combo = AssetComboButton.Pin())
	{
		Combo->SetIsOpen(false);
	}
}

void SDataflowTemplatePicker::OnAssetDoubleClicked(const FAssetData& AssetData)
{
	OnAssetSelected(AssetData);
	OnOKClicked();
}

void SDataflowTemplatePicker::OnClearAssetSelection()
{
	if (CandidateResult.Type == EDataflowPickerResult::ExistingAsset)
	{
		CandidateResult = FDataflowPickerResult{};
	}
}

FText SDataflowTemplatePicker::GetSelectedAssetLabel() const
{
	if (CandidateResult.Type == EDataflowPickerResult::ExistingAsset && CandidateResult.SelectedAsset.IsValid())
	{
		return FText::FromName(CandidateResult.SelectedAsset.AssetName);
	}
	return LOCTEXT("NoAssetSelected", "None");
}

bool SDataflowTemplatePicker::HasAssetSelected() const
{
	return CandidateResult.Type == EDataflowPickerResult::ExistingAsset && CandidateResult.SelectedAsset.IsValid();
}

FReply SDataflowTemplatePicker::OnNoDataflowClicked()
{
	// Ignore CandidateResult
	Result = { EDataflowPickerResult::None, INDEX_NONE, NAME_None, FAssetData() };
	CloseOwnerWindow();
	return FReply::Handled();
}

FReply SDataflowTemplatePicker::OnOKClicked()
{
	// Result already populated by tile / asset selection callbacks.
	Result = CandidateResult;
	CloseOwnerWindow();
	return FReply::Handled();
}

FReply SDataflowTemplatePicker::OnCancelClicked()
{
	Result = FDataflowPickerResult{};   // default: Cancelled
	CloseOwnerWindow();
	return FReply::Handled();
}

void SDataflowTemplatePicker::CloseOwnerWindow()
{
	if (TSharedPtr<SWindow> Window = OwnerWindow.Pin())
	{
		Window->RequestDestroyWindow();
	}
}

ECheckBoxState SDataflowTemplatePicker::GetTileCheckState(int32 Index) const
{
	return (CandidateResult.Type == EDataflowPickerResult::Template && CandidateResult.TemplateIndex == Index)
		? ECheckBoxState::Checked
		: ECheckBoxState::Unchecked;
}

bool SDataflowTemplatePicker::IsOKEnabled() const
{
	switch (CandidateResult.Type)
	{
	case EDataflowPickerResult::Template:
		return Templates.IsValidIndex(CandidateResult.TemplateIndex);
	case EDataflowPickerResult::ExistingAsset:
		return CandidateResult.SelectedAsset.IsValid();
	default:
		return false;
	}
}

#undef LOCTEXT_NAMESPACE
