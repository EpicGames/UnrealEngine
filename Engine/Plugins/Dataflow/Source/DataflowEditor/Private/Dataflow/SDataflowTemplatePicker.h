// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AssetRegistry/AssetData.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Input/SComboButton.h"
#include "Dataflow/DataflowTemplateRegistry.h"

///////////////////////////////////////////////////////////////////////////////////////////////////

/** Discriminator for FDataflowPickerResult::Type. */
enum class EDataflowPickerResult : uint8
{
	/** User selected a template tile. TemplateIndex and TemplateId are valid. */
	Template,
	/** User picked an existing asset from the content browser. SelectedAsset is valid. */
	ExistingAsset,
	/** User clicked "No Dataflow". */
	None,
	/** User cancelled or closed the window. */
	Cancelled,
};

///////////////////////////////////////////////////////////////////////////////////////////////////

struct FDataflowPickerResult
{
	EDataflowPickerResult Type          = EDataflowPickerResult::Cancelled;
	int32                 TemplateIndex = INDEX_NONE;
	FName                 TemplateId;
	FAssetData            SelectedAsset;
};

///////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * Modal dialog that lets the user:
 *   • choose one of several Dataflow templates (name + icon tiles),
 *   • pick an existing Dataflow asset from the content browser, or
 *   • opt out ("No Dataflow").
 *
 * Typical usage:
 *   FDataflowPickerResult Result = SDataflowTemplatePicker::ShowModal(Templates, ParentWindow);
 */
class DATAFLOWEDITOR_API SDataflowTemplatePicker : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SDataflowTemplatePicker)
		: _ParentWindow()
	{}
		SLATE_ARGUMENT(TArray<FDataflowTemplateOption>, Templates)
		SLATE_ARGUMENT(TWeakPtr<SWindow>,               ParentWindow)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	/**
	 * Creates a modal window, blocks until dismissed, then returns the result.
	 * Pass a valid ParentWindow to centre the dialog; nullptr uses the main window.
	 */
	static FDataflowPickerResult ShowModal(
		const TArray<FDataflowTemplateOption>& Templates,
		TSharedPtr<SWindow>                    ParentWindow = nullptr);

	const FDataflowPickerResult& GetResult() const { return Result; }

private:
	// Build helpers
	TSharedRef<SWidget> BuildTemplateGrid();
	TSharedRef<SWidget> MakeTemplateTile(const FDataflowTemplateOption& Option, int32 Index);
	TSharedRef<SWidget> BuildAssetPickerSection();
	TSharedRef<SWidget> MakeAssetPickerMenu();
	TSharedRef<SWidget> BuildBottomBar();

	// Interaction
	void    OnTemplateTileChecked(ECheckBoxState NewState, int32 Index);
	void    OnAssetSelected(const FAssetData& AssetData);
	void    OnAssetDoubleClicked(const FAssetData& AssetData);
	void    OnClearAssetSelection();
	FReply  OnNoDataflowClicked();
	FReply  OnOKClicked();
	FReply  OnCancelClicked();
	void    CloseOwnerWindow();

	// Attribute getters (used by Slate attribute bindings)
	ECheckBoxState GetTileCheckState(int32 Index) const;
	bool           IsOKEnabled() const;
	FText          GetSelectedAssetLabel() const;
	bool           HasAssetSelected() const;

	// State
	TArray<FDataflowTemplateOption> Templates;
	TWeakPtr<SWindow>               OwnerWindow;
	TWeakPtr<SComboButton>          AssetComboButton;
	FDataflowPickerResult           CandidateResult;
	FDataflowPickerResult           Result;
};
