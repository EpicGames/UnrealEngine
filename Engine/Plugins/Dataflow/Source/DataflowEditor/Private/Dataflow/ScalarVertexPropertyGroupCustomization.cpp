// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/ScalarVertexPropertyGroupCustomization.h"
#include "Dataflow/DataflowCollectionAddScalarVertexPropertyNode.h"

namespace UE::Dataflow
{
	TSharedRef<IPropertyTypeCustomization> FScalarVertexPropertyGroupCustomization::MakeInstance()
	{
		return MakeShareable(new FScalarVertexPropertyGroupCustomization);
	}

	TArray<FName> FScalarVertexPropertyGroupCustomization::GetTargetGroupNames(const FManagedArrayCollection& Collection) const
	{
		return FDataflowAddScalarVertexPropertyCallbackRegistry::Get().GetTargetGroupNames();
	}

	FName FScalarVertexPropertyGroupCustomization::GetCollectionPropertyName() const
	{
		return GET_MEMBER_NAME_CHECKED(FDataflowCollectionAddScalarVertexPropertyNode, Collection);
	}
}  // End namespace UE::Chaos::ClothAsset
