// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Dataflow/PropertyGroupCustomization.h"

namespace UE::Dataflow
{
	/**
	 * Customization for scalara vertex node group names input.
	 */
	class FScalarVertexPropertyGroupCustomization : public FPropertyGroupCustomization
	{
	public:
		static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	private:
		//~ Begin FPropertyGroupCustomization interface
		virtual TArray<FName> GetTargetGroupNames(const FManagedArrayCollection& Collection) const override;
		virtual FName GetCollectionPropertyName() const override;
		//~ End FPropertyGroupCustomization interface
	};
}
