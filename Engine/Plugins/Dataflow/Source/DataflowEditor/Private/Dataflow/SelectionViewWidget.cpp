// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/SelectionViewWidget.h"
#include "Widgets/Input/SButton.h"
#include "Styling/StarshipCoreStyle.h"
#include "Widgets/Input/SCheckBox.h"

#define LOCTEXT_NAMESPACE "SelectionViewWidget"

const FName FSelectionViewHeader::IndexColumnNameTransform = FName("Transform Index");
const FName FSelectionViewHeader::IndexColumnNameFace = FName("Face Index");
const FName FSelectionViewHeader::IndexColumnNameVertex = FName("Vertex Index");
const FName FSelectionViewHeader::SelectionStatusColumnName = FName("Selection Status");

const FName FSelectionViewItem::SelectedName = FName("Selected");
const FName FSelectionViewItem::NotSelectedName = FName("-");


void SSelectionViewRow::Construct(const FArguments& InArgs, TSharedRef<STableViewBase> OwnerTableView, const TSharedPtr<const FSelectionViewHeader>& InHeader, const TSharedPtr<const FSelectionViewItem>& InItem)
{
	Header = InHeader;
	Item = InItem;

	SMultiColumnTableRow<TSharedPtr<const FSelectionViewItem>>::Construct(
		FSuperRowType::FArguments()
		.Style(&FStarshipCoreStyle::GetCoreStyle().GetWidgetStyle<FTableRowStyle>("TableView.AlternatingRow")),
		OwnerTableView);
}


TSharedRef<SWidget> SSelectionViewRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	int32 FoundIndex;
	if (Header->ColumnNames.Find(ColumnName, FoundIndex))
	{
		const FString& AttrValue = Item->Values[FoundIndex];

		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(FText::FromString(AttrValue))
				.ShadowColorAndOpacity(FLinearColor(0.1f, 0.1f, 0.1f, 1.f))
				.Visibility(EVisibility::Visible)
				.Font_Lambda([this]() -> FSlateFontInfo
				{
					if (Item->Values[1] == FSelectionViewItem::SelectedName.ToString())
					{
						return FCoreStyle::GetDefaultFontStyle("Bold", 10);
					}

					return FCoreStyle::GetDefaultFontStyle("Regular", 10);
				})
			];
	}

	return SNullWidget::NullWidget;
}

//
// ----------------------------------------------------------------------------
//

void SSelectionView::Construct(const FArguments& InArgs)
{
	SelectedOutput = InArgs._SelectedOutput;

	HeaderRowWidget =
		SNew(SHeaderRow)
		.Visibility(EVisibility::Visible);

	if (!SelectionInfoMap.IsEmpty())
	{
		RegenerateHeader();
		RepopulateListView();
	}

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.Padding(FMargin(0.0f, 3.f))
		[
			SNew(SOverlay)
			+ SOverlay::Slot()
			[
				SAssignNew(ListView, SListView<TSharedPtr<const FSelectionViewItem>>)
				.SelectionMode(ESelectionMode::Multi)
				.ListItemsSource(&ListItems)
				.OnGenerateRow(this, &SSelectionView::GenerateRow)
				.HeaderRow(HeaderRowWidget)
			]
		]
	];
}


void SSelectionView::SetSelectedOutput(const FName& InSelectedOutput)
{
	SelectedOutput = InSelectedOutput;

	RegenerateHeader();
	RepopulateListView();
}


const FName& SSelectionView::GetSelectedOutput() const
{
	return SelectedOutput;
}


void SSelectionView::RegenerateHeader()
{
	HeaderRowWidget->ClearColumns();

	Header = MakeShared<FSelectionViewHeader>();

	if (SelectionInfoMap.Num() > 0 && !SelectedOutput.IsNone() && SelectedOutput.ToString() != "")
	{
		if (SelectionInfoMap[SelectedOutput.ToString()].OutputType == "FDataflowTransformSelection")
		{
			Header->ColumnNames.Add(FSelectionViewHeader::IndexColumnNameTransform);
		}
		else if (SelectionInfoMap[SelectedOutput.ToString()].OutputType == "FDataflowVertexSelection")
		{
			Header->ColumnNames.Add(FSelectionViewHeader::IndexColumnNameVertex);
		}
		else if (SelectionInfoMap[SelectedOutput.ToString()].OutputType == "FDataflowFaceSelection")
		{
			Header->ColumnNames.Add(FSelectionViewHeader::IndexColumnNameFace);
		}
	}
	else
	{
		Header->ColumnNames.Add(FName(" "));
	}

	Header->ColumnNames.Add(FSelectionViewHeader::SelectionStatusColumnName);

	constexpr float CustomFillWidth = 1.0f;

	for (int32 AttributeNameIndex = 0; AttributeNameIndex < Header->ColumnNames.Num(); ++AttributeNameIndex)
	{
		const FName& ColumnName = Header->ColumnNames[AttributeNameIndex];

		HeaderRowWidget->AddColumn(
			SHeaderRow::Column(ColumnName)
			.DefaultLabel(FText::FromName(ColumnName))
			.FillWidth(CustomFillWidth)
			.HAlignCell(HAlign_Center)
			.HAlignHeader(HAlign_Center)
			.VAlignCell(VAlign_Center)
		);
	}
}


void SSelectionView::RepopulateListView()
{
	ListItems.Empty();

	if (SelectionInfoMap.Num() > 0 && !SelectedOutput.IsNone() && SelectedOutput.ToString() != "")
	{
		const int32 NumElems = SelectionInfoMap[SelectedOutput.ToString()].SelectionArray.Num();
		int32 NumSelectedElems = 0;

		for (int32 Idx = 0; Idx < NumElems; ++Idx)
		{
			const TSharedPtr<FSelectionViewItem> NewItem = MakeShared<FSelectionViewItem>();
			NewItem->Values.SetNum(Header->ColumnNames.Num());

			for (int32 ColumnIdx = 0; ColumnIdx < Header->ColumnNames.Num(); ++ColumnIdx)
			{
				const FName& ColumnName = Header->ColumnNames[ColumnIdx];
				if (ColumnName == FSelectionViewHeader::SelectionStatusColumnName)
				{
					if (SelectionInfoMap[SelectedOutput.ToString()].SelectionArray[Idx])
					{
						NewItem->Values[ColumnIdx] = FSelectionViewItem::SelectedName.ToString();

						NumSelectedElems++;
					}
					else
					{
						NewItem->Values[ColumnIdx] = FSelectionViewItem::NotSelectedName.ToString();
					}
				}
				else
				{
					NewItem->Values[ColumnIdx] = FString::FromInt(Idx);
				}
			}

			ListItems.Add(NewItem);
		}

		NumItems = NumElems;
		NumSelectedItems = NumSelectedElems;
	}

	ListView->RequestListRefresh();
}

TSharedRef<ITableRow> SSelectionView::GenerateRow(TSharedPtr<const FSelectionViewItem> InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	const FTableRowStyle AlternatingTableRowStyle = FStarshipCoreStyle::GetCoreStyle().GetWidgetStyle<FTableRowStyle>("TableView.AlternatingRow");

	TSharedRef<SSelectionViewRow> NewSelectionViewRow = SNew(SSelectionViewRow, OwnerTable, this->Header, InItem);

	return NewSelectionViewRow;
}


//
// ----------------------------------------------------------------------------
//

void SSelectionViewWidget::NodeOutputsComboBoxSelectionChanged(FName InSelectedOutput, ESelectInfo::Type InSelectInfo)
{
	if (SelectionTable)
	{
		if (SelectionTable->GetSelectedOutput() != InSelectedOutput)
		{
			SelectionTable->SetSelectedOutput(InSelectedOutput);

			NodeOutputsComboBoxLabel->SetText(FText::FromName(SelectionTable->GetSelectedOutput()));

			SetStatusText();
		}
	}
}

const FSlateBrush* SSelectionViewWidget::GetPinButtonImage() const
{
	if (bIsPinnedDown)
	{
		return FAppStyle::Get().GetBrush("Icons.Pinned");
	}
	else
	{
		return FAppStyle::Get().GetBrush("Icons.Unpinned");
	}
}

const FSlateBrush* SSelectionViewWidget::GetLockButtonImage() const
{
	if (bIsRefreshLocked)
	{
		return FAppStyle::Get().GetBrush("Icons.Lock");
	}
	else
	{
		return FAppStyle::Get().GetBrush("Icons.Unlock");
	}
}

void SSelectionViewWidget::Construct(const FArguments& InArgs)
{
	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			// Node: [ MakeBox_0                 ] [O]
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(10.0f, 10.0f, 0.0f, 5.0f)
			[
				SNew(STextBlock)
				.Text(FText::FromString("Node: "))
			]

			+ SHorizontalBox::Slot()
			.FillWidth(1.f)
			.Padding(10.0f, 10.0f, 10.0f, 5.0f)
			[
				SAssignNew(NodeNameTextBlock, STextBlock)
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(10.0f, 10.0f, 4.0f, 5.0f)
			[
				SNew(SCheckBox)
				.Style(FAppStyle::Get(), "ToggleButtonCheckbox")
				.ToolTipText(LOCTEXT("PinDownButtonToolTip", "The button pins down the panel. When it pinned down it doesn't react to node selection change."))
				.IsChecked_Lambda([this]()-> ECheckBoxState
				{
					if (bIsPinnedDown)
					{
						return ECheckBoxState::Checked;
					}
					return ECheckBoxState::Unchecked;
				})
				.OnCheckStateChanged_Lambda([this](ECheckBoxState State)
				{
					if (!NodeNameTextBlock->GetText().IsEmpty())
					{
						bIsPinnedDown = !bIsPinnedDown;
						OnPinnedDownChangedDelegate.Broadcast(bIsPinnedDown);
					}
				})
				.Padding(2.0f)
				.HAlign(HAlign_Center)
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(this, &SSelectionViewWidget::GetPinButtonImage)
				]
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(4.0f, 10.0f, 10.0f, 5.0f)
			[
				SNew(SCheckBox)
				.Style(FAppStyle::Get(), "ToggleButtonCheckbox")
				.ToolTipText(LOCTEXT("LockRefreshButtonToolTip", "The button locks the refresh of the values in the panel."))
				.IsChecked_Lambda([this]()-> ECheckBoxState
				{
					if (bIsRefreshLocked)
					{
						return ECheckBoxState::Checked;
					}
					return ECheckBoxState::Unchecked;
				})
				.OnCheckStateChanged_Lambda([this](ECheckBoxState State)
				{
					if (!NodeNameTextBlock->GetText().IsEmpty())
					{
						bIsRefreshLocked = !bIsRefreshLocked;
						OnRefreshLockedChangedDelegate.Broadcast(bIsRefreshLocked);
					}
				})
				.Padding(2.0f)
				.HAlign(HAlign_Center)
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(this, &SSelectionViewWidget::GetLockButtonImage)
				]
			]
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			// Output: [ TransformSelection        |V|]
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(10.0f, 0.0f, 0.0f, 10.0f)
			[
				SNew(STextBlock)
				.Text(FText::FromString("Output: "))
			]

			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.Padding(10.0f, 0.0f, 10.0f, 10.0f)
			[
				SAssignNew(NodeOutputsComboBox, SComboBox<FName>)
				.ToolTipText(LOCTEXT("NodeOutputsToolTip", "Select a node output to see the output's data"))
				.OptionsSource(&NodeOutputs)
				.OnGenerateWidget(SComboBox<FName>::FOnGenerateWidget::CreateLambda([](FName Item)->TSharedRef<SWidget>
				{
					return SNew(STextBlock)
						.Text(FText::FromName(Item));
				}))
				.OnSelectionChanged(this, &SSelectionViewWidget::NodeOutputsComboBoxSelectionChanged)
				[
					SAssignNew(NodeOutputsComboBoxLabel, STextBlock)
					.Text(GetNoOutputText())
				]
			]
		]

		+ SVerticalBox::Slot()
		[
			SAssignNew(SelectionTable, SSelectionView)
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(10.0f, 10.0f, 0.0f, 5.0f)
			[
				SAssignNew(StatusTextBlock, STextBlock)
			]
		]
	];
}


void SSelectionViewWidget::SetData(const FString& InNodeName)
{
	NodeName = InNodeName;

	NodeOutputs.Empty();

	if (!NodeName.IsEmpty())
	{
		if (SelectionTable->GetSelectionInfoMap().Num() > 0)
		{
			for (auto& Info : SelectionTable->GetSelectionInfoMap())
			{
				NodeOutputs.Add(FName(*Info.Key));
			}
		}
	}
}


void SSelectionViewWidget::RefreshWidget()
{
	NodeNameTextBlock->SetText(FText::FromString(NodeName));

	NodeOutputsComboBox->RefreshOptions();
	NodeOutputsComboBox->ClearSelection();

	if (NodeOutputs.Num() > 0)
	{
		NodeOutputsComboBox->SetSelectedItem(NodeOutputs[0]);
	}
	else
	{
		NodeOutputsComboBoxLabel->SetText(GetNoOutputText());
	}
}


void SSelectionViewWidget::SetStatusText()
{
	if (!NodeName.IsEmpty())
	{
		FString Str = FString::Printf(TEXT("Selected %d of %d"), SelectionTable->GetNumSelectedItems(), SelectionTable->GetNumItems());
		StatusTextBlock->SetText(FText::FromString(Str));
	}
	else
	{
		StatusTextBlock->SetText(FText::FromString(" "));
	}
}

FText SSelectionViewWidget::GetNoOutputText()
{
	return LOCTEXT("NoOutput", "No Output(s)");
}

#undef LOCTEXT_NAMESPACE
