// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Dataflow/DataflowSelection.h"
#include "DetailLayoutBuilder.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SComboBox.h"

/**
* Struct to hold OutputType/Selection data for the outputs
* Data is stored in a map using the OutputName as key: TMap<FString, SSelectionInfo>
*/
struct SSelectionInfo
{
	FString OutputType;
	const TBitArray<> SelectionArray;
};


/** 
* Header
* 1st column possible values: Transform Index/Face Index/Vertex Index based on the OutputType
* 2nd column is SelectionStatus
*/
struct FSelectionViewHeader
{
	static const FName IndexColumnNameTransform;
	static const FName IndexColumnNameFace;
	static const FName IndexColumnNameVertex;
	static const FName SelectionStatusColumnName;

	TArray<FName> ColumnNames;
};


/** 
* Representing a row in the table 
* Index/SelectionStatus
*/
struct FSelectionViewItem
{
	static const FName SelectedName;
	static const FName NotSelectedName;

	TArray<FString> Values;
};


/** 
* 
*/
class SSelectionViewRow : public SMultiColumnTableRow<TSharedPtr<const FSelectionViewItem>>
{
public:
	SLATE_BEGIN_ARGS(SSelectionViewRow) 
	{}
		SLATE_ARGUMENT(TSharedPtr<const FSelectionViewHeader>, Header)
		SLATE_ARGUMENT(TSharedPtr<const FSelectionViewItem>, Item)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedRef<STableViewBase> OwnerTableView, const TSharedPtr<const FSelectionViewHeader>& InHeader, const TSharedPtr<const FSelectionViewItem>& InItem);

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

private:
	TSharedPtr<const FSelectionViewHeader> Header;
	TSharedPtr<const FSelectionViewItem> Item;
};

/** 
* 2xn grid to display Index/SelectionStatus data
*/
class SSelectionView : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SSelectionView) {}
		SLATE_ARGUMENT(FName, SelectedOutput)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	TMap<FString, SSelectionInfo>& GetSelectionInfoMap() { return SelectionInfoMap; }

	const FName& GetSelectedOutput() const;
	void SetSelectedOutput(const FName& InSelectedOutput);
	
	int32 GetNumItems() const { return NumItems; }
	int32 GetNumSelectedItems() const { return NumSelectedItems; }

	TSharedRef<ITableRow> GenerateRow(TSharedPtr<const FSelectionViewItem> InItem, const TSharedRef<STableViewBase>& OwnerTable);

private:
	FName SelectedOutput = FName(TEXT(""));
	TMap<FString, SSelectionInfo> SelectionInfoMap;

	TSharedPtr<SListView<TSharedPtr<const FSelectionViewItem>>> ListView;
	TArray<TSharedPtr<const FSelectionViewItem>> ListItems;

	TSharedPtr<FSelectionViewHeader> Header;
	TSharedPtr<SHeaderRow> HeaderRowWidget;

	int32 NumItems = 0, NumSelectedItems = 0;
	void RegenerateHeader();
	void RepopulateListView();
};

/**
* Widget for the SelectionView panel
*/
class SSelectionViewWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SSelectionViewWidget)
	{}
	SLATE_END_ARGS()

	/**
	* Slate construction function
	*
	* @param InArgs - Arguments passed from Slate
	*
	*/
	void Construct(const FArguments& InArgs);

	void SetData(const FString& InNodeName);
	void RefreshWidget();
	TSharedPtr<SSelectionView> GetSelectionTable() { return SelectionTable; }
	void SetStatusText();

	// UI callbacks
	void NodeOutputsComboBoxSelectionChanged(FName InSelectedOutput, ESelectInfo::Type InSelectInfo);
	FText GetNoOutputText();

	/** Gets a multicast delegate which is called whenever the PinnedDown button clicked */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnPinnedDownChanged, const bool);
	FOnPinnedDownChanged& GetOnPinnedDownChangedDelegate() { return OnPinnedDownChangedDelegate; }

	/** Gets a multicast delegate which is called whenever the LockRefresh button clicked */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnRefreshLockedChanged, const bool);
	FOnRefreshLockedChanged& GetOnRefreshLockedChangedDelegate() { return OnRefreshLockedChangedDelegate; }

	const FSlateBrush* GetPinButtonImage() const;
	const FSlateBrush* GetLockButtonImage() const;

private:
	TSharedPtr<STextBlock> NodeNameTextBlock;
	TSharedPtr<SComboBox<FName>> NodeOutputsComboBox;
	TSharedPtr<STextBlock> NodeOutputsComboBoxLabel;
	TSharedPtr<SSelectionView> SelectionTable;
	TSharedPtr<STextBlock> StatusTextBlock;

	FString NodeName;
	TArray<FName> NodeOutputs;
	bool bIsPinnedDown = false;
	bool bIsRefreshLocked = false;

protected:
	FOnPinnedDownChanged OnPinnedDownChangedDelegate;
	FOnRefreshLockedChanged OnRefreshLockedChangedDelegate;

};