// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataflowEditorTools/DataflowDynamicMeshMapping.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/NonManifoldMappingSupport.h"

namespace UE::Dataflow
{
	void FDynamicMeshMapping::Init(const UE::Geometry::FDynamicMesh3& Mesh)
	{
		Reset();

		const int32 MeshMaxVertexID = Mesh.MaxVertexID();
		if (MeshMaxVertexID <= 0)
		{
			return;
		}

		using namespace UE::Geometry;
		const FNonManifoldMappingSupport NonManifoldMapping(Mesh);

		bHasNonManifoldMapping = NonManifoldMapping.IsNonManifoldVertexInSource();
		if (bHasNonManifoldMapping)
		{
			int32 MinSourceVertID = TNumericLimits<int32>::Max();
			int32 MaxSourceVertID = 0;
			DynamicMeshToSource.SetNumUninitialized(MeshMaxVertexID);
			for (int32 DynamicMeshVertID = 0; DynamicMeshVertID < MeshMaxVertexID; ++DynamicMeshVertID)
			{
				if (Mesh.IsVertex(DynamicMeshVertID))
				{
					const int32 SourceVertID = NonManifoldMapping.GetOriginalNonManifoldVertexID(DynamicMeshVertID);
					MinSourceVertID = FMath::Min(MinSourceVertID, SourceVertID);
					MaxSourceVertID = FMath::Max(MaxSourceVertID, SourceVertID);
					DynamicMeshToSource[DynamicMeshVertID] = SourceVertID;
				}
				else
				{
					DynamicMeshToSource[DynamicMeshVertID] = INDEX_NONE;
				}
			}

			if (MaxSourceVertID >= MinSourceVertID)
			{
				// This number is not the total number of verts in the original mesh but the number of source verts mapped to the dynamic mesh
				const int32 NumSourceVerts = MaxSourceVertID - MinSourceVertID + 1;

				SourceVertexIDOffset = MinSourceVertID;
				SourceToDynamicMesh.SetNum(NumSourceVerts);
				for (int32 DynamicMeshVertID = 0; DynamicMeshVertID < MeshMaxVertexID; ++DynamicMeshVertID)
				{
					const int32 SourceVertID = DynamicMeshToSource[DynamicMeshVertID];
					if (SourceVertID != INDEX_NONE)
					{
						const int32 LocalSourceVertID = (SourceVertID - SourceVertexIDOffset);
						if (SourceToDynamicMesh.IsValidIndex(LocalSourceVertID))
						{
							SourceToDynamicMesh[SourceVertID - SourceVertexIDOffset].Add(DynamicMeshVertID);
						}
					}
				}
			}
		}
	}

	void FDynamicMeshMapping::Reset()
	{
		bHasNonManifoldMapping = false;
		DynamicMeshToSource.Reset();
		SourceToDynamicMesh.Reset();
		SourceVertexIDOffset = 0;
	}

	void FDynamicMeshMapping::RemapVertices(const TSet<int32>& InVertices, TSet<int32>& OutVertices, ERemapDirection Direction) const
	{
		OutVertices.Reset();
		if (!bHasNonManifoldMapping)
		{
			// Direct mapping, regardless of the direction
			OutVertices.Append(InVertices);
			return;
		}
		if (Direction == ERemapDirection::DynamicMeshToSource)
		{
			OutVertices.Reserve(InVertices.Num());
			for (const int32 DynamicMeshVertID : InVertices)
			{
				if (DynamicMeshToSource.IsValidIndex(DynamicMeshVertID))
				{
					if (ensure(DynamicMeshToSource[DynamicMeshVertID] != INDEX_NONE))
					{
						OutVertices.Add(DynamicMeshToSource[DynamicMeshVertID]);
					}
				}
			}
		}
		else if (Direction == ERemapDirection::SourceToDynamicMesh)
		{
			OutVertices.Reserve(InVertices.Num());
			for (const int32 SourceVertID : InVertices)
			{
				const int32 Index = (SourceVertID - SourceVertexIDOffset);
				if (SourceToDynamicMesh.IsValidIndex(Index))
				{
					OutVertices.Append(SourceToDynamicMesh[Index]);
				}
			}
		}
	}
}
