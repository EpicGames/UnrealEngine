// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Containers/Array.h"

#define UE_API DATAFLOWEDITOR_API

namespace UE { namespace Geometry { class FDynamicMesh3; } }

namespace UE::Dataflow
{
	/**
	* Represent data to map from the dynamic mesh to the source mesh
	* in some case the dynamic mesh may have more vertices created during its construction than the source
	*/
	struct FDynamicMeshMapping
	{
	public:
		enum class ERemapDirection : uint8
		{
			SourceToDynamicMesh,
			DynamicMeshToSource
		};

		void Init(const UE::Geometry::FDynamicMesh3& Mesh);
		void Reset();

		const TArray<int32>& GetMappedSourceVertices() const { return DynamicMeshToSource; }

		void RemapVertices(const TSet<int32>& InVertices, TSet<int32>& OutVertices, ERemapDirection Direction) const;

	private:
		/** true if the source mesh was non manifold and the dynamic mesh stores a mapping */
		bool bHasNonManifoldMapping = false;

		/**
		* mapping from the dynamic mesh vertices to the source mesh vertices (1 to 1 vertices)
		* empty if bHasNonManifoldMapping = false
		*/
		TArray<int32> DynamicMeshToSource;

		/**
		* source vertex ID that correspond to the first entry of SourceToDynamicMesh
		* this is used to save memory when only a subset of the orignal mesh is respresented by the dynamic mesh
		*/
		int32 SourceVertexIDOffset = 0;

		/**
		* mapping from the source mesh vertices to the dynamic mesh ones (1 to N relationship)
		* empty if bHasNonManifoldMapping = false
		*/
		TArray<TArray<int32>> SourceToDynamicMesh;
	};
}

#undef UE_API
