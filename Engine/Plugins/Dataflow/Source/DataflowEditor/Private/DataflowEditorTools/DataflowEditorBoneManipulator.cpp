// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataflowEditorTools/DataflowEditorBoneManipulator.h"

#include "BaseGizmos/CombinedTransformGizmo.h"
#include "BaseGizmos/TransformGizmoUtil.h"
#include "SkeletalMesh/RefSkeletonPoser.h"
#include "Transforms/TransformGizmoDataBinder.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowEditorBoneManipulator)

//////////////////////////////////////////////////////////////////////////////////////////////////////////////

void UDataflowBoneManipulator::Setup(UInteractiveToolManager* ToolManager, const FReferenceSkeleton& RefSkeleton)
{
	ensure(ToolManager);
	if (!ToolManager)
	{
		return;
	}

	UInteractiveGizmoManager* const GizmoManager = ToolManager->GetPairedGizmoManager();
	ensure(GizmoManager);
	if (!GizmoManager)
	{
		return;
	}

	RefSkeletonPoser = NewObject<URefSkeletonPoser>();
	RefSkeletonPoser->SetRefSkeleton(RefSkeleton);

	// create the transform rotation only gizmo
	TransformProxy = NewObject<UTransformProxy>(this);
	TransformProxy->SetTransform(FTransform::Identity);
	TransformProxy->OnBeginTransformEdit.AddUObject(this, &UDataflowBoneManipulator::OnStartTransformEdit);
	TransformProxy->OnTransformChanged.AddUObject(this, &UDataflowBoneManipulator::OnTransformUpdated);
	TransformProxy->OnEndTransformEdit.AddUObject(this, &UDataflowBoneManipulator::OnEndTransformEdit);
	//TransformProxy->OnTransformChangedUndoRedo.AddUObject(this, &UDataflowBoneManipulator::OnWidgetTransformUpdated);

	// create the rotation only gizmo
	TransformGizmo = UE::TransformGizmoUtil::CreateCustomTransformGizmo(GizmoManager, ETransformGizmoSubElements::FreeRotate | ETransformGizmoSubElements::RotateAllAxes, this);
	TransformGizmo->SetActiveTarget(TransformProxy, ToolManager);
	TransformGizmo->SetVisibility(false);
	TransformGizmo->bUseContextCoordinateSystem = false;
	TransformGizmo->bUseContextGizmoMode = false;
	TransformGizmo->CurrentCoordinateSystem = EToolContextCoordinateSystem::Local;
}

void UDataflowBoneManipulator::Shutdown(UInteractiveToolManager* ToolManager)
{
	if (TransformProxy)
	{
		TransformProxy->OnBeginTransformEdit.RemoveAll(this);
		TransformProxy->OnTransformChanged.RemoveAll(this);
		TransformProxy->OnEndTransformEdit.RemoveAll(this);
	}

	if (ensure(ToolManager))
	{
		ToolManager->GetPairedGizmoManager()->DestroyAllGizmosByOwner(this);
	}
	TransformGizmo = nullptr;
}

void UDataflowBoneManipulator::SetEnabled(bool bInEnabled)
{
	if (bEnabled == bInEnabled)
	{
		return;
	}
	bEnabled = bInEnabled;

	// update gizmo accordingly
	SetSelectedBoneByIndex(BoneIndex);
}

void UDataflowBoneManipulator::SetSelectedBoneByName(FName BoneName)
{
	if (RefSkeletonPoser)
	{
		const FReferenceSkeleton& RefSkeleton = RefSkeletonPoser->GetRefSkeleton();
		BoneIndex = RefSkeleton.FindBoneIndex(BoneName);
		SetSelectedBoneByIndex(BoneIndex);
	}
}

void UDataflowBoneManipulator::SetSelectedBoneByIndex(int32 InBoneIndex)
{
	if (RefSkeletonPoser)
	{
		const FReferenceSkeleton& RefSkeleton = RefSkeletonPoser->GetRefSkeleton();
		BoneIndex = RefSkeleton.IsValidIndex(InBoneIndex) ? InBoneIndex : INDEX_NONE;
		
		if (TransformGizmo && TransformProxy)
		{
			const bool bValidBone = (BoneIndex != INDEX_NONE);
			TransformGizmo->SetVisibility(bValidBone && bEnabled);
			if (bValidBone)
			{
				const FTransform& BoneTransform = RefSkeletonPoser->GetComponentSpaceTransform(BoneIndex);
				TransformGizmo->ReinitializeGizmoTransform(BoneTransform, /*bKeepGizmoUnscaled*/true);
			}
		}
	}
}

void UDataflowBoneManipulator::OnStartTransformEdit(UTransformProxy*)
{
	if (RefSkeletonPoser && BoneIndex != INDEX_NONE)
	{
		StartTransform = RefSkeletonPoser->GetComponentSpaceTransform(BoneIndex);
		RefSkeletonPoser->BeginPoseChange();
	}
}

void UDataflowBoneManipulator::OnTransformUpdated(UTransformProxy*, FTransform NewTransform)
{
	if (BoneIndex != INDEX_NONE)
	{
		if (RefSkeletonPoser && RefSkeletonPoser->IsRecordingPoseChange())
		{
			RefSkeletonPoser->ModifyBoneAdditiveTransform(BoneIndex,
				[this, &NewTransform](FTransform& InTransform)
				{
					InTransform = NewTransform.GetRelativeTransform(StartTransform);
				});

			if (OnReferenceSkeletonUpdated.IsBound())
			{
				OnReferenceSkeletonUpdated.Broadcast(*this);
			}
		}
	}
}

void UDataflowBoneManipulator::OnEndTransformEdit(UTransformProxy*)
{
	if (RefSkeletonPoser && RefSkeletonPoser->IsRecordingPoseChange())
	{
		RefSkeletonPoser->EndPoseChange();
	}
}