// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "DataflowEditorBoneManipulator.generated.h"

#define UE_API DATAFLOWEDITOR_API

struct FReferenceSkeleton;

class UCombinedTransformGizmo;
class UInteractiveToolManager;
class URefSkeletonPoser;
class UTransformProxy;

UCLASS(MinimalAPI)
class UDataflowBoneManipulator : public UObject
{
	GENERATED_BODY()

public:
	void Setup(UInteractiveToolManager* ToolManager, const FReferenceSkeleton& RefSkeleton);
	void Shutdown(UInteractiveToolManager* ToolManager);

	void SetSelectedBoneByName(FName BoneName);
	void SetSelectedBoneByIndex(int32 BoneIndex);

	URefSkeletonPoser* GetRefSkeletonPoser() const { return RefSkeletonPoser; }

	void SetEnabled(bool bEnabled);

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnReferenceSkeletonUpdated, UDataflowBoneManipulator&);
	FOnReferenceSkeletonUpdated OnReferenceSkeletonUpdated;

private:
	void OnStartTransformEdit(UTransformProxy*);
	void OnTransformUpdated(UTransformProxy*, FTransform NewTransform);
	void OnEndTransformEdit(UTransformProxy*);

	// Source mesh transform gizmo support
	UPROPERTY(Transient)
	TObjectPtr<UTransformProxy> TransformProxy;

	UPROPERTY(Transient)
	TObjectPtr<UCombinedTransformGizmo> TransformGizmo;

	UPROPERTY(Transient)
	TObjectPtr<URefSkeletonPoser> RefSkeletonPoser;

	UPROPERTY(Transient)
	int32 BoneIndex = INDEX_NONE;

	FTransform StartTransform;

	bool bEnabled = true;
};

#undef UE_API
