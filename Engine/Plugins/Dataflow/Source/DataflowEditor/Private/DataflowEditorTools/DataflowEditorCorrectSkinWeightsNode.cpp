// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataflowEditorTools/DataflowEditorCorrectSkinWeightsNode.h"

#include "Dataflow/DataflowCollectionEditSkinWeightsNode.h"
#include "Dataflow/DataflowInputOutput.h"
#include "Dataflow/DataflowConnectionTypes.h"
#include "Dataflow/DataflowNode.h"
#include "Dataflow/DataflowRenderingFactory.h"
#include "Dataflow/CollectionRenderingPatternUtility.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/DynamicVertexSkinWeightsAttribute.h"
#include "GeometryCollection/Facades/CollectionRenderingFacade.h"
#include "Operations/SmoothBoneWeights.h"
#include "SkeletalMeshAttributes.h"
#include "Chaos/CollectionPropertyFacade.h"
#include "Parameterization/MeshDijkstra.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowEditorCorrectSkinWeightsNode)

#define LOCTEXT_NAMESPACE "DataflowEditorCorrectSkinWeights"


namespace UE::Dataflow::Private
{
	namespace SkinWeightOperations
	{
		struct FWeightsAndIndicesRef
		{
			TArray<TArray<float>>& Weights;
			TArray<TArray<int32>>& Indices;
		};

		struct FOperationParams
		{
			const FWeightsAndIndicesRef Setup;
			TConstArrayView<float> SelectionMap;
			FWeightsAndIndicesRef Final;
			bool bUseSelectionAsPerVertexFactor = false;
		};
	};

static void BuildDynamicMeshes(const FDataflowNode* DataflowNode, UE::Dataflow::FContext& Context, TArray<Geometry::FDynamicMesh3>& DynamicMeshes)
{
	const TArray<UE::Dataflow::FRenderingParameter> RenderParameters = DataflowNode->GetRenderParameters();
	if(RenderParameters.Num() == 1)
	{
		TSharedPtr<FManagedArrayCollection> RenderCollection(new FManagedArrayCollection);
		GeometryCollection::Facades::FRenderingFacade RenderingFacade(*RenderCollection);
		RenderingFacade.DefineSchema();

		if(const IDataflowConstructionViewMode* ViewMode =
			UE::Dataflow::FRenderingViewModeFactory::GetInstance().GetViewMode(RenderParameters.Last().ViewMode))
		{
			FGraphRenderingState RenderingState(FGuid::NewGuid(), DataflowNode, RenderParameters.Last(), Context, *ViewMode, false);
			Dataflow::FRenderingFactory::GetInstance()->RenderNodeOutput(RenderingFacade, RenderingState);

			const int32 NumGeometry = RenderingFacade.NumGeometry();
			for (int32 MeshIndex = 0; MeshIndex < NumGeometry; ++MeshIndex)
			{
				UE::Geometry::FDynamicMesh3& DynamicMesh = DynamicMeshes.AddDefaulted_GetRef();
				UE::Dataflow::Conversion::RenderingFacadeToDynamicMesh(RenderingFacade, MeshIndex, DynamicMesh, false);

				DynamicMesh.Attributes()->AttachSkinWeightsAttribute(FName("Default"),
					new UE::Geometry::FDynamicMeshVertexSkinWeightsAttribute(&DynamicMesh));
			}
		}
	}
}

static void AccumulateSkinWeights(const float InterpolationWeight, 
		const TArray<int32>& BoneIndices, const TArray<float>& BoneWeights, TMap<int32,float>& SkinWeights, float& TotalWeight)
{
	for (int32 WeightIndex = 0, NumWeights = BoneIndices.Num(); WeightIndex < NumWeights; ++WeightIndex)
	{
		const float InterpolatedWeight = InterpolationWeight * BoneWeights[WeightIndex];

		if (float* WeightValue = SkinWeights.Find(BoneIndices[WeightIndex]))
		{
			*WeightValue += InterpolatedWeight;
		}
		else
		{
			SkinWeights.Add(BoneIndices[WeightIndex], InterpolatedWeight);
		}
		TotalWeight += InterpolatedWeight;
	}
}

static void ReportSkinWeights(TArray<int32>& BoneIndices, TArray<float>& BoneWeights, const TMap<int32, float>& SkinWeights, const float TotalWeight)
{
	if (TotalWeight > 0.0)
	{
		BoneIndices.Reset();
		BoneWeights.Reset();

		for (const TPair<int32, float>& BoneWeight : SkinWeights)
		{
			BoneIndices.Add(BoneWeight.Key);
			BoneWeights.Add(BoneWeight.Value / TotalWeight);
		}
	}
}

// Small structure to store the averaged bone weight over some vertices
struct FAveragedBoneWeight
{
	FAveragedBoneWeight(const float InWeightValue, const int32 InNumVertices) : 
		WeightValue(InWeightValue), NumVertices(InNumVertices) 
	{}

	float WeightValue;
	int32 NumVertices;
};

static bool AverageSkinWeights(const TSet<int32>& NeighborVertices, const int32 VertexOffset, const TArray<TArray<int32>>& FinalIndices, const TArray<TArray<float>>& FinalWeights, TMap<int32, FAveragedBoneWeight>& NeighborWeights)
{
	for (const int32 NeighborVertex : NeighborVertices)
	{
		const int32 GlobalNeighbor = VertexOffset + NeighborVertex;
		check(FinalIndices[GlobalNeighbor].Num() == FinalWeights[GlobalNeighbor].Num());
		for (int32 WeightIndex = 0, NumWeights = FinalIndices[GlobalNeighbor].Num(); WeightIndex < NumWeights; ++WeightIndex)
		{
			if (FinalWeights[GlobalNeighbor][WeightIndex] > 0.0f && FinalWeights[GlobalNeighbor][WeightIndex] <= 1.0f)
			{
				if (FAveragedBoneWeight* WeightValue = NeighborWeights.Find(FinalIndices[GlobalNeighbor][WeightIndex]))
				{
					WeightValue->WeightValue += FinalWeights[GlobalNeighbor][WeightIndex];
					WeightValue->NumVertices += 1;
				}
				else
				{
					NeighborWeights.Add(FinalIndices[GlobalNeighbor][WeightIndex], {FinalWeights[GlobalNeighbor][WeightIndex], 1});
				}
			}
		}
	}

	float TotalWeight = 0.0;
	for (TPair<int32, FAveragedBoneWeight>& NeighborWeight : NeighborWeights)
	{
		// Any averaged weight added has a minimum of 1 vertex 
		NeighborWeight.Value.WeightValue /= NeighborWeight.Value.NumVertices;
		TotalWeight += NeighborWeight.Value.WeightValue;
	}
	if(!FMath::IsNearlyZero(TotalWeight))
	{ 
		for (TPair<int32, FAveragedBoneWeight>& NeighborWeight : NeighborWeights)
		{
			NeighborWeight.Value.WeightValue /= TotalWeight;
		}
		return true;
	}
	return false;
}

static void RestrictSkinWeights(const int32 ClampingNumber, TArray<int32>& FinalIndices, TArray<float>& FinalWeights)
{
	if(FinalIndices.Num() > ClampingNumber)
	{ 
		TArray<TPair<int32, float>> SortedWeights;
		SortedWeights.Reserve(FinalWeights.Num());
		for (int32 WeightIndex = 0, NumWeights = FinalWeights.Num(); WeightIndex < NumWeights; ++WeightIndex)
		{
			SortedWeights.Add({ FinalIndices[WeightIndex], FinalWeights[WeightIndex] });
		}
		// sort in descending order by weight
		Algo::Sort(SortedWeights, [](const TPair<int32, float>& A, const TPair<int32, float>& B)
			{
				return A.Value > B.Value;
			});

		FinalIndices.Reset();
		FinalWeights.Reset();

		float TotalWeight = 0.0f;
		for (int32 WeightIndex = 0; WeightIndex < ClampingNumber; ++WeightIndex)
		{
			FinalIndices.Add(SortedWeights[WeightIndex].Key);
			FinalWeights.Add(SortedWeights[WeightIndex].Value);
			TotalWeight += SortedWeights[WeightIndex].Value;
		}
		if (TotalWeight != 0.0f)
		{
			for (int32 WeightIndex = 0; WeightIndex < ClampingNumber; ++WeightIndex)
			{
				FinalWeights[WeightIndex] /= TotalWeight;
			}
		}
	}
}
	
// Build all the collocated vertices
static void BuildCollocatedVertices(const Geometry::FDynamicMesh3& DynamicMesh, TArray<TArray<int32>>& CollocatedVertices)
{
	CollocatedVertices.Reset();
	
	const int32 NumVertices = DynamicMesh.VertexCount();
	
	TArray<TPair<int32, FVector3d>> BorderVertices;
	BorderVertices.Reserve(NumVertices);

	// Gather all the border vertices
	for (int32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
	{
		if (DynamicMesh.IsBoundaryVertex(VertexIndex))
		{
			BorderVertices.Add({VertexIndex, DynamicMesh.GetVertex(VertexIndex) });
		}
	}
	// Sort them based on the position 
	Algo::Sort(BorderVertices, [](const TPair<int32, FVector3d>& A, const TPair<int32, FVector3d>& B)
	{
		return (A.Value.X != B.Value.X) ? A.Value.X < B.Value.X : (A.Value.Y != B.Value.Y) ? A.Value.Y < B.Value.Y : A.Value.Z < B.Value.Z;
	});
	const int32 NumBorders = BorderVertices.Num();

	if(NumBorders > 0)
	{
		CollocatedVertices.Reserve(NumBorders);
		
		// Identify collocated vertices consecutively 
		CollocatedVertices.AddDefaulted_GetRef().Add(BorderVertices[0].Key);
		for (int32 BorderIndex = 1; BorderIndex < NumBorders; ++BorderIndex)
		{
			if((BorderVertices[BorderIndex].Value - BorderVertices[BorderIndex -1].Value).SquaredLength() < UE_SMALL_NUMBER)
			{
				CollocatedVertices.Last().Add(BorderVertices[BorderIndex].Key);
			}
			else
			{
				CollocatedVertices.AddDefaulted_GetRef().Add(BorderVertices[BorderIndex].Key);
			}
		}
	}
}

// Merge the skin weights of border vertices if collocated
static void MergeSkinWeights(const Geometry::FDynamicMesh3& DynamicMesh, const TArray<TArray<int32>>& CollocatedVertices, const int32 VertexOffset, const TArray<float>& SelectionMap, TArray<TArray<int32>>& FinalIndices, TArray<TArray<float>>& FinalWeights)
{
	const int32 NumVertices = DynamicMesh.VertexCount();
	if(SelectionMap.Num() <= NumVertices + VertexOffset)
	{ 
		// Loop over the selected collocated vertices (at least one of the border vertices is selected)
		for (int32 CollocatedIndex = 0; CollocatedIndex < CollocatedVertices.Num(); ++CollocatedIndex)
		{
			bool bIsSelected = false;
			for (const int32 VertexIndex : CollocatedVertices[CollocatedIndex])
			{
				if(SelectionMap[VertexIndex + VertexOffset] > 0.0f)
				{
					bIsSelected = true;
				}
			}
			if(bIsSelected)
			{ 
				TMap<int32, float> SkinWeights;
				float TotalWeight = 0.0;

				// Accumulate weights from all the border vertices 
				for (const int32 VertexIndex : CollocatedVertices[CollocatedIndex])
				{
					AccumulateSkinWeights(1.0f, FinalIndices[VertexIndex + VertexOffset], FinalWeights[VertexIndex + VertexOffset], SkinWeights, TotalWeight);
				}

				// Report the accumulated skin weight onto all the collocated vertices
				if (TotalWeight > 0.0)
				{
					for (const int32 VertexIndex : CollocatedVertices[CollocatedIndex])
					{
						ReportSkinWeights(FinalIndices[VertexIndex + VertexOffset], FinalWeights[VertexIndex + VertexOffset], SkinWeights, TotalWeight);
					}
				}
			}
		}
	}
}

static void PruneSkinWeights(const float PruningThreshold, const SkinWeightOperations::FOperationParams& Params)
{
	const int32 NumVertices = Params.Final.Indices.Num();
	for (int32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
	{
		if (Params.SelectionMap[VertexIndex] > 0)
		{
			float TotalWeight = 0.0f;
			for (int32 WeightIndex = 0, NumWeights = Params.Setup.Weights[VertexIndex].Num(); WeightIndex < NumWeights; ++WeightIndex)
			{
				if (Params.Setup.Weights[VertexIndex][WeightIndex] >= PruningThreshold)
				{
					Params.Final.Indices[VertexIndex].Add(Params.Setup.Indices[VertexIndex][WeightIndex]);
					Params.Final.Weights[VertexIndex].Add(Params.Setup.Weights[VertexIndex][WeightIndex]);
					TotalWeight += Params.Setup.Weights[VertexIndex][WeightIndex];
				}
			}
			if (TotalWeight != 0.0f)
			{
				for (int32 WeightIndex = 0, NumWeights = Params.Final.Weights[VertexIndex].Num(); WeightIndex < NumWeights; ++WeightIndex)
				{
					Params.Final.Weights[VertexIndex][WeightIndex] /= TotalWeight;
				}
			}
		}
		else
		{
			Params.Final.Indices[VertexIndex] = Params.Setup.Indices[VertexIndex];
			Params.Final.Weights[VertexIndex] = Params.Setup.Weights[VertexIndex];
		}
	}
}
	
static void NormalizeSkinWeights(const SkinWeightOperations::FOperationParams& Params)
{
	const int32 NumVertices = Params.Final.Indices.Num();
	for (int32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
	{
		if (Params.SelectionMap[VertexIndex] > 0)
		{
			float TotalWeight = 0.0f;
			for (int32 WeightIndex = 0, NumWeights = Params.Setup.Weights[VertexIndex].Num(); WeightIndex < NumWeights; ++WeightIndex)
			{
				Params.Final.Indices[VertexIndex].Add(Params.Setup.Indices[VertexIndex][WeightIndex]);
				Params.Final.Weights[VertexIndex].Add(Params.Setup.Weights[VertexIndex][WeightIndex]);
				TotalWeight += Params.Setup.Weights[VertexIndex][WeightIndex];
			}
			if(TotalWeight != 0.0f)
			{
				for(int32 WeightIndex = 0, NumWeights = Params.Final.Weights[VertexIndex].Num(); WeightIndex < NumWeights; ++WeightIndex)
				{
					Params.Final.Weights[VertexIndex][WeightIndex] /= TotalWeight;
				}
			}
		}
		else
		{
			Params.Final.Indices[VertexIndex] = Params.Setup.Indices[VertexIndex];
			Params.Final.Weights[VertexIndex] = Params.Setup.Weights[VertexIndex];
		}
	}
}

static void ClampSkinWeights(const int32 ClampingNumber, const SkinWeightOperations::FOperationParams& Params)
{
	const int32 NumVertices = Params.Final.Indices.Num();
	for (int32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
	{
		Params.Final.Indices[VertexIndex] = Params.Setup.Indices[VertexIndex];
		Params.Final.Weights[VertexIndex] = Params.Setup.Weights[VertexIndex];

		if((Params.SelectionMap[VertexIndex] > 0) && (Params.Setup.Weights[VertexIndex].Num() > ClampingNumber))
		{
			RestrictSkinWeights(ClampingNumber, Params.Final.Indices[VertexIndex], Params.Final.Weights[VertexIndex]);
		}
	}
}
	
static void SmoothVertexWeights(
	const UE::Geometry::FDynamicMesh3& DynamicMesh, 
	const TArray<int32>& CollocatedVertices, 
	const int32 VertexOffset, 
	const float SmoothingFactor, 
	const SkinWeightOperations::FWeightsAndIndicesRef& Final,
	TConstArrayView<float> PerVertexSmoothFactors
)
{
	// Get list of all neighboring vertices, AND this vertex
	TSet<int32> NeighborVertices;
	for(const int32 CollocatedVertex : CollocatedVertices)
	{
		NeighborVertices.Add(CollocatedVertex);
		for (const int32 NeighborVertex : DynamicMesh.VtxVerticesItr(CollocatedVertex))
		{
			NeighborVertices.Add(NeighborVertex);
		}
	}
	// Average the per bone weights values
	TMap<int32, FAveragedBoneWeight> NeighborWeights;
	if (AverageSkinWeights(NeighborVertices, VertexOffset, Final.Indices, Final.Weights, NeighborWeights))
	{
		for (const int32 CollocatedVertex : CollocatedVertices)
		{
			const int32 GlobalVertex = VertexOffset + CollocatedVertex;

			TArray<int32> BoneIndices;
			TArray<float> BoneWeights;

			float TotalWeight = 0.0;
			for (const TPair<int32, FAveragedBoneWeight>& NeighborWeight : NeighborWeights)
			{
				const int32 BoneIndex = NeighborWeight.Key;
				float SmoothWeight = NeighborWeight.Value.WeightValue;
				float StoredWeight = 0.0f;
				
				for (int32 WeightIndex = 0, NumWeights = Final.Indices[GlobalVertex].Num(); WeightIndex < NumWeights; ++WeightIndex)
				{
					if (BoneIndex == Final.Indices[GlobalVertex][WeightIndex])
					{
						StoredWeight = Final.Weights[GlobalVertex][WeightIndex];
						break;
					}
				}
				const float PerVertexSmoothFactor = PerVertexSmoothFactors.IsValidIndex(GlobalVertex)? PerVertexSmoothFactors[GlobalVertex]: 1.0f;
				BoneWeights.Add(FMath::Lerp<float>(StoredWeight, SmoothWeight, SmoothingFactor * PerVertexSmoothFactor));
				BoneIndices.Add(BoneIndex);

				TotalWeight += BoneWeights.Last();
			}

			if (!FMath::IsNearlyZero(TotalWeight))
			{
				for (float& VertexWeight : BoneWeights)
				{
					VertexWeight /= TotalWeight;
				}
				Final.Indices[GlobalVertex] = BoneIndices;
				Final.Weights[GlobalVertex] = BoneWeights;
			}
		}
	}
}

// This is an equivalent of the RelaxSkinWeights using the collocated vertices in order to smooth skin weights across seam	
static void SmoothSkinWeights(
	const UE::Geometry::FDynamicMesh3& DynamicMesh, 
	const TArray<TArray<int32>>& CollocatedVertices, 
	const int32 VertexOffset, 
	const float SmoothStrength, 
	const int32 NumIterations,
	const SkinWeightOperations::FOperationParams& Params
)
{
	const int32 NumVertices = DynamicMesh.VertexCount();

	TArray<TArray<int32>> SelectedVertices;
	SelectedVertices.Reserve(NumVertices);

	for(const TArray<int32>& CollocatedVertex : CollocatedVertices)
	{
		for (const int32 VertexIndex : CollocatedVertex)
		{
			if (Params.SelectionMap[VertexIndex + VertexOffset] > 0.0f)
			{
				// Add the collocated vertex if one of the underlying vertices is selected 
				SelectedVertices.Add(CollocatedVertex);
				break;
			}
		}
	}
	int32 NumCollocated = SelectedVertices.Num();
	for (int32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
	{
		const int32 GlobalVertex = VertexOffset+VertexIndex;
		if(Params.SelectionMap[GlobalVertex] > 0.0f && !DynamicMesh.IsBoundaryVertex(VertexIndex))
		{
			SelectedVertices.Add({VertexIndex});
		}
		// Initialize the final indices/weights with the setup ones
		Params.Final.Indices[GlobalVertex] = Params.Setup.Indices[GlobalVertex];
		Params.Final.Weights[GlobalVertex] = Params.Setup.Weights[GlobalVertex];
	}

	static constexpr float PercentPerIteration = 0.95f;
	const float SmoothingFactor = SmoothStrength * PercentPerIteration;
	
	const TArray<float> NoPerVertexSmoothingFactors;
	const TConstArrayView<float> PerVertexSmoothingFactors = Params.bUseSelectionAsPerVertexFactor ? Params.SelectionMap : NoPerVertexSmoothingFactors;
	for (int32 IterationIndex = 0; IterationIndex < NumIterations; ++IterationIndex)
	{
		for (const TArray<int32>& SelectedVertex : SelectedVertices)
		{
			SmoothVertexWeights(DynamicMesh, SelectedVertex, VertexOffset, SmoothingFactor, Params.Final, PerVertexSmoothingFactors);
		}
	}
	for (const TArray<int32>& SelectedVertex : SelectedVertices)
	{
		for (const int32 VertexIndex : SelectedVertex)
		{
			RestrictSkinWeights(MAX_TOTAL_INFLUENCES, Params.Final.Indices[VertexIndex], Params.Final.Weights[VertexIndex]);
		}
	}
}

static void RelaxSkinWeights(
	const UE::Geometry::FDynamicMesh3& DynamicMesh, 
	const int32 VertexOffset, 
	const float SmoothStrength, 
	const int32 NumIterations,
	const SkinWeightOperations::FOperationParams& Params
)
{
	if(UE::Geometry::FDynamicMeshVertexSkinWeightsAttribute* SkinWeights = DynamicMesh.Attributes()->GetSkinWeightsAttribute(FName("Default")))
	{
		SkinWeights->Initialize();
		const int32 NumVertices = DynamicMesh.VertexCount();
		TArray<int32> SelectedVertices;
		
		TArray<UE::AnimationCore::FBoneWeight> BoneWeightsBuffer;
		for (int32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
		{
			const int32 GlobalVertex = VertexOffset+VertexIndex;
			BoneWeightsBuffer.SetNumUninitialized(Params.Setup.Indices[GlobalVertex].Num());

			for(int32 BoneIndex = 0; BoneIndex < Params.Setup.Indices[GlobalVertex].Num(); ++BoneIndex)
			{
				BoneWeightsBuffer[BoneIndex].SetBoneIndex(Params.Setup.Indices[GlobalVertex][BoneIndex]);
				BoneWeightsBuffer[BoneIndex].SetWeight(Params.Setup.Weights[GlobalVertex][BoneIndex]);
			}
			SkinWeights->SetValue(VertexIndex, UE::AnimationCore::FBoneWeights::Create(BoneWeightsBuffer));
			
			if(Params.SelectionMap[GlobalVertex] > 0.0f)
			{
				SelectedVertices.Add(VertexIndex);
			}
			else
			{
				Params.Final.Indices[GlobalVertex] = Params.Setup.Indices[GlobalVertex];
				Params.Final.Weights[GlobalVertex] = Params.Setup.Weights[GlobalVertex];
			}
		}
		
		static constexpr float PercentPerIteration = 0.95f;
		UE::Geometry::FSmoothDynamicMeshVertexSkinWeights SmoothBoneWeights(&DynamicMesh, FSkeletalMeshAttributes::DefaultSkinWeightProfileName);
		if (SmoothBoneWeights.Validate() == UE::Geometry::EOperationValidationResult::Ok)
		{
			for (int32 IterationIndex = 0; IterationIndex < NumIterations; ++IterationIndex)
			{
				for (const int32 SelectedVertex : SelectedVertices)
				{
					SmoothBoneWeights.SmoothWeightsAtVertex(SelectedVertex, SmoothStrength * PercentPerIteration);
				}
			}
		}

		for (const int32 SelectedVertex : SelectedVertices)
		{
			const int32 GlobalVertex = VertexOffset+SelectedVertex;
			SkinWeights->GetValue(SelectedVertex, Params.Final.Indices[GlobalVertex], Params.Final.Weights[GlobalVertex]);
		}
	}
}

static void HammerSkinWeights(
	const UE::Geometry::FDynamicMesh3& DynamicMesh, 
	const int32 VertexOffset, 
	const float SelectionThreshold,
	const SkinWeightOperations::FOperationParams& Params
)
{
	const int32 NumVertices = DynamicMesh.VertexCount();
	
	TSet<int32> NeighborVertices;
	TArray<int32> SelectedVertices;
	
	for (int32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
	{
		const int32 GlobalVertex = VertexOffset+VertexIndex;
		if(Params.SelectionMap[GlobalVertex] > SelectionThreshold)
		{
			SelectedVertices.Add(VertexIndex);
			for (const int32 NeighborIndex : DynamicMesh.VtxVerticesItr(VertexIndex))
			{
				if (Params.SelectionMap[VertexOffset+NeighborIndex] <= SelectionThreshold)
				{
					NeighborVertices.Add(NeighborIndex);
				}
			}
		}
		else
		{
			Params.Final.Indices[GlobalVertex] = Params.Setup.Indices[GlobalVertex];
			Params.Final.Weights[GlobalVertex] = Params.Setup.Weights[GlobalVertex];
		}
	}
	UE::Geometry::TMeshDijkstra<UE::Geometry::FDynamicMesh3> PathFinder(&DynamicMesh);
	TArray<UE::Geometry::TMeshDijkstra<UE::Geometry::FDynamicMesh3>::FSeedPoint> SeedPoints;
	for (const int32 NeighborVertex : NeighborVertices)
	{
		SeedPoints.Add({ NeighborVertex, NeighborVertex, 0 });
	}
	PathFinder.ComputeToMaxDistance(SeedPoints, TNumericLimits<double>::Max());

	// for each selected vertex, find the nearest surrounding vertex and copy it's weights
	TArray<int32> VertexPath;
	for (const int32 SelectedVertex : SelectedVertices)
	{
		// find the closest surrounding vertex to this selected vertex
		const int32 ClosestVertex = PathFinder.FindPathToNearestSeed(SelectedVertex, VertexPath) ? VertexPath.Last() : SelectedVertex;
		const int32 GlobalVertex = VertexOffset+SelectedVertex;
		const int32 GlobalClosest = VertexOffset+ClosestVertex;

		TMap<int32, float> SkinWeights;
		float TotalWeight = 0.0;

		const float InterpolationWeight = FMath::Clamp((Params.SelectionMap[GlobalVertex]-SelectionThreshold) / (1.0-SelectionThreshold), 0.0f, 1.0f);

		AccumulateSkinWeights(InterpolationWeight, Params.Setup.Indices[GlobalClosest], Params.Setup.Weights[GlobalClosest], SkinWeights, TotalWeight);
		AccumulateSkinWeights(1.0f-InterpolationWeight, Params.Setup.Indices[GlobalVertex], Params.Setup.Weights[GlobalVertex], SkinWeights, TotalWeight);

		if(TotalWeight != 0.0f)
		{
			ReportSkinWeights(Params.Final.Indices[GlobalVertex], Params.Final.Weights[GlobalVertex], SkinWeights, TotalWeight);
		}
		else
		{
			Params.Final.Indices[GlobalVertex] = Params.Setup.Indices[GlobalVertex];
			Params.Final.Weights[GlobalVertex] = Params.Setup.Weights[GlobalVertex];
		}
	}
}
	
template<typename ArrayType>
static bool SetAttributeValues(FManagedArrayCollection& SelectedCollection,
	const FCollectionAttributeKey& AttributeKey, const TArray<ArrayType>& AttributeValues)
{
	if (!AttributeValues.IsEmpty() && !AttributeKey.Attribute.IsEmpty() && !AttributeKey.Group.IsEmpty())
	{
		const FName AttributeName(AttributeKey.Attribute);
		const FName AttributeGroup(AttributeKey.Group);
		
		if(TManagedArray<ArrayType>* AttributeArray = SelectedCollection.FindAttributeTyped<ArrayType>(AttributeName, AttributeGroup))
		{
			if(AttributeArray->Num() == AttributeValues.Num())
			{
				for(int32 VertexIndex = 0, NumVertices = AttributeArray->Num(); VertexIndex < NumVertices; ++VertexIndex)
				{
					AttributeArray->GetData()[VertexIndex] = AttributeValues[VertexIndex];
				}
			}
			return true;
		}
	}
	return false;
}
	
static bool GetSkinningSelection(FManagedArrayCollection& SelectedCollection,
	const FCollectionAttributeKey& AttributeKey, TArray<float>& AttributeValues)
{
	if (!AttributeKey.Attribute.IsEmpty() && !AttributeKey.Group.IsEmpty())
	{
		const FName AttributeName(AttributeKey.Attribute);
		const FName AttributeGroup(AttributeKey.Group);
		
		TManagedArray<float>& AttributeArray = SelectedCollection.AddAttribute<float>(AttributeName, AttributeGroup);
		AttributeValues = AttributeArray.GetConstArray();
		return true;
	}
	return false;
}

}

//
// FDataflowCorrectSkinWeightsNode
//

const FName FDataflowCorrectSkinWeightsNode::PruneSkinWeightsSelectionName = "PruneSkinWeightsSelection";
const FName FDataflowCorrectSkinWeightsNode::HammerSkinWeightsSelectionName = "HammerSkinWeightsSelection";
const FName FDataflowCorrectSkinWeightsNode::RelaxSkinWeightsSelectionName = "RelaxSkinWeightsSelection";
const FName FDataflowCorrectSkinWeightsNode::ClampSkinWeightsSelectionName = "ClampSkinWeightsSelection";
const FName FDataflowCorrectSkinWeightsNode::NormalizeSkinWeightsSelectionName = "NormalizeSkinWeightsSelection";

FDataflowCorrectSkinWeightsNode::FDataflowCorrectSkinWeightsNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&BoneIndicesKey);
	RegisterInputConnection(&BoneWeightsKey);
	RegisterInputConnection(&SelectionMapKey);

	RegisterInputConnection(&SmoothingIterations)
		.SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&SmoothingFactor)
		.SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&bUseSelectionAsPerVertexFactor)
		.SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&PruningThreshold)
		.SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&ClampingNumber)
		.SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&SelectionThreshold)
		.SetCanHidePin(true).SetPinIsHidden(true);

	RegisterOutputConnection(&Collection, &Collection);
	RegisterOutputConnection(&BoneIndicesKey, &BoneIndicesKey);
	RegisterOutputConnection(&BoneWeightsKey, &BoneWeightsKey);
}

TArray<UE::Dataflow::FRenderingParameter> FDataflowCorrectSkinWeightsNode::GetRenderParametersImpl() const
{
	return FDataflowAddScalarVertexPropertyCallbackRegistry::Get().GetRenderingParameters(VertexGroup.Name);
}

void FDataflowCorrectSkinWeightsNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace UE::Dataflow;
	
	// Get the pin value if plugged
	FCollectionAttributeKey BoneIndicesKeyValue = GetBoneIndicesKey(Context);
	FCollectionAttributeKey BoneWeightsKeyValue = GetBoneWeightsKey(Context);
	FCollectionAttributeKey SelectionMapKeyValue = GetSelectionMapKey(Context);

	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		// Evaluate in collection
		FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);

		// Set temporary collection output
		FManagedArrayCollection OutCollection = InCollection;
		SetValue(Context, MoveTemp(OutCollection), &Collection);

		if (!BoneIndicesKeyValue.Attribute.IsEmpty() && !BoneWeightsKeyValue.Attribute.IsEmpty() && !SelectionMapKeyValue.Attribute.IsEmpty())
		{
			TArray<TArray<float>> SetupWeights, FinalWeights;
			TArray<TArray<int32>> SetupIndices, FinalIndices;

			if(FDataflowCollectionEditSkinWeightsNode::FillAttributeWeights(InCollection, BoneIndicesKeyValue, BoneWeightsKeyValue, SetupIndices, SetupWeights))
			{
				TArray<float> SelectionMap;
				if(UE::Dataflow::Private::GetSkinningSelection(InCollection, SelectionMapKeyValue, SelectionMap))
				{
					FinalIndices.SetNum(SetupIndices.Num());
					FinalWeights.SetNum(SetupWeights.Num());
				
					const UE::Dataflow::Private::SkinWeightOperations::FOperationParams Params
					{
						.Setup = { SetupWeights, SetupIndices },
						.SelectionMap = SelectionMap,
						.Final = { FinalWeights, FinalIndices },
						.bUseSelectionAsPerVertexFactor = GetValue(Context, &bUseSelectionAsPerVertexFactor),
					};

					if(CorrectionType == ESkinWeightsCorrectionType::Prune)
					{
						const float InPruningThreshold = GetValue(Context, &PruningThreshold);
						UE::Dataflow::Private::PruneSkinWeights(InPruningThreshold, Params);
					}
					else if(CorrectionType == ESkinWeightsCorrectionType::Clamp)
					{
						const int32 InClampingNumber = GetValue(Context, &ClampingNumber);
						UE::Dataflow::Private::ClampSkinWeights(InClampingNumber, Params);
					}
					else if(CorrectionType == ESkinWeightsCorrectionType::Normalize)
					{
						UE::Dataflow::Private::NormalizeSkinWeights(Params);
					}
					else
					{
						// Do the work here
						TArray<UE::Geometry::FDynamicMesh3> DynamicMeshes;
						UE::Dataflow::Private::BuildDynamicMeshes(this, Context, DynamicMeshes);

						int32 VertexOffset = 0;
						TArray<TArray<int32>> CollocatedVertices;
						for(UE::Geometry::FDynamicMesh3& DynamicMesh : DynamicMeshes)
						{
							UE::Dataflow::Private::BuildCollocatedVertices(DynamicMesh, CollocatedVertices);
							
							if(CorrectionType == ESkinWeightsCorrectionType::Relax)
							{
								const float InSmoothingFactor = GetValue(Context, &SmoothingFactor);
								const int32 InSmoothingIterations = GetValue(Context, &SmoothingIterations);
								UE::Dataflow::Private::SmoothSkinWeights(DynamicMesh, CollocatedVertices, VertexOffset, InSmoothingFactor, InSmoothingIterations, Params);
							}
							else if (CorrectionType == ESkinWeightsCorrectionType::Hammer)
							{
								const float InSelectionThreshold = GetValue(Context, &SelectionThreshold);
								UE::Dataflow::Private::HammerSkinWeights(DynamicMesh, VertexOffset, InSelectionThreshold, Params);
							}

							UE::Dataflow::Private::MergeSkinWeights(DynamicMesh, CollocatedVertices, VertexOffset, SelectionMap, FinalIndices, FinalWeights);

							VertexOffset += DynamicMesh.VertexCount();
						}
					}

					FDataflowCollectionEditSkinWeightsNode::SetAttributeWeights(InCollection, BoneIndicesKeyValue, BoneWeightsKeyValue, FinalIndices, FinalWeights);
				}
			}
		}

		SetValue(Context, MoveTemp(InCollection), &Collection);
	}
	else if (Out->IsA<FCollectionAttributeKey>(&BoneIndicesKey))
	{
		SetValue(Context, MoveTemp(BoneIndicesKeyValue), &BoneIndicesKey);
	}
	else if (Out->IsA<FCollectionAttributeKey>(&BoneWeightsKey))
	{
		SetValue(Context, MoveTemp(BoneWeightsKeyValue), &BoneWeightsKey);
	}
}

FCollectionAttributeKey FDataflowCorrectSkinWeightsNode::GetBoneIndicesKey(UE::Dataflow::FContext& Context) const
{
	// Get the pin value if plugged
	FCollectionAttributeKey Key = GetValue(Context, &BoneIndicesKey, BoneIndicesKey);

	// If nothing set used the local value
	if(Key.Attribute.IsEmpty() && Key.Group.IsEmpty())
	{
		Key.Group = VertexGroup.Name.ToString();
		Key.Attribute = BoneIndicesName;
	}
	return Key;
}

FCollectionAttributeKey FDataflowCorrectSkinWeightsNode::GetBoneWeightsKey(UE::Dataflow::FContext& Context) const
{
	// Get the pin value if plugged
	FCollectionAttributeKey Key = GetValue(Context, &BoneWeightsKey, BoneWeightsKey);

	// If nothing set used the local value
	if(Key.Attribute.IsEmpty() && Key.Group.IsEmpty())
	{
		Key.Group = VertexGroup.Name.ToString();
		Key.Attribute = BoneWeightsName;
	}
	return Key;
}

FCollectionAttributeKey FDataflowCorrectSkinWeightsNode::GetSelectionMapKey(UE::Dataflow::FContext& Context) const
{
	// Get the pin value if plugged
	FCollectionAttributeKey Key = GetValue(Context, &SelectionMapKey, SelectionMapKey);

	// If nothing set used the local value
	if(Key.Attribute.IsEmpty() && Key.Group.IsEmpty())
	{
		Key.Group = VertexGroup.Name.ToString();
		Key.Attribute = SelectionMapName;
	}
	return Key;
}

FDataflowSetSkinningSelectionNode::FDataflowSetSkinningSelectionNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&SelectionMapKey);
	RegisterOutputConnection(&Collection, &Collection);
}

TArray<UE::Dataflow::FRenderingParameter> FDataflowSetSkinningSelectionNode::GetRenderParametersImpl() const
{
	return FDataflowAddScalarVertexPropertyCallbackRegistry::Get().GetRenderingParameters(VertexGroup.Name);
}

void FDataflowSetSkinningSelectionNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace UE::Dataflow;
	
	// Get the pin value if plugged
	FCollectionAttributeKey SelectionMapKeyValue = GetSelectionMapKey(Context);

	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		// Evaluate in collection
		FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		const TSharedRef<FManagedArrayCollection> OutCollection = MakeShared<FManagedArrayCollection>(MoveTemp(InCollection));
		
		Chaos::Softs::FCollectionPropertyMutableFacade PropertyFacade(OutCollection);
		PropertyFacade.DefineSchema();

		if(CorrectionType == ESkinWeightsCorrectionType::Prune)
		{
			PropertyFacade.AddStringValue(FName(SelectionMapKeyValue.Group + FString("::") + FDataflowCorrectSkinWeightsNode::PruneSkinWeightsSelectionName.ToString()), SelectionMapKeyValue.Attribute);
		}
		else if(CorrectionType == ESkinWeightsCorrectionType::Relax)
		{
			PropertyFacade.AddStringValue(FName(SelectionMapKeyValue.Group + FString("::") + FDataflowCorrectSkinWeightsNode::RelaxSkinWeightsSelectionName.ToString()), SelectionMapKeyValue.Attribute);
		}
		else if(CorrectionType == ESkinWeightsCorrectionType::Hammer)
		{
			PropertyFacade.AddStringValue(FName(SelectionMapKeyValue.Group + FString("::") + FDataflowCorrectSkinWeightsNode::HammerSkinWeightsSelectionName.ToString()), SelectionMapKeyValue.Attribute);
		}
		else if(CorrectionType == ESkinWeightsCorrectionType::Clamp)
		{
			PropertyFacade.AddStringValue(FName(SelectionMapKeyValue.Group + FString("::") + FDataflowCorrectSkinWeightsNode::ClampSkinWeightsSelectionName.ToString()), SelectionMapKeyValue.Attribute);
		}
		else if(CorrectionType == ESkinWeightsCorrectionType::Normalize)
		{
			PropertyFacade.AddStringValue(FName(SelectionMapKeyValue.Group + FString("::") + FDataflowCorrectSkinWeightsNode::NormalizeSkinWeightsSelectionName.ToString()), SelectionMapKeyValue.Attribute);
		}

		SetValue(Context, MoveTemp(*OutCollection), &Collection);
	}
}

FCollectionAttributeKey FDataflowSetSkinningSelectionNode::GetSelectionMapKey(UE::Dataflow::FContext& Context) const
{
	// Get the pin value if plugged
	FCollectionAttributeKey Key = GetValue(Context, &SelectionMapKey, SelectionMapKey);

	// If nothing set used the local value
	if(Key.Attribute.IsEmpty() && Key.Group.IsEmpty())
	{
		Key.Group = VertexGroup.Name.ToString();
		Key.Attribute = SelectionMapName;
	}
	return Key;
}


FDataflowGetSkinningSelectionNode::FDataflowGetSkinningSelectionNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterOutputConnection(&Collection, &Collection);
	RegisterOutputConnection(&SelectionMapKey);
}

TArray<UE::Dataflow::FRenderingParameter> FDataflowGetSkinningSelectionNode::GetRenderParametersImpl() const
{
	return FDataflowAddScalarVertexPropertyCallbackRegistry::Get().GetRenderingParameters(VertexGroup.Name);
}

void FDataflowGetSkinningSelectionNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace UE::Dataflow;
	
	if (Out->IsA<FManagedArrayCollection>(&Collection) || Out->IsA<FCollectionAttributeKey>(&SelectionMapKey))
	{
		// Evaluate in collection
		FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		const TSharedRef<FManagedArrayCollection> OutCollection = MakeShared<FManagedArrayCollection>(MoveTemp(InCollection));

		if(Out->IsA<FCollectionAttributeKey>(&SelectionMapKey))
		{
			// Get the pin value if plugged
			FCollectionAttributeKey SelectionMapKeyValue;
			SelectionMapKeyValue.Group = VertexGroup.Name.ToString();
			
			Chaos::Softs::FCollectionPropertyConstFacade PropertyFacade(OutCollection);  

			if(CorrectionType == ESkinWeightsCorrectionType::Prune)
			{
				SelectionMapKeyValue.Attribute = PropertyFacade.GetStringValue(FName(SelectionMapKeyValue.Group + FString("::") + FDataflowCorrectSkinWeightsNode::PruneSkinWeightsSelectionName.ToString()));
			}
			else if(CorrectionType == ESkinWeightsCorrectionType::Relax)
			{
				SelectionMapKeyValue.Attribute = PropertyFacade.GetStringValue(FName(SelectionMapKeyValue.Group + FString("::") + FDataflowCorrectSkinWeightsNode::RelaxSkinWeightsSelectionName.ToString()));
			}
			else if(CorrectionType == ESkinWeightsCorrectionType::Hammer)
			{
				SelectionMapKeyValue.Attribute = PropertyFacade.GetStringValue(FName(SelectionMapKeyValue.Group + FString("::") + FDataflowCorrectSkinWeightsNode::HammerSkinWeightsSelectionName.ToString()));
			}
			else if(CorrectionType == ESkinWeightsCorrectionType::Clamp)
			{
				SelectionMapKeyValue.Attribute = PropertyFacade.GetStringValue(FName(SelectionMapKeyValue.Group + FString("::") + FDataflowCorrectSkinWeightsNode::ClampSkinWeightsSelectionName.ToString()));
			}
			else if(CorrectionType == ESkinWeightsCorrectionType::Normalize)
			{
				SelectionMapKeyValue.Attribute = PropertyFacade.GetStringValue(FName(SelectionMapKeyValue.Group + FString("::") + FDataflowCorrectSkinWeightsNode::NormalizeSkinWeightsSelectionName.ToString()));
			}
			SetValue(Context, MoveTemp(SelectionMapKeyValue), &SelectionMapKey);
		}
		if(Out->IsA<FManagedArrayCollection>(&Collection))
		{
			SetValue(Context, MoveTemp(*OutCollection), &Collection);
		}
	}
}

#undef LOCTEXT_NAMESPACE
