// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 


#include "Dataflow/DataflowCollectionAddScalarVertexPropertyNode.h"
#include "Dataflow/DataflowNode.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "Dataflow/DataflowCollectionAttributeKeyNodes.h"

#include "DataflowEditorCorrectSkinWeightsNode.generated.h"

#define UE_API DATAFLOWEDITOR_API

/** Skin weights correction type */
UENUM()
enum class ESkinWeightsCorrectionType : uint8
{
	Relax,
	Prune,
	Hammer,
	Clamp,
	Normalize
};

/** Correct skin weights vertex properties. */
USTRUCT(Meta = (Experimental, DataflowCollection))
struct FDataflowCorrectSkinWeightsNode : public FDataflowNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowCorrectSkinWeightsNode, "CorrectSkinWeights", "Collection", "Correct skin weights and save it to collection")

public:

	static UE_API const FName PruneSkinWeightsSelectionName;
	static UE_API const FName HammerSkinWeightsSelectionName;
	static UE_API const FName RelaxSkinWeightsSelectionName;
	static UE_API const FName ClampSkinWeightsSelectionName;
	static UE_API const FName NormalizeSkinWeightsSelectionName;

	UE_API FDataflowCorrectSkinWeightsNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

	UPROPERTY(Meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection"))
	FManagedArrayCollection Collection;

	/** The name to be set for the bone indices. */
	UPROPERTY(EditAnywhere, Category = "Vertex Attributes", meta = (DisplayName = "Bone Indices"))
	FString BoneIndicesName;

	/** The name to be set for the bone weights. */
	UPROPERTY(EditAnywhere, Category = "Vertex Attributes", meta = (DisplayName = "Bone Weights"))
	FString BoneWeightsName;

	/** Map name to be used to select vertices to correct */
	UPROPERTY(EditAnywhere, Category = "Vertex Attributes", meta = (DisplayName = "Selection Map"))
	FString SelectionMapName;

	/** Target group in which the attributes are stored */
	UPROPERTY(EditAnywhere, Category = "Vertex Attributes", meta = (DisplayName = "Vertex Group"))
	FScalarVertexPropertyGroup VertexGroup;

	/** Bone indices key to be used in other nodes if necessary */
	UPROPERTY(meta = (DisplayName = "Bone Indices", DataflowInput, DataflowOutput, DataflowPassthrough = "BoneIndicesKey"))
	FCollectionAttributeKey BoneIndicesKey;

	/** Bone weights key to be used in other nodes if necessary */
	UPROPERTY(meta = (DisplayName = "Bone Weights", DataflowInput, DataflowOutput, DataflowPassthrough = "BoneWeightsKey"))
	FCollectionAttributeKey BoneWeightsKey;

	/** Selection map key to be used in other nodes if necessary */
	UPROPERTY(meta = (DisplayName = "Selection Map", DataflowInput))
	FCollectionAttributeKey SelectionMapKey;
	
	/** Selection map key to be used in other nodes if necessary */
	UPROPERTY(EditAnywhere, Category = "Skin Weights", meta = (DisplayName = "Correction Type"))
	ESkinWeightsCorrectionType CorrectionType = ESkinWeightsCorrectionType::Relax;

	/** Number of iteration required for the smoothing */
	UPROPERTY(EditAnywhere, Category = "Skin Weights", meta = (DisplayName = "Smoothing Iterations", DataflowInput, EditCondition = "CorrectionType == ESkinWeightsCorrectionType::Relax", EditConditionHides))
	int32 SmoothingIterations = 5;

	/** Lerp value in between the current and the average weight values */
	UPROPERTY(EditAnywhere, Category = "Skin Weights", meta = (DisplayName = "Smooting Factor", DataflowInput, EditCondition = "CorrectionType == ESkinWeightsCorrectionType::Relax", EditConditionHides))
	float SmoothingFactor = 0.5;

	/** When true, use the decimal values of the selection as a per vertex factor for the selected operation - Currently available for Relax operation only - false by default */
	UPROPERTY(EditAnywhere, Category = "Skin Weights", meta = (DataflowInput, EditCondition = "CorrectionType == ESkinWeightsCorrectionType::Relax", EditConditionHides))
	bool bUseSelectionAsPerVertexFactor = false;

	/** All weights below this threshold will be pruned */
	UPROPERTY(EditAnywhere, Category = "Skin Weights", meta = (DisplayName = "Pruning Threshold", DataflowInput, EditCondition = "CorrectionType == ESkinWeightsCorrectionType::Prune", EditConditionHides))
	float PruningThreshold = 0.01f;
	
	/** Max number of bones to consider for the skin weights */
    UPROPERTY(EditAnywhere, Category = "Skin Weights", meta = (DisplayName = "Clamping Number", DataflowInput, EditCondition = "CorrectionType == ESkinWeightsCorrectionType::Clamp", EditConditionHides))
    int32 ClampingNumber = 8;

	/** Selection threshold to consider a neighbor skin weight */
    UPROPERTY(EditAnywhere, Category = "Skin Weights", meta = (DisplayName = "Selection Threshold", DataflowInput, EditCondition = "CorrectionType == ESkinWeightsCorrectionType::Hammer", EditConditionHides, ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
    float SelectionThreshold = 0.5f;

	/** Get the weights attribute key to retrieve/set the bone indices values*/
	UE_API FCollectionAttributeKey GetBoneIndicesKey(UE::Dataflow::FContext& Context) const;

	/** Get the weights attribute key to retrieve/set the bone weights values*/
    UE_API FCollectionAttributeKey GetBoneWeightsKey(UE::Dataflow::FContext& Context) const;

	/** Get the selection map key to get the vertex selection on which the correction will be applied */
	UE_API FCollectionAttributeKey GetSelectionMapKey(UE::Dataflow::FContext& Context) const;
	
private:

	//~ Begin FDataflowPrimitiveNode interface
	UE_API virtual TArray<UE::Dataflow::FRenderingParameter> GetRenderParametersImpl() const override;
	UE_API virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowPrimitiveNode interface
};

/** Set skin weights selection attributes. */
USTRUCT(Meta = (Experimental, DataflowCollection))
struct FDataflowSetSkinningSelectionNode : public FDataflowNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowSetSkinningSelectionNode, "SetSkinningSelection", "Collection", "Set the skin weights selection for a future correction")

public:

	UE_API FDataflowSetSkinningSelectionNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

	UPROPERTY(Meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection"))
	FManagedArrayCollection Collection;

	/** Map name to be used to select vertices to correct */
	UPROPERTY(EditAnywhere, Category = "Vertex Attributes", meta = (DisplayName = "Selection Map"))
	FString SelectionMapName;

	/** Target group in which the attributes are stored */
	UPROPERTY(EditAnywhere, Category = "Vertex Attributes", meta = (DisplayName = "Vertex Group"))
	FScalarVertexPropertyGroup VertexGroup;

	/** Selection map key to be used in other nodes if necessary */
	UPROPERTY(meta = (DisplayName = "Selection Map", DataflowInput))
	FCollectionAttributeKey SelectionMapKey;
	
	/** Selection map key to be used in other nodes if necessary */
	UPROPERTY(EditAnywhere, Category = "Skin Weights", meta = (DisplayName = "Correction Type"))
	ESkinWeightsCorrectionType CorrectionType = ESkinWeightsCorrectionType::Relax;

	/** Get the selection map key to get the vertex selection on which the correction will be applied */
	UE_API FCollectionAttributeKey GetSelectionMapKey(UE::Dataflow::FContext& Context) const;
	
private:

	//~ Begin FDataflowPrimitiveNode interface
	UE_API virtual TArray<UE::Dataflow::FRenderingParameter> GetRenderParametersImpl() const override;
	UE_API virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowPrimitiveNode interface
};



/** Get skin weights selection attributes. */
USTRUCT(Meta = (Experimental, DataflowCollection))
struct FDataflowGetSkinningSelectionNode : public FDataflowNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowGetSkinningSelectionNode, "GetSkinningSelection", "Collection", "Get the skin weights selection for a future correction")

public:

	UE_API FDataflowGetSkinningSelectionNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

	UPROPERTY(Meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection"))
	FManagedArrayCollection Collection;

	/** Target group in which the attributes are stored */
	UPROPERTY(EditAnywhere, Category = "Vertex Attributes", meta = (DisplayName = "Vertex Group"))
	FScalarVertexPropertyGroup VertexGroup;

	/** Selection map key to be used in other nodes if necessary */
	UPROPERTY(meta = (DisplayName = "Selection Map", DataflowOutput))
	FCollectionAttributeKey SelectionMapKey;
	
	/** Selection map key to be used in other nodes if necessary */
	UPROPERTY(EditAnywhere, Category = "Skin Weights", meta = (DisplayName = "Correction Type"))
	ESkinWeightsCorrectionType CorrectionType = ESkinWeightsCorrectionType::Relax;
	
private:

	//~ Begin FDataflowPrimitiveNode interface
	UE_API virtual TArray<UE::Dataflow::FRenderingParameter> GetRenderParametersImpl() const override;
	UE_API virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowPrimitiveNode interface
};


#undef UE_API
