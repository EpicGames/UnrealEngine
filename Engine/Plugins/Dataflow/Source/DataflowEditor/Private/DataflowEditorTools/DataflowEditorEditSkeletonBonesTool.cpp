// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataflowEditorTools/DataflowEditorEditSkeletonBonesTool.h"

#include "ContextObjectStore.h"
#include "EditorModeManager.h"
#include "InteractiveGizmoManager.h"
#include "Dataflow/DataflowCollectionEditSkeletonBonesNode.h"
#include "Dataflow/DataflowContent.h"
#include "Dataflow/DataflowObjectInterface.h"
#include "Dataflow/DataflowSNode.h"
#include "Engine/World.h"
#include "InteractiveToolManager.h"
#include "ToolTargetManager.h"
#include "ModelingToolTargetUtil.h"
#include "Components/SkeletalMeshComponent.h"
#include "Dataflow/DataflowConstructionViewportClient.h"
#include "DynamicMesh/DynamicVertexSkinWeightsAttribute.h"
#include "EditorGizmos/EditorTransformGizmoSource.h"
#include "EditorGizmos/EditorTransformGizmoUtil.h"
#include "TargetInterfaces/MeshDescriptionCommitter.h"
#include "TargetInterfaces/MeshDescriptionProvider.h"
#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowEditorEditSkeletonBonesTool)

struct FDataflowCollectionEditSkeletonBonesNode;
using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UDataflowEditorEditSkeletonBonesTool"

void UDataflowEditorEditSkeletonBonesToolBuilder::GetSupportedConstructionViewModes(const UDataflowContextObject& ContextObject, TArray<const UE::Dataflow::IDataflowConstructionViewMode*>& Modes) const
{}

bool UDataflowEditorEditSkeletonBonesToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	auto HasManagedArrayCollection = [](const FDataflowNode* InDataflowNode, const TSharedPtr<UE::Dataflow::FEngineContext> Context)
	{
		if (InDataflowNode && Context)
		{
			for (const FDataflowOutput* const Output : InDataflowNode->GetOutputs())
			{
				if (Output->GetType() == FName("FManagedArrayCollection"))
				{
					return true;
				}
			}
		}

		return false;
	};

	if (USkeletonEditingToolBuilder::CanBuildTool(SceneState))
	{
		if (SceneState.SelectedComponents.Num() == 1 && SceneState.SelectedComponents[0]->IsA<USkeletalMeshComponent>())
		{
			if (UDataflowBaseContent* ContextObject = SceneState.ToolManager->GetContextObjectStore()->FindContext<UDataflowBaseContent>())
			{
				if (const TSharedPtr<UE::Dataflow::FEngineContext> EvaluationContext = ContextObject->GetDataflowContext())
				{
					if (const FDataflowNode* PrimarySelection = ContextObject->GetSelectedNodeOfType<FDataflowCollectionEditSkeletonBonesNode>())
					{
						return HasManagedArrayCollection(PrimarySelection, EvaluationContext);
					}
				}
			}
		}
	}
	return false;
}

const FToolTargetTypeRequirements& UDataflowEditorEditSkeletonBonesToolBuilder::GetTargetRequirements() const
{
	return USkeletonEditingToolBuilder::GetTargetRequirements();
}

UInteractiveTool* UDataflowEditorEditSkeletonBonesToolBuilder::BuildTool(const FToolBuilderState& SceneState) const
{
	UDataflowEditorEditSkeletonBonesTool* EditTool = NewObject<UDataflowEditorEditSkeletonBonesTool>(SceneState.ToolManager);

	if (UDataflowBaseContent* ContextObject = SceneState.ToolManager->GetContextObjectStore()->FindContext<UDataflowBaseContent>())
	{
		if (FDataflowNode* PrimarySelection = ContextObject->GetSelectedNodeOfType<FDataflowCollectionEditSkeletonBonesNode>())
		{
			FDataflowCollectionEditSkeletonBonesNode* EditSkeletonBonesNode =
				StaticCast<FDataflowCollectionEditSkeletonBonesNode*>(PrimarySelection);
			TSharedPtr<FDataflowSkeletalMeshEditorBinding> SkeletalMeshBinding = MakeShared<FDataflowSkeletalMeshEditorBinding>();
			
			EditTool->EditSkeletonBonesNode = EditSkeletonBonesNode;
			
			EditSkeletonBonesNode->OnBoneSelectionChanged.AddLambda(
				[EditTool, SkeletalMeshBinding](const TArray<FName>& BoneNames)
			{
				EditTool->GetNotifier()->HandleNotification(BoneNames, ESkeletalMeshNotifyType::BonesSelected);
				SkeletalMeshBinding->BoneSelection = BoneNames;
			});
			EditTool->SetDataflowEditorContextObject(ContextObject);
		}
	}
	
	UToolTarget* Target = SceneState.TargetManager->BuildFirstSelectedTargetable(SceneState, GetTargetRequirements());
	EditTool->SetTarget(Target);
	EditTool->Init(SceneState);
	
	return EditTool;
}

void UDataflowEditorEditSkeletonBonesTool::Setup()
{
	USkeletonEditingTool::Setup();
}

void UDataflowEditorEditSkeletonBonesTool::Shutdown(EToolShutdownType ShutdownType)
{
	USkeletonEditingTool::Shutdown(ShutdownType);

	if(Target && ShutdownType == EToolShutdownType::Accept)
	{
		EditSkeletonBonesNode->Invalidate();

		// Avoid rebuilding the skeletal mesh after updating the skelton bones
		EditSkeletonBonesNode->ValidateSkeletalMeshes();
	}
}

TScriptInterface<ITransformGizmoSource> UDataflowEditorEditSkeletonBonesTool::BuildTransformSource()
{
	const UEditorTransformGizmoContextObject* ToolGizmoContext = nullptr;
	if (GetToolManager() && GetToolManager()->GetPairedGizmoManager())
	{
		if (IToolsContextQueriesAPI* QueriesAPI = GetToolManager()->GetPairedGizmoManager()->GetContextQueriesAPI())
		{
			if (FViewport* Viewport = QueriesAPI->GetFocusedViewport())
			{
				if (FViewportClient* ViewportClient = Viewport->GetClient())
				{
					if (FEditorModeTools*EditorModeTools = static_cast<FEditorViewportClient*>(ViewportClient)->GetModeTools())
					{
						ToolGizmoContext = EditorModeTools->GetGizmoContext();
					}
				}
			}
		}
	}
	
	return UDataflowTransformGizmoSource::CreateNew(this, ToolGizmoContext);
}

FDataflowSkeletalMeshEditorBinding::FDataflowSkeletalMeshEditorBinding()
	: ISkeletalMeshEditorBinding()
{}

TSharedPtr<ISkeletalMeshNotifier> FDataflowSkeletalMeshEditorBinding::GetNotifier()
{
	if (!DataflowNotifier)
	{
		DataflowNotifier = MakeShared<FDataflowSkeletalMeshEditorNotifier>();
	}
	return DataflowNotifier;	
}

ISkeletalMeshEditorBinding::NameFunction FDataflowSkeletalMeshEditorBinding::GetNameFunction()
{
	return [this](HHitProxy* InHitProxy) -> TOptional<FName>
	{
		if (const HDataflowElementHitProxy* ElementHitProxy = HitProxyCast<HDataflowElementHitProxy>(InHitProxy))
		{
			return ElementHitProxy->ElementName;
		}

		static const TOptional<FName> Dummy;
		return Dummy;
	};
}

TArray<FName> FDataflowSkeletalMeshEditorBinding::GetSelectedBones() const
{
	return BoneSelection;
}

FDataflowSkeletalMeshEditorNotifier::FDataflowSkeletalMeshEditorNotifier()
	: ISkeletalMeshNotifier()
{}

void FDataflowSkeletalMeshEditorNotifier::HandleNotification(const TArray<FName>& BoneNames, const ESkeletalMeshNotifyType InNotifyType)
{}

EGizmoTransformMode UDataflowTransformGizmoSource::GetGizmoMode() const
{
	if (const FEditorViewportClient* ViewportClient = GetViewportClient())
	{
		const FDataflowConstructionViewportClient* DataflowClient = static_cast<const FDataflowConstructionViewportClient*>(ViewportClient);

		switch (DataflowClient->GetToolWidgetMode())
		{
			case UE::Widget::EWidgetMode::WM_Translate: return EGizmoTransformMode::Translate;
			case UE::Widget::EWidgetMode::WM_Rotate: return EGizmoTransformMode::Rotate;
			case UE::Widget::EWidgetMode::WM_Scale: return EGizmoTransformMode::Scale;
			default: return EGizmoTransformMode::None;
		}
	}
	return GizmoMode;
}
bool UDataflowTransformGizmoSource::GetVisible(const EViewportContext InViewportContext) const
{
	return CanInteract(InViewportContext);
}

bool UDataflowTransformGizmoSource::CanInteract(const EViewportContext InViewportContext) const
{
	return true;
}

const FRotationContext& UDataflowTransformGizmoSource::GetRotationContext() const
{
	static const FRotationContext DefaultContext;
	return WeakContext.IsValid() ? WeakContext->RotationContext : DefaultContext;
}

UDataflowTransformGizmoSource* UDataflowTransformGizmoSource::CreateNew(UObject* Outer, 
		const UEditorTransformGizmoContextObject* ContextObject)
{
	UDataflowTransformGizmoSource* NewSource = NewObject<UDataflowTransformGizmoSource>(Outer);
	NewSource->WeakContext = ContextObject;
	return NewSource;
}

#undef LOCTEXT_NAMESPACE
