// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "DataflowEditorTools/DataflowEditorToolBuilder.h"
#include "EditorGizmos/EditorTransformGizmoSource.h"
#include "EditorGizmos/TransformGizmoInterfaces.h"
#include "SkeletalMesh/SkeletonEditingTool.h"

#include "DataflowEditorEditSkeletonBonesTool.generated.h"

#define UE_API DATAFLOWEDITOR_API

class UDataflowContextObject;
class UDataflowEditorMode;

/**
 * Dataflow edit skeleton tool builder
 */
UCLASS(MinimalAPI)
class UDataflowEditorEditSkeletonBonesToolBuilder : public USkeletonEditingToolBuilder, public IDataflowEditorToolBuilder
{
	GENERATED_BODY()

private:
	//~ Begin IDataflowEditorToolBuilder interface
	UE_API virtual void GetSupportedConstructionViewModes(const UDataflowContextObject& ContextObject, TArray<const UE::Dataflow::IDataflowConstructionViewMode*>& Modes) const override;
	virtual bool CanSetConstructionViewWireframeActive() const override { return false; }
	//~ End IDataflowEditorToolBuilder interface

	//~ Begin USkeletonEditToolBuilder interface
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual UInteractiveTool* BuildTool(const FToolBuilderState& SceneState) const override;
	//~ End USkeletonEditToolBuilder interface

protected:
	UE_API virtual const FToolTargetTypeRequirements& GetTargetRequirements() const override;
};

/**
 * Dataflow edit skeleton tool
 */
UCLASS(MinimalAPI)
class UDataflowEditorEditSkeletonBonesTool : public USkeletonEditingTool
{
	GENERATED_BODY()
	
	//~ Begin USkeletonEditingTool interface
	UE_API virtual void Setup() override;
	UE_API virtual void Shutdown(EToolShutdownType ShutdownType) override;
	UE_API virtual TScriptInterface<ITransformGizmoSource> BuildTransformSource() override;
	//~ End USkeletonEditingTool interface

	friend class UDataflowEditorEditSkeletonBonesToolBuilder;

private:

	/** Set the dataflow context object */
	void SetDataflowEditorContextObject(TObjectPtr<UDataflowContextObject> InDataflowEditorContextObject)
	{
		DataflowEditorContextObject = InDataflowEditorContextObject;
	}
	
	/** Edit Skeleton node associated with that tool */
	struct FDataflowCollectionEditSkeletonBonesNode* EditSkeletonBonesNode = nullptr;

	/** Dataflow context object to be used to retrieve the node context data*/
	TObjectPtr<UDataflowContextObject> DataflowEditorContextObject;
};

/** Notifier when skeleton selection is changing */
class FDataflowSkeletalMeshEditorNotifier : public ISkeletalMeshNotifier
{
public:
	FDataflowSkeletalMeshEditorNotifier();

	//~ Begin ISkeletalMeshEditorBinding interface
	virtual void HandleNotification(const TArray<FName>& BoneNames, const ESkeletalMeshNotifyType InNotifyType) override;
	//~ End ISkeletalMeshEditorBinding interface

private:
};

/** Binding used to retrieve the bone names (morph targets not used)*/
class FDataflowSkeletalMeshEditorBinding : public ISkeletalMeshEditorBinding
{
public:
	FDataflowSkeletalMeshEditorBinding();

	//~ Begin ISkeletalMeshEditorBinding interface
	virtual TSharedPtr<ISkeletalMeshNotifier> GetNotifier() override;
	virtual NameFunction GetNameFunction() override;
	virtual TArray<FName> GetSelectedBones() const override;
	//~ End ISkeletalMeshEditorBinding interface

	TArray<FName> BoneSelection;

private :

	/** Notifier used when the tool selection is changing */
	TSharedPtr<FDataflowSkeletalMeshEditorNotifier> DataflowNotifier;
};

/**
 * UDataflowTransformGizmoSource is an ITransformGizmoSource implementation that provides
 * current state information used to configure the Editor transform gizmo.
 */
UCLASS(MinimalAPI)
class UDataflowTransformGizmoSource : public UEditorTransformGizmoSource
{
	GENERATED_BODY()
public:

	//~ Begin ITransformGizmoSource interface
	UE_API virtual EGizmoTransformMode GetGizmoMode() const override;
	virtual EToolContextCoordinateSystem GetGizmoCoordSystemSpace() const override {return EToolContextCoordinateSystem::Local;}
	virtual float GetGizmoScale() const override {return 1.0f;}
	UE_API virtual bool GetVisible(const EViewportContext InViewportContext = EViewportContext::Focused) const override;
	UE_API virtual bool CanInteract(const EViewportContext InViewportContext = EViewportContext::Focused) const override;
	virtual EGizmoTransformScaleType GetScaleType() const override {return EGizmoTransformScaleType::Default;}
	UE_API virtual const FRotationContext& GetRotationContext() const override;
	//~ End ITransformGizmoSource interface

	/** Transform source creation */
	static UE_API UDataflowTransformGizmoSource* CreateNew(
		UObject* Outer = (UObject*)GetTransientPackage(),
		const class UEditorTransformGizmoContextObject* ContextObject = nullptr);

private:

	/** Default transform mode to be used when moving bones */
	EGizmoTransformMode GizmoMode = EGizmoTransformMode::Translate;
};


#undef UE_API
