// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataflowEditorTools/DataflowEditorSkinWeightsPaintTool.h"

#include "ContextObjectStore.h"
#include "Dataflow/DataflowCollectionEditSkinWeightsNode.h"
#include "Dataflow/DataflowContent.h"
#include "Dataflow/DataflowObjectInterface.h"
#include "Dataflow/DataflowSNode.h"
#include "Dataflow/DataflowEditorUtil.h"
#include "DynamicMesh/MeshNormals.h"
#include "Engine/World.h"
#include "InteractiveToolManager.h"
#include "ModelingToolTargetUtil.h"
#include "Components/SkeletalMeshComponent.h"
#include "DynamicMesh/DynamicVertexSkinWeightsAttribute.h"
#include "DynamicMesh/NonManifoldMappingSupport.h"
#include "Misc/ScopedSlowTask.h"
#include "SkeletalMeshOperations.h"
#include "SkeletalMesh/RefSkeletonPoser.h"
#include "SkeletalMesh/SkeletalMeshToolsHelper.h"
#include "TargetInterfaces/SkeletonProvider.h"
#include "TargetInterfaces/MeshDescriptionCommitter.h"
#include "TargetInterfaces/MeshDescriptionProvider.h"
#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowEditorSkinWeightsPaintTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UDataflowEditorSkinWeightsPaintTool"

//////////////////////////////////////////////////////////////////////////////////////////////////////////////

void UDataflowEditorSkinWeightsPaintToolBuilder::GetSupportedConstructionViewModes(const UDataflowContextObject& ContextObject, TArray<const UE::Dataflow::IDataflowConstructionViewMode*>& Modes) const
{}

bool UDataflowEditorSkinWeightsPaintToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	auto HasManagedArrayCollection = [](const FDataflowNode* InDataflowNode, const TSharedPtr<UE::Dataflow::FEngineContext> Context)
	{
		if (InDataflowNode && Context)
		{
			for (const FDataflowOutput* const Output : InDataflowNode->GetOutputs())
			{
				if (Output->GetType() == FName("FManagedArrayCollection"))
				{
					return true;
				}
			}
		}

		return false;
	};

	auto HasSkinWeightsAndBonesData = [](const UActorComponent* Component) -> bool
		{
			if (const USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(Component))
			{
				if (const USkinnedAsset* SkinnedAsset = SkeletalMeshComponent->GetSkinnedAsset())
				{
					return (SkinnedAsset->GetRefSkeleton().GetNum() > 0);
				}
			}
			else if (const UDynamicMeshComponent* DynamicMeshComponent = Cast<UDynamicMeshComponent>(Component))
			{
				if (const FDynamicMesh3* DynamicMesh = DynamicMeshComponent->GetMesh())
				{
					if (const FDynamicMeshAttributeSet* Attributes = DynamicMesh->Attributes())
					{
						const FName DefaultProfileName("Default");
						const bool bHasSkinWeights = Attributes->HasSkinWeightsAttribute(DefaultProfileName);
						const int32 NumBones = Attributes->GetNumBones();
						return bHasSkinWeights && (NumBones > 0);
					}
				}
			}
			return false;
		};

	if (USkinWeightsPaintToolBuilder::CanBuildTool(SceneState))
	{
		if (SceneState.SelectedComponents.Num() == 1)
		{
			const bool bIsASkeletalMesh = SceneState.SelectedComponents[0]->IsA<USkeletalMeshComponent>();
			const bool bIsADynamicMesh  = SceneState.SelectedComponents[0]->IsA<UDynamicMeshComponent>();
			if (bIsASkeletalMesh || bIsADynamicMesh)
			{
				if (UDataflowBaseContent* ContextObject = SceneState.ToolManager->GetContextObjectStore()->FindContext<UDataflowBaseContent>())
				{
					if (const TSharedPtr<UE::Dataflow::FEngineContext> EvaluationContext = ContextObject->GetDataflowContext())
					{
						if (const FDataflowNode* SelectedNode = ContextObject->GetSelectedNodeOfType<FDataflowCollectionEditSkinWeightsNode>())
						{
							const bool bIsCollectionOutput = HasManagedArrayCollection(SelectedNode, EvaluationContext);
							const bool bHasSkinWeightsAndBoneData = HasSkinWeightsAndBonesData(SceneState.SelectedComponents[0]);

							if (!bHasSkinWeightsAndBoneData)
							{
								const FString Message = FString::Printf(TEXT("Cannot open the Edit Skin Weight Tool, the mesh is missing skin weights / bones data"));
								EvaluationContext->Error(Message, SelectedNode);
							}

							return bIsCollectionOutput && bHasSkinWeightsAndBoneData;
						}
					}
				}
			}
		}
	}
	return false;
}

const FToolTargetTypeRequirements& UDataflowEditorSkinWeightsPaintToolBuilder::GetTargetRequirements() const
{
	static FToolTargetTypeRequirements TypeRequirements({
		UMeshDescriptionProvider::StaticClass(),
		UMeshDescriptionCommitter::StaticClass(),
		});
	return TypeRequirements;
}

UMeshSurfacePointTool* UDataflowEditorSkinWeightsPaintToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	UDataflowEditorSkinWeightsPaintTool* PaintTool = NewObject<UDataflowEditorSkinWeightsPaintTool>(SceneState.ToolManager);
	PaintTool->SetTargetManager(SceneState.TargetManager);
	
	if (UDataflowBaseContent* ContextObject = SceneState.ToolManager->GetContextObjectStore()->FindContext<UDataflowBaseContent>())
	{
		if (FDataflowNode* PrimarySelection = ContextObject->GetSelectedNodeOfType<FDataflowCollectionEditSkinWeightsNode>())
		{
			FDataflowCollectionEditSkinWeightsNode* SkinWeightNode = PrimarySelection->AsType<FDataflowCollectionEditSkinWeightsNode>();
			PaintTool->SetSkinWeightNode(SkinWeightNode);
			PaintTool->SetDataflowEditorContextObject(ContextObject);
		}
	}
	return PaintTool;
}

//////////////////////////////////////////////////////////////////////////////////////////////////////////////

void UDataflowEditorSkinWeightsPaintTool::SetSkinWeightNode(FDataflowCollectionEditSkinWeightsNode* Node)
{
	if (SkinWeightNode)
	{
		SkinWeightNode->OnBoneSelectionChanged.Remove(OnBoneSelectionChangedDelegateHandle);
	}

	SkinWeightNode = Node;

	if (SkinWeightNode)
	{
		OnBoneSelectionChangedDelegateHandle = SkinWeightNode->OnBoneSelectionChanged.AddUObject(this, &UDataflowEditorSkinWeightsPaintTool::HandleOnBoneSelectionChanged);
	}
}

const TArray<FTransform>& UDataflowEditorSkinWeightsPaintTool::GetComponentSpaceBoneTransforms()
{
	if (BoneManipulator)
	{
		if (URefSkeletonPoser* Poser = BoneManipulator->GetRefSkeletonPoser())
		{
			return Poser->GetComponentSpaceTransforms();
		}
	}
	return Super::GetComponentSpaceBoneTransforms();
}

void UDataflowEditorSkinWeightsPaintTool::ToggleBoneManipulation(bool bEnable)
{
	if (BoneManipulator)
	{
		BoneManipulator->SetEnabled(bEnable);
	}
	Super::ToggleBoneManipulation(bEnable);
}

void UDataflowEditorSkinWeightsPaintTool::HandleOnBoneSelectionChanged(const TArray<FName>& BoneNames)
{
	GetNotifier()->HandleNotification(BoneNames, ESkeletalMeshNotifyType::BonesSelected);
	// this may not be called on the game thread so we need to make sure we post an update on the right thread
	if (BoneManipulator)
	{
		TWeakObjectPtr<UDataflowBoneManipulator> WeakManipulator = BoneManipulator;
		const FName FirstSelectedName = BoneNames.IsEmpty() ? NAME_None : BoneNames[0];
	
		Async(EAsyncExecution::TaskGraphMainTick, 
		[WeakManipulator, FirstSelectedName]
			{
				if (TStrongObjectPtr<UDataflowBoneManipulator> Manipulator = WeakManipulator.Pin())
				{
					Manipulator->SetSelectedBoneByName(FirstSelectedName);
				}
			});
	}
}

void UDataflowEditorSkinWeightsPaintTool::SetDataflowEditorContextObject(TObjectPtr<UDataflowContextObject> InDataflowEditorContextObject)
{
	DataflowEditorContextObject = InDataflowEditorContextObject;
}

int32 UDataflowEditorSkinWeightsPaintTool::GetVertexOffset() const
{
	if(const IPrimitiveComponentBackedTarget* TargetComponent = Cast<IPrimitiveComponentBackedTarget>(Target))
	{
		if(USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(TargetComponent->GetOwnerComponent()))
		{
			if(USkeletalMesh* SkeletalMesh = SkeletalMeshComponent->GetSkeletalMeshAsset())
			{
				return SkinWeightNode->GetSkeletalMeshOffset(SkeletalMesh);
			}
		}
		else if (UDynamicMeshComponent* DynamicMeshComponent = Cast<UDynamicMeshComponent>(TargetComponent->GetOwnerComponent()))
		{
			if (UDynamicMesh* DynamicMesh = DynamicMeshComponent->GetDynamicMesh())
			{
				// for dynamic meshes vertex offset is always 0, because the NonManifoldMappingSupport will return the right offset from the original mesh  
				return 0;
			}
		}
	}
	return INDEX_NONE;
}


bool UDataflowEditorSkinWeightsPaintTool::ExtractSkinWeights(TArray<TArray<int32>>& OutCurrentIndices, TArray<TArray<float>>& OutCurrentWeights, TArray<TArray<int32>>& OutSetupIndices, TArray<TArray<float>>& OutSetupWeights)
{
	// Setup DynamicMeshToWeight conversion and get Input weight map (if it exists)
	if (DataflowEditorContextObject && SkinWeightNode)
	{
		// Find the map if it exists.
		if (TSharedPtr<UE::Dataflow::FEngineContext> DataflowContext = DataflowEditorContextObject->GetDataflowContext())
		{
			if (DataflowEditorContextObject->GetSelectedCollection().IsValid())
			{
				// Fill the attribute values
				SkinWeightNode->FillAttributeWeights(*DataflowEditorContextObject->GetSelectedCollection().Get(),
					SkinWeightNode->GetBoneIndicesKey(*DataflowContext), SkinWeightNode->GetBoneWeightsKey(*DataflowContext),
					OutSetupIndices, OutSetupWeights);

				OutCurrentWeights.SetNumZeroed(OutSetupWeights.Num());
				OutCurrentIndices.SetNumZeroed(OutSetupIndices.Num());

				SkinWeightNode->ExtractVertexWeights(*DataflowContext, OutSetupIndices, OutSetupWeights,
					MakeArrayView(OutCurrentIndices), MakeArrayView(OutCurrentWeights));

				return true;
			}
		}
	}
	return false;
}

void UDataflowEditorSkinWeightsPaintTool::Setup()
{
	USkinWeightsPaintTool::Setup();

	ISkeletonProvider* SkeletonProvider = CastChecked<ISkeletonProvider>(Target);
	const FReferenceSkeleton& RefSkeleton = SkeletonProvider->GetSkeleton();

	ensure(BoneManipulator == nullptr);
	BoneManipulator = NewObject<UDataflowBoneManipulator>();
	BoneManipulator->Setup(GetToolManager(), RefSkeleton);
	BoneManipulator->OnReferenceSkeletonUpdated.AddWeakLambda(this, 
		[this](UDataflowBoneManipulator& Manipulator) 
		{
			if (URefSkeletonPoser* Poser = Manipulator.GetRefSkeletonPoser())
			{
				PoseMesh(Poser);
				if (SkinWeightNode)
				{
					if (TRefCountPtr<FDataflowDebugDrawSkeletonObject> DebugDrawSkeleton = SkinWeightNode->GetDebugDrawSkeleton())
					{
						DebugDrawSkeleton->OverrideBoneTransforms(Poser->GetComponentSpaceTransforms());
					}
				}
			}
		});

	if (WeightToolProperties)
	{
		ToggleBoneManipulation(WeightToolProperties->EditingMode == EWeightEditMode::Bones);
	}

	// enable back face option 
	if (WeightToolProperties)
	{
		WeightToolProperties->bShowHitBackFaces = true;
	}

	// detect changes so that the accept button can be enabled only after a change has be made
	OnWeightsChanged.AddWeakLambda(this,
		[this]()
		{
			bWeightsHaveChanged = true;
		});
}

void UDataflowEditorSkinWeightsPaintTool::OnShutdown(EToolShutdownType ShutdownType)
{
	USkinWeightsPaintTool::OnShutdown(ShutdownType);

	TSharedPtr<UE::Dataflow::FEngineContext> DataflowContext = DataflowEditorContextObject->GetDataflowContext();
	if(DataflowContext && Target && ShutdownType == EToolShutdownType::Accept)
	{
		const int32 VertexOffset = GetVertexOffset();
		if(VertexOffset != INDEX_NONE)
		{
			// Save previous state for undo
			if (TObjectPtr<UDataflow> Dataflow = DataflowEditorContextObject->GetDataflowAsset())
			{
				if (UInteractiveToolManager* ToolManager = GetToolManager())
				{
					if (IToolsContextTransactionsAPI* TransactionAPI = ToolManager->GetContextTransactionsAPI())
					{
						TransactionAPI->AppendChange(
							Dataflow,
							SkinWeightNode->MakeEditNodeToolChange(),
							LOCTEXT("UDataflowEditorSkinWeightsPaintTool_ChangeDescription", "Update Skin Weight Node")
						);
					}
				}
				Dataflow->MarkPackageDirty();
			}

			TArray<TArray<int32>> CurrentIndices;
			TArray<TArray<float>> CurrentWeights;
			TArray<TArray<int32>> SetupIndices;
			TArray<TArray<float>> SetupWeights;

			if (ExtractSkinWeights(CurrentIndices, CurrentWeights, SetupIndices, SetupWeights))
			{
				// profile to edit
				const FName ActiveProfile = GetActiveSkinWeightProfileName();

				FDynamicMeshVertexSkinWeightsAttribute* SkinWeights = EditedMesh.Attributes()->GetSkinWeightsAttribute(ActiveProfile);

				FNonManifoldMappingSupport NonManifoldMappingSupport(EditedMesh);
				for (int32 DynaMeshVertexIndex : EditedMesh.VertexIndicesItr())
				{
					int32 VertexIndex = NonManifoldMappingSupport.GetOriginalNonManifoldVertexID(DynaMeshVertexIndex);

					UE::AnimationCore::FBoneWeights BoneWeights;
					SkinWeights->GetValue(DynaMeshVertexIndex, BoneWeights);

					const int32 CurrentIndex = VertexOffset + VertexIndex;

					if (CurrentIndex < CurrentWeights.Num() && CurrentIndex < CurrentIndices.Num() &&
						CurrentWeights[CurrentIndex].Num() == CurrentIndices[CurrentIndex].Num())
					{
						CurrentIndices[CurrentIndex].Reset(BoneWeights.Num());
						CurrentWeights[CurrentIndex].Reset(BoneWeights.Num());

						TArray<TPair<int32, float>> SortedWeights;
						SortedWeights.Reserve(BoneWeights.Num());

						for (int32 WeightIndex = 0, NumWeights = BoneWeights.Num(); WeightIndex < NumWeights; ++WeightIndex)
						{
							SortedWeights.Add({ BoneWeights[WeightIndex].GetBoneIndex(), BoneWeights[WeightIndex].GetWeight() });
						}
						Algo::Sort(SortedWeights, [](const TPair<int32, float>& A, const TPair<int32, float>& B)
							{
								return A.Value > B.Value;
							});

						for (int32 WeightIndex = 0, NumWeights = BoneWeights.Num(); WeightIndex < NumWeights; ++WeightIndex)
						{
							CurrentIndices[CurrentIndex].Add(SortedWeights[WeightIndex].Key);
							CurrentWeights[CurrentIndex].Add(SortedWeights[WeightIndex].Value);
						}
					}
				}

				SkinWeightNode->ReportVertexWeights(*DataflowContext, SetupIndices, SetupWeights, CurrentIndices, CurrentWeights);
				SkinWeightNode->Invalidate();

				// Avoid rebuilding the skeletal mesh after updating the skin weights
				SkinWeightNode->ValidateSkeletalMeshes();
			}
		}
	}

	SetSkinWeightNode(nullptr);

	if (BoneManipulator)
	{
		BoneManipulator->Shutdown(GetToolManager());
	}
}

void UDataflowEditorSkinWeightsPaintTool::PoseMesh(URefSkeletonPoser* Poser)
{
	const TArray<FTransform>& ComponentSpaceTransforms = Poser->GetComponentSpaceTransforms();
	if (ComponentSpaceTransforms.IsEmpty())
	{
		return;
	}

	const TArray<FTransform>& ComponentSpaceTransformsRefPose = Poser->GetComponentSpaceTransformsRefPose();
	TArray<FMatrix> BoneMatrices = SkeletalMeshToolsHelper::ComputeBoneMatrices(ComponentSpaceTransformsRefPose, ComponentSpaceTransforms);

	auto DeformPreviewMeshFunc = [this, &BoneMatrices](FDynamicMesh3& VisualMesh)
		{
			auto WriteFunc = [&VisualMesh](SkeletalMeshToolsHelper::FVertInfo VertInfo, const FVector& PosedVertPos)
				{
					VisualMesh.SetVertex(VertInfo.VertID, PosedVertPos);
				};

			const TMap<FName, float> MorphTargetWeights;
			SkeletalMeshToolsHelper::GetPosedMesh(WriteFunc, EditedMesh, BoneMatrices, NAME_None, MorphTargetWeights);
			UE::Geometry::FMeshNormals::QuickRecomputeOverlayNormals(VisualMesh);
		};

	constexpr bool bRebuildSpatial = false;
	PreviewMesh->DeferredEditMesh(DeformPreviewMeshFunc, bRebuildSpatial);
	PreviewMesh->NotifyDeferredEditCompleted(UPreviewMesh::ERenderUpdateMode::FastUpdate, EMeshRenderAttributeFlags::Positions | EMeshRenderAttributeFlags::VertexNormals, bRebuildSpatial);
}

#undef LOCTEXT_NAMESPACE
