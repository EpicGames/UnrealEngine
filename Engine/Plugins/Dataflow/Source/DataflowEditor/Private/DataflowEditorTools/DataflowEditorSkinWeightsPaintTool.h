// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "DataflowEditorTools/DataflowEditorToolBuilder.h"
#include "DataflowEditorTools/DataflowEditorBoneManipulator.h"
#include "SkeletalMesh/SkinWeightsPaintTool.h"

#include "DataflowEditorSkinWeightsPaintTool.generated.h"

#define UE_API DATAFLOWEDITOR_API

class UDataflowContextObject;
class UDataflowEditorMode;
struct FDataflowCollectionEditSkinWeightsNode;

class URefSkeletonPoser;
struct FReferenceSkeleton;

/**
 * Dataflow skin weights tool builder
 */
UCLASS(MinimalAPI)
class UDataflowEditorSkinWeightsPaintToolBuilder : public USkinWeightsPaintToolBuilder, public IDataflowEditorToolBuilder
{
	GENERATED_BODY()

private:
	//~ Begin IDataflowEditorToolBuilder interface
	UE_API virtual void GetSupportedConstructionViewModes(const UDataflowContextObject& ContextObject, TArray<const UE::Dataflow::IDataflowConstructionViewMode*>& Modes) const override;
	virtual bool CanSetConstructionViewWireframeActive() const override { return false; }
	//~ End IDataflowEditorToolBuilder interface

	//~ Begin USkinWeightsPaintToolBuilder interface
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual UMeshSurfacePointTool* CreateNewTool(const FToolBuilderState& SceneState) const override;
	//~ End USkinWeightsPaintToolBuilder interface

protected:
	UE_API virtual const FToolTargetTypeRequirements& GetTargetRequirements() const override;
};

/**
 * Dataflow skin weights painting tool
 */
UCLASS(MinimalAPI)
class UDataflowEditorSkinWeightsPaintTool : public USkinWeightsPaintTool
{
	GENERATED_BODY()

public:
	void SetSkinWeightNode(FDataflowCollectionEditSkinWeightsNode* Node);

	/** Set the dataflow context object */
	void SetDataflowEditorContextObject(TObjectPtr<UDataflowContextObject> InDataflowEditorContextObject);

private:
	//~ Begin USkinWeightsPaintTool interface
	virtual FEditorViewportClient* GetViewportClient() const override {return nullptr;}
	UE_API virtual void Setup() override;
	UE_API virtual void OnShutdown(EToolShutdownType ShutdownType) override;
	virtual bool CanAccept() const override { return bWeightsHaveChanged; }
	UE_API virtual void ToggleBoneManipulation(bool bEnable);
	UE_API virtual const TArray<FTransform>& GetComponentSpaceBoneTransforms();
	//~ End USkinWeightsPaintTool interface

	void HandleOnBoneSelectionChanged(const TArray<FName>& BoneNames);
	void HandleOnUpdateRefSkeletonToDraw(FReferenceSkeleton& InOutRefSkeletonToUpdate);

	/** extract the initial weights and indices values as well as the delta values */
	bool ExtractSkinWeights(TArray<TArray<int32>>& OutCurrentIndices, TArray<TArray<float>>& OutCurrentWeights, TArray<TArray<int32>>& OutSetupIndices, TArray<TArray<float>>& OutSetupWeights);

	void PoseMesh(URefSkeletonPoser* Poser);

	/** Get the skeletal mesh vertex offset in the node collection */
	int32 GetVertexOffset() const;

	/** Skin weight node associated with that tool */
	FDataflowCollectionEditSkinWeightsNode* SkinWeightNode = nullptr;

	/** Dataflow context object to be used to retrieve the node context data*/
	TObjectPtr<UDataflowContextObject> DataflowEditorContextObject;

	TObjectPtr<UDataflowBoneManipulator> BoneManipulator = nullptr;
	
	FDelegateHandle OnBoneSelectionChangedDelegateHandle;

	bool bWeightsHaveChanged = false;
};


#undef UE_API
