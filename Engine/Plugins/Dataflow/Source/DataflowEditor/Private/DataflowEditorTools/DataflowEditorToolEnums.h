// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataflowEditorToolEnums.generated.h"

UENUM()
enum class EDataflowEditorToolEditMode : uint8
{
	Brush,
	Mesh,
};

UENUM()
enum class EDataflowEditorToolBrushAreaType : uint8
{
	Connected,
	Volumetric
};

UENUM()
enum class EDataflowEditorToolEditOperation : uint8
{
	Add,
	Replace,
	Multiply,
	Invert,
	Relax,
};

UENUM()
enum class EDataflowEditorToolColorMode : uint8
{
	Greyscale,
	Ramp,
	FullMaterial,
};

UENUM()
enum class EDataflowEditorToolVisibilityType : uint8
{
	None,
	Unoccluded,
};

// mirror direction mode
UENUM()
enum class EDataflowEditorToolMirrorDirection : uint8
{
	PositiveToNegative,
	NegativeToPositive,
};

/** Value Query mode  */
UENUM()
enum class EDataflowEditorToolValueQueryType : uint8
{
	/** Value is interpolated from triangle vertices */
	Interpolated UMETA(DisplayName = "Interpolated"),

	/** Return the value at the closest vertex of the triangle under the mouse cursor */
	NearestVertexFast UMETA(DisplayName = "Nearest Vertex (Fast)"),

	/** Return the value of the nearest vertex inside the brush radius, even if the vertex is not on the triangle under the mouse cursor */
	NearestVertexAccurate UMETA(DisplayName = "Nearest Vertex (Accurate)")
};