// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataflowEditorTools/DataflowEditorVertexAttributePaintTool.h"

#include "BaseBehaviors/TwoAxisPropertyEditBehavior.h"
#include "BaseGizmos/BrushStampIndicator.h"
#include "CanvasTypes.h"
#include "CanvasItem.h"
#include "ContextObjectStore.h"
#include "Dataflow/DataflowCollectionAddScalarVertexPropertyNode.h"
#include "Dataflow/DataflowConstructionScene.h"
#include "Dataflow/DataflowContent.h"
#include "Dataflow/DataflowEditorCollectionComponent.h"
#include "Dataflow/DataflowEditorMode.h"
#include "Dataflow/DataflowNode.h"
#include "Dataflow/DataflowRenderingViewMode.h"
#include "DataflowEditorTools/DataflowEditorWeightMapPaintBrushOps.h"
#include "DataflowEditorTools/DataflowEditorWeightMapPaintTool.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/DynamicMeshChangeTracker.h"
#include "Drawing/MeshElementsVisualizer.h"
#include "DynamicMesh/MeshIndexUtil.h"
#include "DynamicMesh/NonManifoldMappingSupport.h"
#include "DynamicMeshToMeshDescription.h"
#include "Engine/Engine.h"
#include "InteractiveToolManager.h"
#include "InteractiveGizmoManager.h"
#include "Intersection/IntrLine2Line2.h"
#include "Intersection/IntrSegment2Segment2.h"
#include "IPersonaEditorModeManager.h"
#include "Framework/Notifications/NotificationManager.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Logging/LogMacros.h"
#include "Math/Box.h"
#include "ModelingToolTargetUtil.h"
#include "Polygon2.h"
#include "PreviewMesh.h"
#include "StaticMeshAttributes.h"
#include "Selection/PolygonSelectionMechanic.h"
#include "Selections/MeshConnectedComponents.h"
#include "Selections/MeshVertexSelection.h"
#include "Spatial/PointHashGrid3.h"
#include "Spatial/SparseDynamicOctree3.h"
#include "ToolSetupUtil.h"
#include "Util/BufferUtil.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Generators/RectangleMeshGenerator.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowEditorVertexAttributePaintTool)

DEFINE_LOG_CATEGORY_STATIC(LogDataflowEditorVertexAttributePaintTool, Warning, All);

#define LOCTEXT_NAMESPACE "DataflowEditorVertexAttributePaintTool"

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////


namespace UE::DataflowEditorVertexAttributePaintTool::CVars
{
	bool DataflowEditorUseNewWeightMapTool = true;
	FAutoConsoleVariableRef CVarDataflowEditorUseNewWeightMapTool(
		TEXT("p.DataflowEditor.UseNewWeightMapTool"),
		DataflowEditorUseNewWeightMapTool,
		TEXT("when on, the new weight map tool with improved UX will be used"));
}

namespace UE::DataflowEditorVertexAttributePaintTool::Private
{
	static const FString MirrorBrushGizmoType = TEXT("DataflowEditorVertexAttributePaintToolMirrorGizmo");

	// probably should be something defined for the whole tool framework...
#if WITH_EDITOR
	static EAsyncExecution WeightPaintToolAsyncExecTarget = EAsyncExecution::LargeThreadPool;
#else
	static EAsyncExecution WeightPaintToolAsyncExecTarget = EAsyncExecution::ThreadPool;
#endif

	static bool HasManagedArrayCollection(const FDataflowNode* InDataflowNode, const TSharedPtr<UE::Dataflow::FEngineContext> Context)
	{
		static const FName CollectionTypeName = "FManagedArrayCollection";
		if (InDataflowNode && Context)
		{
			for (const FDataflowOutput* const Output : InDataflowNode->GetOutputs())
			{
				if (Output->GetType() == CollectionTypeName)
				{
					return true;
				}
			}
		}
		return false;
	}

	static void ShowEditorMessage(ELogVerbosity::Type InMessageType, const FText& InMessage)
	{
		FNotificationInfo Notification(InMessage);
		Notification.bUseSuccessFailIcons = true;
		Notification.ExpireDuration = 5.0f;

		SNotificationItem::ECompletionState State = SNotificationItem::CS_Success;

		switch (InMessageType)
		{
		case ELogVerbosity::Warning:
			UE_LOGF(LogDataflowEditorVertexAttributePaintTool, Warning, "%ls", *InMessage.ToString());
			break;
		case ELogVerbosity::Error:
			State = SNotificationItem::CS_Fail;
			UE_LOGF(LogDataflowEditorVertexAttributePaintTool, Error, "%ls", *InMessage.ToString());
			break;
		default:
			break; // don't log anything unless a warning or error
		}

		FSlateNotificationManager::Get().AddNotification(Notification)->SetCompletionState(State);
	}

	/**
		* A wrapper change that applies a given change to the unwrap canonical mesh of an input, and uses that
		* to update the other views. Causes a broadcast of OnCanonicalModified.
		*/
	class  FMeshChange : public FToolCommandChange
	{
	public:
		FMeshChange(UDynamicMeshComponent* DynamicMeshComponentIn, TUniquePtr<UE::Geometry::FDynamicMeshChange> DynamicMeshChangeIn)
			: DynamicMeshComponent(DynamicMeshComponentIn)
			, DynamicMeshChange(MoveTemp(DynamicMeshChangeIn))
		{
			ensure(DynamicMeshComponentIn);
			ensure(DynamicMeshChange);
		};

		virtual void Apply(UObject* Object) override
		{
			DynamicMeshChange->Apply(DynamicMeshComponent->GetMesh(), false);
		}

		virtual void Revert(UObject* Object) override
		{
			DynamicMeshChange->Apply(DynamicMeshComponent->GetMesh(), true);
		}

		virtual bool HasExpired(UObject* Object) const override
		{
			return !(DynamicMeshComponent.IsValid() && DynamicMeshChange);
		}

		virtual FString ToString() const override
		{
			return TEXT("DataflowEditorVertexAttributePaintToolMeshChange");
		}

	protected:
		TWeakObjectPtr<UDynamicMeshComponent> DynamicMeshComponent;
		TUniquePtr<UE::Geometry::FDynamicMeshChange> DynamicMeshChange;
	};
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
//
// ToolBuilder
//
///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

void UDataflowEditorVertexAttributePaintToolBuilder::GetSupportedConstructionViewModes(const UDataflowContextObject& ContextObject, TArray<const UE::Dataflow::IDataflowConstructionViewMode*>& Modes) const
{
	if (!UE::DataflowEditorVertexAttributePaintTool::CVars::DataflowEditorUseNewWeightMapTool && FallbackToolBuilder)
	{
		if (const IDataflowEditorToolBuilder* FallbackDataflowToolBuilder = Cast<IDataflowEditorToolBuilder>(FallbackToolBuilder))
		{
			FallbackDataflowToolBuilder->GetSupportedConstructionViewModes(ContextObject, Modes);
			return;
		}
	}

	if (const FDataflowVertexAttributeEditableNode* SelectedNode = ContextObject.GetSelectedNodeOfType<FDataflowVertexAttributeEditableNode>())
	{
		if (const TSharedPtr<UE::Dataflow::FEngineContext>& EvalContext = ContextObject.GetDataflowContext())
		{
			TArray<FName> ViewModeNames;
			SelectedNode->GetSupportedViewModes(*EvalContext, ViewModeNames);

			const UE::Dataflow::FRenderingViewModeFactory& Factory = UE::Dataflow::FRenderingViewModeFactory::GetInstance();
			for (FName ViewModeName : ViewModeNames)
			{
				if (const UE::Dataflow::IDataflowConstructionViewMode* ViewMode = Factory.GetViewMode(ViewModeName))
				{
					Modes.Add(ViewMode);
				}
			}
		}
	}
}

bool UDataflowEditorVertexAttributePaintToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	if (!UE::DataflowEditorVertexAttributePaintTool::CVars::DataflowEditorUseNewWeightMapTool && FallbackToolBuilder)
	{
		return FallbackToolBuilder->CanBuildTool(SceneState);
	}
	
	if (UMeshSurfacePointMeshEditingToolBuilder::CanBuildTool(SceneState))
	{
		if (SceneState.SelectedComponents.Num() == 1)
		{
			if (TObjectPtr<UDataflowEditorCollectionComponent> Component = Cast<UDataflowEditorCollectionComponent>(SceneState.SelectedComponents[0]))
			{
				if (UDataflowBaseContent* ContextObject = SceneState.ToolManager->GetContextObjectStore()->FindContext<UDataflowBaseContent>())
				{
					if (ContextObject->GetSelectedNode() == Component->Node)
					{
						if (const TSharedPtr<UE::Dataflow::FEngineContext> EvaluationContext = ContextObject->GetDataflowContext())
						{
							if (const FDataflowNode* PrimarySelection = ContextObject->GetSelectedNodeOfType<FDataflowVertexAttributeEditableNode>())
							{
								return UE::DataflowEditorVertexAttributePaintTool::Private::HasManagedArrayCollection(PrimarySelection, EvaluationContext);
							}
						}
					}
				}
			}
		}
	}
	return false;
}

void UDataflowEditorVertexAttributePaintToolBuilder::SetEditorMode(UDataflowEditorMode* InMode)
{
	UE::Dataflow::RegisterModeForToolManager(ModeForToolManager, InMode);
}

UMeshSurfacePointTool* UDataflowEditorVertexAttributePaintToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	UDataflowContextObject* ContextObject = SceneState.ToolManager->GetContextObjectStore()->FindContext<UDataflowContextObject>();
	UDataflowEditorMode* OwningMode = UE::Dataflow::FindModeForToolManager(ModeForToolManager, SceneState.ToolManager);

	if (UE::DataflowEditorVertexAttributePaintTool::CVars::DataflowEditorUseNewWeightMapTool)
	{
		UDataflowEditorVertexAttributePaintTool* PaintTool = NewObject<UDataflowEditorVertexAttributePaintTool>(SceneState.ToolManager);
		PaintTool->SetEditorMode(OwningMode);
		PaintTool->SetWorld(SceneState.World);
		if (ContextObject)
		{
			PaintTool->SetDataflowEditorContextObject(ContextObject);
		}
		return PaintTool;
	}
	else if (FallbackToolBuilder)
	{
		return FallbackToolBuilder->CreateNewTool(SceneState);
	}

	// fallback : use the old tool
	// todo(ccaillaud) : remove this fall back when the new tool is on par with the old one
	UDataflowEditorWeightMapPaintTool* PaintTool = NewObject<UDataflowEditorWeightMapPaintTool>(SceneState.ToolManager);
	PaintTool->SetEditorMode(OwningMode);
	PaintTool->SetWorld(SceneState.World);
	if (ContextObject)
	{
		PaintTool->SetDataflowEditorContextObject(ContextObject);
	}
	return PaintTool;
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
//
// Data Adapter
//
///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
void FDataflowEditorVertexAttributePaintToolDataAdapter::Setup(
	const FDynamicMesh3& InMesh,
	UE::Geometry::FDynamicMeshWeightAttribute* InActiveWeightMap,
	FDataflowVertexAttributeEditableNode* InNodeToUpdate,
	TObjectPtr<UDataflowContextObject> InDataflowEditorContextObject
	)
{
	ActiveWeightMap = InActiveWeightMap;
	NodeToUpdate = InNodeToUpdate;

	if (InDataflowEditorContextObject)
	{
		TArray<int32> ExtraMappingToWeight;
		TArray<TArray<int32>> ExtraMappingFromWeight;
		if (NodeToUpdate && InDataflowEditorContextObject)
		{
			if (const UE::Dataflow::IDataflowConstructionViewMode* ViewMode = InDataflowEditorContextObject->GetConstructionViewMode())
			{
				if (TSharedPtr<UE::Dataflow::FEngineContext> DataflowContext = InDataflowEditorContextObject->GetDataflowContext())
				{
					NodeToUpdate->GetExtraVertexMapping(*DataflowContext, ViewMode->GetName(), ExtraMappingToWeight, ExtraMappingFromWeight);
				}
			}
		}
		

		// Generate mapping array from and to non-manifiold and manifold mesh 
		if (const TSharedPtr<const FManagedArrayCollection> Collection = InDataflowEditorContextObject->GetRenderCollection())
		{
			using namespace UE::Dataflow;
			const UE::Geometry::FNonManifoldMappingSupport NonManifoldMapping(InMesh);
			const bool bHasNonManifoldMapping = NonManifoldMapping.IsNonManifoldVertexInSource();
			const bool bHasExtraMapping = (ExtraMappingToWeight.Num() != 0 && ExtraMappingFromWeight.Num() != 0);

			bHaveDynamicMeshToWeightConversion = bHasNonManifoldMapping || bHasExtraMapping;

			if (bHasNonManifoldMapping)
			{
				DynamicMeshToWeight.SetNumUninitialized(InMesh.VertexCount());
				WeightToDynamicMesh.Reset();
				WeightToDynamicMesh.SetNum(Collection->NumElements("Vertices")); // todo(ccaillaud) Do we realy need the render collection for that ?
				for (int32 DynamicMeshVert = 0; DynamicMeshVert < InMesh.VertexCount(); ++DynamicMeshVert)
				{
					DynamicMeshToWeight[DynamicMeshVert] = NonManifoldMapping.GetOriginalNonManifoldVertexID(DynamicMeshVert);
					if (bHasExtraMapping)
					{
						DynamicMeshToWeight[DynamicMeshVert] = ExtraMappingToWeight[DynamicMeshToWeight[DynamicMeshVert]];
					}
					if (WeightToDynamicMesh.IsValidIndex(DynamicMeshToWeight[DynamicMeshVert]))
					{
						WeightToDynamicMesh[DynamicMeshToWeight[DynamicMeshVert]].Add(DynamicMeshVert);
					}
					else
					{
						bHaveDynamicMeshToWeightConversion = false;
						UE_LOGF(LogTemp, Warning, "Weight map misalignment, disabling remapping");
						break;
					}
				}
			}
			else if (bHasExtraMapping)
			{
				DynamicMeshToWeight = ExtraMappingToWeight;
				WeightToDynamicMesh = ExtraMappingFromWeight;
			}
		}

		if (NodeToUpdate)
		{
			// Setup DynamicMeshToWeight conversion and get Input weight map (if it exists)
			const int32 NumExpectedWeights = bHaveDynamicMeshToWeightConversion ? WeightToDynamicMesh.Num() : InMesh.MaxVertexID();

			TArray<float> CurrentWeights;
			// Get the setup attribute values
			if (TSharedPtr<UE::Dataflow::FEngineContext> DataflowContext = InDataflowEditorContextObject->GetDataflowContext())
			{
				NodeToUpdate->GetVertexAttributeValues(*DataflowContext, CurrentWeights);
			}
			ensure(CurrentWeights.Num() == NumExpectedWeights);

			if (bHaveDynamicMeshToWeightConversion)
			{
				if (WeightToDynamicMesh.Num() == CurrentWeights.Num())	// Only copy node weights if they match the number of mesh vertices
				{
					for (int32 WeightID = 0; WeightID < CurrentWeights.Num(); ++WeightID)
					{
						for (const int32 VertexID : WeightToDynamicMesh[WeightID])
						{
							ActiveWeightMap->SetValue(VertexID, &CurrentWeights[WeightID]);
						}
					}
				}
			}
			else
			{
				if (InMesh.MaxVertexID() == CurrentWeights.Num())	// Only copy node weights if they match the number of mesh vertices
				{
					for (int32 VertexID = 0; VertexID < CurrentWeights.Num(); ++VertexID)
					{
						ActiveWeightMap->SetValue(VertexID, &CurrentWeights[VertexID]);
					}
				}
			}
		}
	}
}

void FDataflowEditorVertexAttributePaintToolDataAdapter::CommitToNode(TObjectPtr<UDataflowContextObject> InDataflowEditorContextObject, IToolsContextTransactionsAPI* TransactionAPI)
{
	if (ActiveWeightMap && NodeToUpdate)
	{
		if (const FDynamicMesh3* Mesh = ActiveWeightMap->GetParent())
		{
			if (TSharedPtr<UE::Dataflow::FEngineContext> DataflowContext = InDataflowEditorContextObject->GetDataflowContext())
			{
				// Save previous state for undo
				if (TObjectPtr<UDataflow> Dataflow = InDataflowEditorContextObject ? InDataflowEditorContextObject->GetDataflowAsset(): nullptr)
				{
					if (TransactionAPI)
					{
						TransactionAPI->AppendChange(
							Dataflow,
							NodeToUpdate->MakeEditNodeToolChange(),
							LOCTEXT("DataflowEditorVertexAttributePaintTool_ChangeDescription", "Update Weight Map Node")
						);
					}
					Dataflow->MarkPackageDirty();
				}

				// apply the new values to the node
				TArray<float> WeightsToApply;
				const int32 NumVertices = Mesh->VertexCount();
				WeightsToApply.SetNumUninitialized(NumVertices);
				for (int32 VertexID = 0; VertexID < NumVertices; ++VertexID)
				{
					ActiveWeightMap->GetValue(VertexID, &WeightsToApply[VertexID]);
				}
				// WeightsToApply represents the weights of the selected sub-mesh, DynamicMeshToWeight allow to remap properly to the full collection mesh
				NodeToUpdate->SetVertexAttributeValues(*DataflowContext, WeightsToApply, DynamicMeshToWeight);
				NodeToUpdate->Invalidate();
			}
		}
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
//
// Tool
//
///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

bool UDataflowEditorVertexAttributePaintTool::SetupToolMesh(FDynamicMesh3& InOutToolMesh, int32& OutInitialAttributeIndex)
{
	// Create a temp paint layer and allow the selected node to read from/write to that layer
	const int32 NumAttributeLayers = InOutToolMesh.Attributes()->NumWeightLayers();

	InOutToolMesh.Attributes()->SetNumWeightLayers(NumAttributeLayers + 1);
	UE::Geometry::FDynamicMeshWeightAttribute* ActiveWeightMap = InOutToolMesh.Attributes()->GetWeightLayer(NumAttributeLayers);
	ActiveWeightMap->SetName(FName("PaintLayer"));
	
	FDataflowVertexAttributeEditableNode* NodeToUpdate = DataflowEditorContextObject->GetSelectedNodeOfType<FDataflowVertexAttributeEditableNode>();
	checkf(NodeToUpdate, TEXT("No Node is currently selected, or more than one node is selected"));
	
	// Initialize the active weight map with weights stored on the node
	DataAdapter.Setup(InOutToolMesh, ActiveWeightMap, NodeToUpdate, DataflowEditorContextObject);
	
	OutInitialAttributeIndex = NumAttributeLayers;

	return true;
}

void UDataflowEditorVertexAttributePaintTool::CommitToolMesh(FDynamicMesh3& InToolMesh)
{
	if (UInteractiveToolManager* ToolManager = GetToolManager())
	{
		ToolManager->BeginUndoTransaction(LOCTEXT("DataflowEditorVertexAttributePaintTool_TransactionName", "Paint Weights"));

		DataAdapter.CommitToNode(DataflowEditorContextObject, ToolManager->GetContextTransactionsAPI());

		ToolManager->EndUndoTransaction();
	}
}

void UDataflowEditorVertexAttributePaintTool::Setup()
{
	Super::Setup();

	// todo(ccaillaud) : this should be a parameter of the tool ? 
	SetToolDisplayName(LOCTEXT("ToolName", "Paint Weight Maps"));

	// Hide all meshes in the DataflowConstructionScene, as we will be painting onto our own Preview mesh
	if (ensure(Mode))
	{
		if (const TSharedPtr<FDataflowConstructionScene> Scene = Mode->GetDataflowConstructionScene().Pin())
		{
			Scene->SetVisibility(false);
			TWeakObjectPtr<UDataflowEditorVertexAttributePaintTool> WeakThis = this;
			Scene->OnConstructionSceneUpdated.AddWeakLambda(this, [WeakThis, WeakScene = TWeakPtr<FDataflowConstructionScene>(Scene)]()
				{
					if (TStrongObjectPtr<UDataflowEditorVertexAttributePaintTool> This = WeakThis.Pin())
					{
						if (This->PreviewMesh)
						{
							if (const TSharedPtr<FDataflowConstructionScene> PinnedScene = WeakScene.Pin())
							{
								PinnedScene->SetVisibility(false);
							}
						}
					}
				});
		}
	}
}

void UDataflowEditorVertexAttributePaintTool::SetFocusInViewport() const
{
	if (ensure(Mode))
	{
		Mode->SetInputFocusOnConstructionViewport();
	}

	//if (UPersonaEditorModeManagerContext* PersonaModeManagerContext = GetToolManager()->GetContextObjectStore()->FindContext<UPersonaEditorModeManagerContext>())
	//{
	//	PersonaModeManagerContext->SetFocusInViewport();
	//}
}

void UDataflowEditorVertexAttributePaintTool::SetDataflowEditorContextObject(TObjectPtr<UDataflowContextObject> InDataflowEditorContextObject)
{
	DataflowEditorContextObject = InDataflowEditorContextObject;
}

void UDataflowEditorVertexAttributePaintTool::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	UDataflowEditorVertexAttributePaintTool* This = CastChecked<UDataflowEditorVertexAttributePaintTool>(InThis);
	Collector.AddReferencedObject(This->DataflowEditorContextObject);
	Super::AddReferencedObjects(InThis, Collector);
}


#undef LOCTEXT_NAMESPACE

