// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataflowEditorTools/DataflowEditorWeightMapPaintBrushOps.h"
#include "Async/ParallelFor.h"
#include "DynamicMesh/DynamicMesh3.h"
//
// Paint Brush
//

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowEditorWeightMapPaintBrushOps)

void FDataflowWeightMapPaintBrushOp::ApplyStampByVertices(
	const FDynamicMesh3* Mesh,
	const FSculptBrushStamp& Stamp,
	const TArray<int32>& Vertices,
	TArray<double>& NewAttributesOut)
{
	UDataflowWeightMapPaintBrushOpProps* const Props = GetPropertySetAs<UDataflowWeightMapPaintBrushOpProps>();
	const double TargetValue = (double)Props->GetAttribute();

	const int32 NumVertices = Vertices.Num();
	ParallelFor(NumVertices, [&](int32 k)
	{
		if (bApplyRadiusLimit)
		{
			const FVector3d& StampPos = Stamp.LocalFrame.Origin;
			const int32 VertIdx = Vertices[k];
			const FVector3d VertexPos = Mesh->GetVertex(VertIdx);
			const double DistanceSquared = (VertexPos - StampPos).SquaredLength();
			if (DistanceSquared >= Stamp.Radius * Stamp.Radius)
			{
				return;
			}
		}

		const double ExistingValue = NewAttributesOut[k];
		const double ValueDiff = TargetValue - ExistingValue;
		NewAttributesOut[k] = FMath::Clamp(ExistingValue + Stamp.Power * ValueDiff, 0.0, 1.0);
	});
}


//
// Erase Brush
//

void FDataflowWeightMapEraseBrushOp::ApplyStampByVertices(
	const FDynamicMesh3* Mesh,
	const FSculptBrushStamp& Stamp,
	const TArray<int32>& Vertices,
	TArray<double>& NewAttributesOut)
{
	UDataflowWeightMapEraseBrushOpProps* Props = GetPropertySetAs<UDataflowWeightMapEraseBrushOpProps>();
	const double EraseAttribute = (double)Props->GetAttribute();

	// TODO: Add something here to get the old value so we can subtract (clamped) the AttributeValue from it.
	// TODO: Handle the stamp's properties for fall off, etc..

	check(NewAttributesOut.Num() == Vertices.Num());

	for (int32 k = 0; k < Vertices.Num(); ++k)
	{
		if (bApplyRadiusLimit)
		{
			const FVector3d& StampPos = Stamp.LocalFrame.Origin;
			const int32 VertIdx = Vertices[k];
			const FVector3d VertexPos = Mesh->GetVertex(VertIdx);
			const double DistanceSquared = (VertexPos - StampPos).SquaredLength();
			if (DistanceSquared >= Stamp.Radius * Stamp.Radius)
			{
				continue;
			}
		}
		NewAttributesOut[k] = EraseAttribute;
	}
}


//
// Smooth Brush
//

void FDataflowWeightMapSmoothBrushOp::ApplyStampByVertices(
	const FDynamicMesh3* Mesh,
	const FSculptBrushStamp& Stamp,
	const TArray<int32>& Vertices,
	TArray<double>& VertexWeightValues)
{
	ApplyStampByVerticesStatic(Mesh, Stamp, Vertices, VertexWeightValues, bApplyRadiusLimit, Falloff);
}


/*static*/ void FDataflowWeightMapSmoothBrushOp::ApplyStampByVerticesStatic(
	const FDynamicMesh3* Mesh,
	const FSculptBrushStamp& Stamp,
	const TArray<int32>& Vertices,
	TArray<double>& VertexWeightValues,
	bool bApplyRadiusLimit,
	TSharedPtr<FMeshSculptFallofFunc> FalloffFunc)
{
	auto FalloffFunction = [&Stamp, &FalloffFunc](const FVector& Position)
		{
			if (FalloffFunc)
			{
				return (float)FalloffFunc->Evaluate(Stamp, Position);
			}
			return 1.0f;
		};
	FDataflowWeightMapSmoothBrushOp::ApplyToVerticesStatic(Mesh, Vertices, VertexWeightValues, Stamp.Power, bApplyRadiusLimit, Stamp.LocalFrame.Origin, Stamp.Radius, FalloffFunction);
}

/*static*/ void FDataflowWeightMapSmoothBrushOp::ApplyToVerticesStatic(
	const FDynamicMesh3* Mesh,
	const TArray<int32>& Vertices,
	TArray<double>& VertexWeightValues,
	const double SmoothingFactor,
	bool bApplyRadiusLimit,
	const FVector Center,
	const float Radius,
	FalloffFunction FalloffFunc
)
{
	// Converted from FClothPaintTool_Smooth::SmoothVertices
	const int32 NumVertices = Vertices.Num();

	TMap<int32, int32> VertexToBufferIndexMap;
	for (int32 BufferIndex = 0; BufferIndex < NumVertices; ++BufferIndex)
	{
		const int32 VertexIndex = Vertices[BufferIndex];
		VertexToBufferIndexMap.Add(VertexIndex, BufferIndex);
	}

	// Compute average values of one-rings for all vertices
	TArray<double> OneRingAverages;
	OneRingAverages.SetNumUninitialized(NumVertices);

	const TMap<int32, int32>& VertexToBufferIndexMapConst = VertexToBufferIndexMap;
	const TArray<double>& VertexWeightValuesConst = VertexWeightValues;
	ParallelFor(NumVertices, 
		[&Vertices, &Mesh, &VertexToBufferIndexMapConst, &VertexWeightValuesConst, &OneRingAverages]
		(int32 BufferIndex)
		{
			const int32 VertexIndex = Vertices[BufferIndex];

			// average include the vertex we are processing to avoid loosing information
			double Accumulator = VertexWeightValuesConst[BufferIndex];
			int32 NumNeighbors = 1;

			for (const int32 NeighborIndex : Mesh->VtxVerticesItr(VertexIndex))
			{
				if (VertexToBufferIndexMapConst.Contains(NeighborIndex))
				{
					const int32 NeighborBufferIndex = VertexToBufferIndexMapConst[NeighborIndex];
					Accumulator += VertexWeightValuesConst[NeighborBufferIndex];
					++NumNeighbors;
				}
			}

			OneRingAverages[BufferIndex] = Accumulator / NumNeighbors;
		}, 
		EParallelForFlags::None
	);

	// Blend vertex value with its average one-ring value
	ParallelFor(NumVertices,
		[&OneRingAverages, &VertexWeightValues, bApplyRadiusLimit, &Center, &Radius, &SmoothingFactor, &Vertices, &Mesh, &FalloffFunc](int32 BufferIndex)
		{
			float FalloffScalar = 1.0;
			if (bApplyRadiusLimit)
			{
				const FVector3d& StampPos = Center;
				const int32 VertexIndex = Vertices[BufferIndex];
				const FVector3d VertexPos = Mesh->GetVertex(VertexIndex);
				const double DistanceSquared = (VertexPos - StampPos).SquaredLength();
				if (DistanceSquared >= Radius * Radius)
				{
					return;
				}
				if (FalloffFunc.IsSet())
				{
					FalloffScalar = FalloffFunc(VertexPos);
				}
			}
			const float NewValue = FMath::Lerp(VertexWeightValues[BufferIndex], OneRingAverages[BufferIndex], SmoothingFactor * FalloffScalar);
			VertexWeightValues[BufferIndex] = FMath::Clamp(NewValue, 0.f, 1.f);
		}, 
		EParallelForFlags::None
	);
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

void FDataflowVertexAttributePaintBrushOp::ApplyStampByVertices(
	const FDynamicMesh3* Mesh,
	const FSculptBrushStamp& Stamp,
	const TArray<int32>& Vertices,
	TArray<double>& OutValues)
{
	check(OutValues.Num() == Vertices.Num());

	if (UDataflowVertexAttributePaintBrushOpProps* Props = GetPropertySetAs<UDataflowVertexAttributePaintBrushOpProps>())
	{
		ApplyStampByVerticesStatic(Mesh, Stamp, Vertices, OutValues, Props->EditOperation, Props->GetAttribute(), bApplyRadiusLimit, Falloff);
	}
}

/*static*/ void FDataflowVertexAttributePaintBrushOp::ApplyStampByVerticesStatic(
	const FDynamicMesh3* Mesh,
	const FSculptBrushStamp& Stamp,
	const TArray<int32>& Vertices,
	TArray<double>& OutValues,
	EDataflowEditorToolEditOperation EditOperation,
	const double ToolValue,
	bool bApplyRadiusLimit,
	TSharedPtr<FMeshSculptFallofFunc> FalloffFunc)
{
	auto FalloffFunction = [&Stamp, &FalloffFunc](const FVector& Position)
		{
			if (FalloffFunc)
			{
				return (float)FalloffFunc->Evaluate(Stamp, Position);
			}
			return 1.0f;
		};
	FDataflowVertexAttributePaintBrushOp::ApplyToVerticesStatic(Mesh, Vertices, OutValues, EditOperation, ToolValue, bApplyRadiusLimit, Stamp.LocalFrame.Origin, Stamp.Radius, FalloffFunction);
}

/*static*/ void FDataflowVertexAttributePaintBrushOp::ApplyToVerticesStatic(
	const FDynamicMesh3* Mesh,
	const TArray<int32>& Vertices,
	TArray<double>& OutValues,
	EDataflowEditorToolEditOperation EditOperation,
	const double ToolValue,
	bool bApplyRadiusLimit,
	const FVector Center,
	const float Radius,
	FalloffFunction FalloffFunc
)
{
	// special case for Relax option 
	if (EditOperation == EDataflowEditorToolEditOperation::Relax)
	{
		FDataflowWeightMapSmoothBrushOp::ApplyToVerticesStatic(Mesh, Vertices, OutValues, ToolValue, bApplyRadiusLimit, Center, Radius);
		return;
	}

	const int32 NumVertices = Vertices.Num();
	ParallelFor(NumVertices,
		[bApplyRadiusLimit, ToolValue, EditOperation, &Center, &Radius, &Mesh, &Vertices, &OutValues, &FalloffFunc]
		(int32 BufferIndex)
		{
			float FalloffScalar = 1.0;
			if (bApplyRadiusLimit)
			{
				const FVector3d& StampPos = Center;
				const int32 MeshVertIdx = Vertices[BufferIndex];
				const FVector3d VertexPos = Mesh->GetVertex(MeshVertIdx);
				const double DistanceSquared = (VertexPos - StampPos).SquaredLength();
				if (DistanceSquared >= Radius * Radius)
				{
					return;
				}
				if (FalloffFunc.IsSet())
				{
					FalloffScalar = FalloffFunc(VertexPos);
				}
			}

			const double CurrentValue = OutValues[BufferIndex];
			double NewValue = CurrentValue;

			switch (EditOperation)
			{
			case EDataflowEditorToolEditOperation::Add:
				NewValue = CurrentValue + ToolValue;
				break;
			case EDataflowEditorToolEditOperation::Replace:
				NewValue = ToolValue;
				break;
			case EDataflowEditorToolEditOperation::Multiply:
				NewValue = CurrentValue * ToolValue;
				break;
			case EDataflowEditorToolEditOperation::Invert:
				NewValue = 1.0 - CurrentValue; // no falloff
				break;
			case EDataflowEditorToolEditOperation::Relax:
				ensure(false); // already handled by SmoothBrush.ApplyStampByVertices
			}

			NewValue = FMath::Lerp(OutValues[BufferIndex], NewValue, FalloffScalar);
			OutValues[BufferIndex] = FMath::Clamp(NewValue, 0.0, 1.0);
		},
		EParallelForFlags::None
	);
}