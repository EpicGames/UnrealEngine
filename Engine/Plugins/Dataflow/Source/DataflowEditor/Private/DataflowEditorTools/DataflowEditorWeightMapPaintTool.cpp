// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataflowEditorTools/DataflowEditorWeightMapPaintTool.h"

#include "Async/ParallelFor.h"
#include "Async/Async.h"
#include "BaseBehaviors/TwoAxisPropertyEditBehavior.h"
#include "BaseGizmos/BrushStampIndicator.h"
#include "CanvasTypes.h"
#include "CanvasItem.h"
#include "Changes/MeshVertexChange.h"
#include "Changes/BasicChanges.h"
#include "ContextObjectStore.h"
#include "DataflowEditorTools/DataflowEditorWeightMapPaintBrushOps.h"
#include "Dataflow/DataflowCollectionAddScalarVertexPropertyNode.h"
#include "Dataflow/DataflowConstructionViewportClient.h"
#include "Dataflow/DataflowContent.h"
#include "Dataflow/DataflowObjectInterface.h"
#include "Dataflow/DataflowEdNode.h"
#include "Dataflow/DataflowEditorCollectionComponent.h"
#include "Dataflow/DataflowEditorMode.h"
#include "Dataflow/DataflowConstructionScene.h"
#include "Dataflow/DataflowObject.h"
#include "Dataflow/DataflowSNode.h"
#include "Dataflow/DataflowGraphEditor.h"
#include "Drawing/MeshElementsVisualizer.h"
#include "DynamicMesh/MeshNormals.h"
#include "DynamicMesh/MeshIndexUtil.h"
#include "DynamicMesh/NonManifoldMappingSupport.h"
#include "Engine/World.h"
#include "Framework/Notifications/NotificationManager.h"
#include "GraphEditor.h"
#include "InteractiveToolManager.h"
#include "InteractiveGizmoManager.h"
#include "Intersection/IntrLine2Line2.h"
#include "Intersection/IntrSegment2Segment2.h"
#include "ModelingToolTargetUtil.h"
#include "MeshWeights.h"
#include "ToolSetupUtil.h"
#include "Polygon2.h"
#include "Sculpting/StampFalloffs.h"
#include "Sculpting/MeshSculptUtil.h"
#include "Selection.h"
#include "Selections/MeshConnectedComponents.h"
#include "Selections/MeshFaceSelection.h"
#include "Selections/MeshVertexSelection.h"
#include "Selection/PolygonSelectionMechanic.h"
#include "Util/BufferUtil.h"
#include "Util/ColorConstants.h"
#include "Widgets/Notifications/SNotificationList.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowEditorWeightMapPaintTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UDataflowEditorWeightMapPaintTool"

namespace UE::Dataflow::Private
{
	// probably should be something defined for the whole tool framework...
#if WITH_EDITOR
	static EAsyncExecution WeightPaintToolAsyncExecTarget = EAsyncExecution::LargeThreadPool;
#else
	static EAsyncExecution WeightPaintToolAsyncExecTarget = EAsyncExecution::ThreadPool;
#endif
}


/*
 * ToolBuilder
 */
void UDataflowEditorWeightMapPaintToolBuilder::GetSupportedConstructionViewModes(const UDataflowContextObject& ContextObject, TArray<const UE::Dataflow::IDataflowConstructionViewMode*>& Modes) const
{
	//Modes.Add(UE::Dataflow::EDataflowPatternVertexType::Sim3D);
	//Modes.Add(UE::Dataflow::EDataflowPatternVertexType::Sim2D);
}

bool UDataflowEditorWeightMapPaintToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	auto HasManagedArrayCollection = [](const FDataflowNode* InDataflowNode, const TSharedPtr<UE::Dataflow::FEngineContext> Context)
	{
		if (InDataflowNode && Context)
		{
			for (const FDataflowOutput* const Output : InDataflowNode->GetOutputs())
			{
				if (Output->GetType() == FName("FManagedArrayCollection"))
				{
					return true;
				}
			}
		}

		return false;
	};

	if (UMeshSurfacePointMeshEditingToolBuilder::CanBuildTool(SceneState))
	{
		if (SceneState.SelectedComponents.Num() == 1)
		{
			if (TObjectPtr<UDataflowEditorCollectionComponent> Component = Cast<UDataflowEditorCollectionComponent>(SceneState.SelectedComponents[0]))
			{
				if (UDataflowBaseContent* ContextObject = SceneState.ToolManager->GetContextObjectStore()->FindContext<UDataflowBaseContent>())
				{
					if (ContextObject->GetSelectedNode() == Component->Node)
					{
						if (const TSharedPtr<UE::Dataflow::FEngineContext> EvaluationContext = ContextObject->GetDataflowContext())
						{
							if (const FDataflowNode* PrimarySelection = ContextObject->GetSelectedNodeOfType<FDataflowCollectionAddScalarVertexPropertyNode>())
							{
								return HasManagedArrayCollection(PrimarySelection, EvaluationContext);
							}
						}
					}
				}
			}
		}
	}
	return false;
}


void UDataflowEditorWeightMapPaintToolBuilder::SetEditorMode(UDataflowEditorMode* InMode)
{
	UE::Dataflow::RegisterModeForToolManager(ModeForToolManager, InMode);
}

UMeshSurfacePointTool* UDataflowEditorWeightMapPaintToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	UDataflowEditorWeightMapPaintTool* PaintTool = NewObject<UDataflowEditorWeightMapPaintTool>(SceneState.ToolManager);

	UDataflowEditorMode* OwningMode = UE::Dataflow::FindModeForToolManager(ModeForToolManager, SceneState.ToolManager);
	PaintTool->SetEditorMode(OwningMode);
	PaintTool->SetWorld(SceneState.World);

	if (UDataflowContextObject* ContextObject = SceneState.ToolManager->GetContextObjectStore()->FindContext<UDataflowContextObject>())
	{
		PaintTool->SetDataflowEditorContextObject(ContextObject);
	}

	return PaintTool;
}


/*
 * ToolActions
 */
void UDataflowEditorMeshWeightMapPaintToolActions::PostAction(EDataflowEditorWeightMapPaintToolActions Action)
{
	if (ParentTool.IsValid())
	{
		ParentTool->RequestAction(Action);
	}
}


/*
 * Properties
 */
void UDataflowEditorUpdateWeightMapProperties::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	// @todo(brice) Fix this
	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UDataflowEditorUpdateWeightMapProperties, Name))
	{
		//Dataflow::FWeightMapTools::MakeWeightMapName(Name);
	}
}


/*
 * Tool
 */

void UDataflowEditorWeightMapPaintTool::SetupBrushEditBehaviorSetup(ULocalTwoAxisPropertyEditInputBehavior& OutBehavior) 
{
	// map the horizontal behavior to change the brush size
	UMeshSculptToolBase::MapHorizontalBrushEditBehaviorToBrushSize(OutBehavior);

	// map the vertical ehavior to change the attribiute value
	OutBehavior.VerticalProperty.GetValueFunc = [this]()
		{
			return FilterProperties->AttributeValue;
		};

	OutBehavior.VerticalProperty.SetValueFunc = [this](float NewValue)
		{
			FilterProperties->AttributeValue = FMath::Min(1.0f, FMath::Max(NewValue, 0.f));
#if WITH_EDITOR
			FPropertyChangedEvent PropertyChangedEvent(UBrushBaseProperties::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UDataflowEditorWeightMapPaintBrushFilterProperties, AttributeValue)));
			FilterProperties->PostEditChangeProperty(PropertyChangedEvent);
#endif
		};
	OutBehavior.VerticalProperty.Name = LOCTEXT("AttributeValue", "Attribute Value");
	OutBehavior.VerticalProperty.EditRate = 0.005f;
	OutBehavior.VerticalProperty.bEnabled = true;
}

void UDataflowEditorWeightMapPaintTool::Setup()
{
	UMeshSculptToolBase::Setup();

	// Get the selected weight map node
	WeightMapNodeToUpdate = DataflowEditorContextObject->GetSelectedNodeOfType<FDataflowCollectionAddScalarVertexPropertyNode>();
	checkf(WeightMapNodeToUpdate, TEXT("No Weight Map Node is currently selected, or more than one node is selected"));

	SetToolDisplayName(LOCTEXT("ToolName", "Paint Weight Maps"));

	// Hide all meshes in the DataflowConstructionScene, as we will be painting onto our own Preview mesh.
	// Also re-hide on every scene rebuild: OnToolEnded -> ResetConstructionScene -> UpdateConstructionScene
	// recreates components default-visible, which (with auto-relaunch on cancel) would occlude our preview.
	if (const TSharedPtr<FDataflowConstructionScene> Scene = Mode->GetDataflowConstructionScene().Pin())
	{
		Scene->SetVisibility(false);
		Scene->OnConstructionSceneUpdated.AddWeakLambda(this, [WeakScene = TWeakPtr<FDataflowConstructionScene>(Scene)]()
		{
			if (const TSharedPtr<FDataflowConstructionScene> PinnedScene = WeakScene.Pin())
			{
				PinnedScene->SetVisibility(false);
			}
		});
	}

	// create dynamic mesh component to use for live preview
	FActorSpawnParameters SpawnInfo;
	PreviewMeshActor = TargetWorld->SpawnActor<AInternalToolFrameworkActor>(FVector::ZeroVector, FRotator::ZeroRotator, SpawnInfo);
	DynamicMeshComponent = NewObject<UDynamicMeshComponent>(PreviewMeshActor);
	InitializeSculptMeshComponent(DynamicMeshComponent, PreviewMeshActor);

	// assign materials
	FComponentMaterialSet MaterialSet = UE::ToolTarget::GetMaterialSet(Target);
	for (int k = 0; k < MaterialSet.Materials.Num(); ++k)
	{
		DynamicMeshComponent->SetMaterial(k, MaterialSet.Materials[k]);
	}

	DynamicMeshComponent->SetInvalidateProxyOnChangeEnabled(false);
	OnDynamicMeshComponentChangedHandle = DynamicMeshComponent->OnMeshVerticesChanged.AddUObject(this, &UDataflowEditorWeightMapPaintTool::OnDynamicMeshComponentChanged);

	FDynamicMesh3* Mesh = GetSculptMesh();
	Mesh->EnableVertexColors(FVector3f::One());
	Mesh->Attributes()->EnablePrimaryColors();
	Mesh->Attributes()->PrimaryColors()->CreatePerVertex(0.f);
	FAxisAlignedBox3d Bounds = Mesh->GetBounds(true);

	TFuture<void> PrecomputeFuture = Async(UE::Dataflow::Private::WeightPaintToolAsyncExecTarget, [&]()
	{
		PrecomputeFilterData();
	});

	TFuture<void> OctreeFuture = Async(UE::Dataflow::Private::WeightPaintToolAsyncExecTarget, [&]()
	{
		// initialize dynamic octree
		if (Mesh->TriangleCount() > 100000)
		{
			Octree.RootDimension = Bounds.MaxDim() / 10.0;
			Octree.SetMaxTreeDepth(4);
		}
		else
		{
			Octree.RootDimension = Bounds.MaxDim();
			Octree.SetMaxTreeDepth(8);
		}
		Octree.Initialize(Mesh);
	});

	// initialize render decomposition
	TUniquePtr<FMeshRenderDecomposition> Decomp = MakeUnique<FMeshRenderDecomposition>();
	FMeshRenderDecomposition::BuildChunkedDecomposition(Mesh, &MaterialSet, *Decomp);
	Decomp->BuildAssociations(Mesh);
	DynamicMeshComponent->SetExternalDecomposition(MoveTemp(Decomp));

	// initialize brush radius range interval, brush properties
	UMeshSculptToolBase::InitializeBrushSizeRange(Bounds);

	// Set up control points mechanic
	PolyLassoMechanic = NewObject<UPolyLassoMarqueeMechanic>(this);
	PolyLassoMechanic->Setup(this);
	PolyLassoMechanic->SetIsEnabled(false);
	PolyLassoMechanic->SpacingTolerance = 10.0f;
	PolyLassoMechanic->OnDrawPolyLassoFinished.AddUObject(this, &UDataflowEditorWeightMapPaintTool::OnPolyLassoFinished);


	// Set up vertex selection mechanic
	PolygonSelectionMechanic = NewObject<UPolygonSelectionMechanic>(this);
	PolygonSelectionMechanic->bAddSelectionFilterPropertiesToParentTool = false;
	PolygonSelectionMechanic->Setup(this);
	PolygonSelectionMechanic->SetIsEnabled(false);
	PolygonSelectionMechanic->OnSelectionChanged.AddUObject(this, &UDataflowEditorWeightMapPaintTool::OnSelectionModified);

	// disable CTRL to remove from selection
	PolygonSelectionMechanic->SetShouldRemoveFromSelectionFunc([]() { return false; });

	PolygonSelectionMechanic->Properties->bSelectEdges = false;
	PolygonSelectionMechanic->Properties->bSelectFaces = false;
	PolygonSelectionMechanic->Properties->bSelectVertices = true;
	
	constexpr bool bAutoBuild = true;
	GradientSelectionTopology = MakeUnique<UE::Geometry::FTriangleGroupTopology>(DynamicMeshComponent->GetMesh(), bAutoBuild);
	
	MeshSpatial = MakeUnique<UE::Geometry::FDynamicMeshAABBTree3>(DynamicMeshComponent->GetMesh(), bAutoBuild);
	PolygonSelectionMechanic->Initialize(DynamicMeshComponent, GradientSelectionTopology.Get(), [this]() { return MeshSpatial.Get(); });

	UpdateWeightMapProperties = NewObject<UDataflowEditorUpdateWeightMapProperties>(this);
	UpdateWeightMapProperties->Name = WeightMapNodeToUpdate->Name;

	UpdateWeightMapProperties->WatchProperty(WeightMapNodeToUpdate->Name, [this](const FString& NewName)
	{
		UpdateWeightMapProperties->Name = NewName;
	});
	AddToolPropertySource(UpdateWeightMapProperties);

	// initialize other properties
	FilterProperties = NewObject<UDataflowEditorWeightMapPaintBrushFilterProperties>(this);
	FilterProperties->WatchProperty(FilterProperties->SubToolType,
		[this](EDataflowEditorWeightMapPaintInteractionType NewType) { UpdateSubToolType(NewType); });
	FilterProperties->WatchProperty(FilterProperties->BrushSize,
		[this](float NewSize) { UMeshSculptToolBase::BrushProperties->BrushSize.AdaptiveSize = NewSize; });
	FilterProperties->BrushSize = UMeshSculptToolBase::BrushProperties->BrushSize.AdaptiveSize;
	FilterProperties->RestoreProperties(this);
	AddToolPropertySource(FilterProperties);

	InitializeIndicator();

	// initialize our properties
	AddToolPropertySource(UMeshSculptToolBase::BrushProperties);
	UMeshSculptToolBase::BrushProperties->bShowPerBrushProps = false;
	UMeshSculptToolBase::BrushProperties->bShowFalloff = true;
	UMeshSculptToolBase::BrushProperties->bShowLazyness = false;
	UMeshSculptToolBase::BrushProperties->FlowRate = 0.0f;
	CalculateBrushRadius();

	PaintBrushOpProperties = NewObject<UDataflowWeightMapPaintBrushOpProps>(this);
	RegisterBrushType((int32)EDataflowEditorWeightMapPaintBrushType::Paint, LOCTEXT("Paint", "Paint"),
		MakeUnique<FLambdaMeshSculptBrushOpFactory>([this]() { return MakeUnique<FDataflowWeightMapPaintBrushOp>(); }),
		PaintBrushOpProperties);

	SmoothBrushOpProperties = NewObject<UDataflowWeightMapSmoothBrushOpProps>(this);
	RegisterBrushType((int32)EDataflowEditorWeightMapPaintBrushType::Smooth, LOCTEXT("SmoothBrushType", "Smooth"),
		MakeUnique<FLambdaMeshSculptBrushOpFactory>([this]() { return MakeUnique<FDataflowWeightMapSmoothBrushOp>(); }),
		SmoothBrushOpProperties);

	// secondary brushes
	EraseBrushOpProperties = NewObject<UDataflowWeightMapEraseBrushOpProps>(this);

	RegisterSecondaryBrushType((int32)EDataflowEditorWeightMapPaintBrushType::Erase, LOCTEXT("Erase", "Erase"),
		MakeUnique<TBasicMeshSculptBrushOpFactory<FDataflowWeightMapEraseBrushOp>>(),
		EraseBrushOpProperties);

	AddToolPropertySource(UMeshSculptToolBase::ViewProperties);
	SetToolPropertySourceEnabled(UMeshSculptToolBase::ViewProperties, true);

	AddToolPropertySource(UMeshSculptToolBase::GizmoProperties);
	SetToolPropertySourceEnabled(UMeshSculptToolBase::GizmoProperties, false);


	// register watchers
	FilterProperties->WatchProperty( FilterProperties->PrimaryBrushType,
		[this](EDataflowEditorWeightMapPaintBrushType NewType) { UpdateBrushType(NewType); });

	// must call before updating brush type so that we register all brush properties?
	UMeshSculptToolBase::OnCompleteSetup();

	UpdateBrushType(FilterProperties->PrimaryBrushType);
	SetActiveSecondaryBrushType((int32)EDataflowEditorWeightMapPaintBrushType::Erase);
	
	ActionsProps = NewObject<UDataflowEditorMeshWeightMapPaintToolActions>(this);
	ActionsProps->Initialize(this);
	AddToolPropertySource(ActionsProps);

	MeshElementsDisplay = NewObject<UMeshElementsVisualizer>(this);
	MeshElementsDisplay->CreateInWorld(DynamicMeshComponent->GetWorld(), DynamicMeshComponent->GetComponentTransform());
	if (ensure(MeshElementsDisplay->Settings))
	{
		MeshElementsDisplay->Settings->bShowNormalSeams = false;
		MeshElementsDisplay->Settings->ThicknessScale = 0.5f;
		MeshElementsDisplay->Settings->BoundaryEdgeColor = FColor(164, 73, 164);
		MeshElementsDisplay->Settings->RestoreProperties(this, TEXT("DataflowEditorWeightMapPaintTool2"));
		AddToolPropertySource(MeshElementsDisplay->Settings);
	}
	MeshElementsDisplay->SetMeshAccessFunction([this](UMeshElementsVisualizer::ProcessDynamicMeshFunc ProcessFunc) {
		ProcessFunc(*GetSculptMesh());
	});

	// disable view properties
	SetViewPropertiesEnabled(false);
	UpdateMaterialMode(EMeshEditingMaterialModes::VertexColor);
	UpdateWireframeVisibility(false);
	UpdateFlatShadingSetting(false);

	// configure panels
	UpdateSubToolType(FilterProperties->SubToolType);

	// Setup DynamicMeshToWeight conversion
	if (DataflowEditorContextObject)
	{
		if (const TSharedPtr<const FManagedArrayCollection> Collection = DataflowEditorContextObject->GetRenderCollection())
		{
			using namespace UE::Dataflow;
			const FNonManifoldMappingSupport NonManifoldMapping(*Mesh);

			bHaveDynamicMeshToWeightConversion = NonManifoldMapping.IsNonManifoldVertexInSource();
			if (bHaveDynamicMeshToWeightConversion)
			{
				DynamicMeshToWeight.SetNumUninitialized(Mesh->VertexCount());
				WeightToDynamicMesh.Reset();
				WeightToDynamicMesh.SetNum(Collection->NumElements("Vertices"));
				for (int32 DynamicMeshVert = 0; DynamicMeshVert < Mesh->VertexCount(); ++DynamicMeshVert)
				{
					DynamicMeshToWeight[DynamicMeshVert] = NonManifoldMapping.GetOriginalNonManifoldVertexID(DynamicMeshVert);
					if (0 <= DynamicMeshToWeight[DynamicMeshVert] && DynamicMeshToWeight[DynamicMeshVert] < WeightToDynamicMesh.Num())
					{
						WeightToDynamicMesh[DynamicMeshToWeight[DynamicMeshVert]].Add(DynamicMeshVert);
					}
					else
					{
						bHaveDynamicMeshToWeightConversion = false;
						UE_LOGF(LogTemp, Warning, "Weight map misalignment.");
						break;
					}
				}
			}
		}
	}

	PrecomputeFuture.Wait();
	OctreeFuture.Wait();


	// Create an attribute layer to temporarily paint into
	const int NumAttributeLayers = Mesh->Attributes()->NumWeightLayers();
	Mesh->Attributes()->SetNumWeightLayers(NumAttributeLayers + 1);
	ActiveWeightMap = Mesh->Attributes()->GetWeightLayer(NumAttributeLayers);
	ActiveWeightMap->SetName(FName("PaintLayer"));
	
	// Setup DynamicMeshToWeight conversion and get Input weight map (if it exists)
	const int32 NumExpectedWeights = bHaveDynamicMeshToWeightConversion ? WeightToDynamicMesh.Num() : GetSculptMesh()->MaxVertexID();
	SetupWeights.SetNumZeroed(NumExpectedWeights);

	if(DataflowEditorContextObject && WeightMapNodeToUpdate)
	{
		// Find the map if it exists.
		if (TSharedPtr<UE::Dataflow::FEngineContext> DataflowContext = DataflowEditorContextObject->GetDataflowContext())
		{
			// Get the attribute key
			FCollectionAttributeKey AttributeKey = WeightMapNodeToUpdate->GetWeightAttributeKey(*DataflowContext);

			// Fill the attribute values
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			WeightMapNodeToUpdate->FillAttributeWeights(DataflowEditorContextObject->GetSelectedCollection(), AttributeKey, SetupWeights);
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
	}

	// Copy weights from selected node to the preview mesh
	TArray<float> CurrentWeights;
	CurrentWeights.SetNumZeroed(NumExpectedWeights);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	WeightMapNodeToUpdate->ExtractVertexWeights(SetupWeights, TArrayView<float>(CurrentWeights));
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	if (bHaveDynamicMeshToWeightConversion)
	{
		if (WeightToDynamicMesh.Num() == CurrentWeights.Num())	// Only copy node weights if they match the number of mesh vertices
		{
			for (int32 WeightID = 0; WeightID < CurrentWeights.Num(); ++WeightID)
			{
				for (const int32 VertexID : WeightToDynamicMesh[WeightID])
				{
					ActiveWeightMap->SetValue(VertexID, &CurrentWeights[WeightID]);
				}
			}
		}
	}
	else
	{
		if (Mesh->MaxVertexID() == CurrentWeights.Num())	// Only copy node weights if they match the number of mesh vertices
		{
			for (int32 VertexID = 0; VertexID < CurrentWeights.Num(); ++VertexID)
			{
				ActiveWeightMap->SetValue(VertexID, &CurrentWeights[VertexID]);
			}
		}
	}

	UpdateWeightMapProperties->Name = WeightMapNodeToUpdate->Name;
	SetToolPropertySourceEnabled(UpdateWeightMapProperties, true);

	// update colors
	UpdateVertexColorOverlay();
	DynamicMeshComponent->FastNotifyVertexAttributesUpdated(EMeshRenderAttributeFlags::VertexColors);
	GetToolManager()->PostInvalidation();

}


void UDataflowEditorWeightMapPaintTool::SetDataflowEditorContextObject(TObjectPtr<UDataflowContextObject> InDataflowEditorContextObject)
{
	DataflowEditorContextObject = InDataflowEditorContextObject;
}

void UDataflowEditorWeightMapPaintTool::Shutdown(EToolShutdownType ShutdownType)
{
	if (DynamicMeshComponent != nullptr)
	{
		DynamicMeshComponent->OnMeshChanged.Remove(OnDynamicMeshComponentChangedHandle);
	}

	if (MeshElementsDisplay)
	{
		if (ensure(MeshElementsDisplay->Settings))
		{
			MeshElementsDisplay->Settings->SaveProperties(this, TEXT("DataflowEditorWeightMapPaintTool2"));
		}
		MeshElementsDisplay->Disconnect();
	}

	if (FilterProperties)
	{
		FilterProperties->SaveProperties(this);
	}

	if (PreviewMeshActor != nullptr)
	{
		PreviewMeshActor->Destroy();
		PreviewMeshActor = nullptr;
	}

	if (PolygonSelectionMechanic)
	{
		PolygonSelectionMechanic->Shutdown();
		PolygonSelectionMechanic = nullptr;
	}

	if (WeightMapNodeToUpdate)
	{
		WeightMapNodeToUpdate->Invalidate();
	}

	UMeshSculptToolBase::Shutdown(ShutdownType);
}


void UDataflowEditorWeightMapPaintTool::CommitResult(UBaseDynamicMeshComponent* Component, bool bModifiedTopology)
{
	GetToolManager()->BeginUndoTransaction(LOCTEXT("WeightPaintToolTransactionName", "Paint Weights"));

	UpdateSelectedNode();

	GetToolManager()->EndUndoTransaction();
}


void UDataflowEditorWeightMapPaintTool::RegisterActions(FInteractiveToolActionSet& ActionSet)
{
	UMeshSculptToolBase::RegisterActions(ActionSet);
	
	ActionSet.RegisterAction(this, (int32)EStandardToolActions::BaseClientDefinedActionID + 500,
		TEXT("PickWeightValueUnderCursor"),
		LOCTEXT("PickWeightValueUnderCursor", "Pick Weight Value"),
		LOCTEXT("PickWeightValueUnderCursorTooltip", "Set the active weight painting value to that currently under the cursor"),
		EModifierKey::Shift, EKeys::G,
		[this]() { bPendingPickWeight = true; });
};


TUniquePtr<FMeshSculptBrushOp>& UDataflowEditorWeightMapPaintTool::GetActiveBrushOp()
{
	if (GetInEraseStroke())
	{
		return SecondaryBrushOp;
	}
	else
	{
		return PrimaryBrushOp;
	}
}


void UDataflowEditorWeightMapPaintTool::OnPropertyModified(UObject* PropertySet, FProperty* Property)
{
	CalculateBrushRadius();
}


void UDataflowEditorWeightMapPaintTool::IncreaseBrushRadiusAction()
{
	Super::IncreaseBrushRadiusAction();
	FilterProperties->BrushSize = BrushProperties->BrushSize.AdaptiveSize;
	NotifyOfPropertyChangeByTool(FilterProperties);
}

void UDataflowEditorWeightMapPaintTool::DecreaseBrushRadiusAction()
{
	Super::DecreaseBrushRadiusAction();
	FilterProperties->BrushSize = BrushProperties->BrushSize.AdaptiveSize;
	NotifyOfPropertyChangeByTool(FilterProperties);
}

void UDataflowEditorWeightMapPaintTool::IncreaseBrushRadiusSmallStepAction()
{
	Super::IncreaseBrushRadiusSmallStepAction();
	FilterProperties->BrushSize = BrushProperties->BrushSize.AdaptiveSize;
	NotifyOfPropertyChangeByTool(FilterProperties);
}

void UDataflowEditorWeightMapPaintTool::DecreaseBrushRadiusSmallStepAction()
{
	Super::DecreaseBrushRadiusSmallStepAction();
	FilterProperties->BrushSize = BrushProperties->BrushSize.AdaptiveSize;
	NotifyOfPropertyChangeByTool(FilterProperties);
}


bool UDataflowEditorWeightMapPaintTool::IsInBrushSubMode() const
{
	return FilterProperties->SubToolType == EDataflowEditorWeightMapPaintInteractionType::Brush
		|| FilterProperties->SubToolType == EDataflowEditorWeightMapPaintInteractionType::Fill;
}


void UDataflowEditorWeightMapPaintTool::OnBeginStroke(const FRay& WorldRay)
{
	if (!ActiveWeightMap)
	{
		return;
	}

	UpdateBrushPosition(WorldRay);

	if (PaintBrushOpProperties)
	{
		PaintBrushOpProperties->AttributeValue = FilterProperties->AttributeValue;
		PaintBrushOpProperties->Strength = FilterProperties->Strength * FilterProperties->Strength;
	}
	if (EraseBrushOpProperties)
	{
		EraseBrushOpProperties->AttributeValue = 0.0;
	}
	if (SmoothBrushOpProperties)
	{
		SmoothBrushOpProperties->Strength = FilterProperties->Strength * FilterProperties->Strength;
	}

	// initialize first "Last Stamp", so that we can assume all stamps in stroke have a valid previous stamp
	LastStamp.WorldFrame = GetBrushFrameWorld();
	LastStamp.LocalFrame = GetBrushFrameLocal();
	LastStamp.Radius = GetCurrentBrushRadius();
	LastStamp.Falloff = GetCurrentBrushFalloff();
	LastStamp.Direction = GetInInvertStroke() ? -1.0 : 1.0;
	LastStamp.Depth = GetCurrentBrushDepth();
	LastStamp.Power = GetActivePressure() * GetCurrentBrushStrength();
	LastStamp.TimeStamp = FDateTime::Now();

	FSculptBrushOptions SculptOptions;
	SculptOptions.ConstantReferencePlane = GetCurrentStrokeReferencePlane();

	TUniquePtr<FMeshSculptBrushOp>& UseBrushOp = GetActiveBrushOp();
	UseBrushOp->ConfigureOptions(SculptOptions);
	UseBrushOp->BeginStroke(GetSculptMesh(), LastStamp, VertexROI);

	AccumulatedTriangleROI.Reset();

	// begin change here? or wait for first stamp?
	BeginChange();
}

void UDataflowEditorWeightMapPaintTool::OnEndStroke()
{
	if (!ActiveWeightMap)
	{
		return;
	}

	GetActiveBrushOp()->EndStroke(GetSculptMesh(), LastStamp, VertexROI);

	UpdateVertexColorOverlay(&TriangleROI);
	DynamicMeshComponent->FastNotifyVertexAttributesUpdated(EMeshRenderAttributeFlags::VertexColors);

	// close change record
	EndChange();
}




void UDataflowEditorWeightMapPaintTool::UpdateROI(const FSculptBrushStamp& BrushStamp)
{
	SCOPE_CYCLE_COUNTER(WeightMapPaintTool_UpdateROI);

	const FVector3d& BrushPos = BrushStamp.LocalFrame.Origin;
	const FDynamicMesh3* Mesh = GetSculptMesh();
	float RadiusSqr = GetCurrentBrushRadius() * GetCurrentBrushRadius();
	FAxisAlignedBox3d BrushBox(
		BrushPos - GetCurrentBrushRadius() * FVector3d::One(),
		BrushPos + GetCurrentBrushRadius() * FVector3d::One());

	TriangleROI.Reset();

	int32 CenterTID = GetBrushTriangleID();
	if (Mesh->IsTriangle(CenterTID))
	{
		TriangleROI.Add(CenterTID);
	}

	FVector3d CenterNormal = Mesh->IsTriangle(CenterTID) ? TriNormals[CenterTID] : FVector3d::One();		// One so that normal check always passes

	bool bVolumetric = (FilterProperties->BrushAreaMode == EMeshVertexPaintBrushAreaType::Volumetric);
	bool bUseAngleThreshold = FilterProperties->AngleThreshold < 180.0f;
	double DotAngleThreshold = FMathd::Cos(FilterProperties->AngleThreshold * FMathd::DegToRad);
	bool bStopAtUVSeams = FilterProperties->bUVSeams;
	bool bStopAtNormalSeams = FilterProperties->bNormalSeams;

	auto CheckEdgeCriteria = [&](int32 t1, int32 t2) -> bool
	{
		if (bUseAngleThreshold == false || CenterNormal.Dot(TriNormals[t2]) > DotAngleThreshold)
		{
			int32 eid = Mesh->FindEdgeFromTriPair(t1, t2);
			if (bStopAtUVSeams == false || UVSeamEdges[eid] == false)
			{
				if (bStopAtNormalSeams == false || NormalSeamEdges[eid] == false)
				{
					return true;
				}
			}
		}
		return false;
	};
	bool bFill = (FilterProperties->SubToolType == EDataflowEditorWeightMapPaintInteractionType::Fill);
	
	if (bVolumetric)
	{
		Octree.RangeQuery(BrushBox,
			[&](int TriIdx) {
			if ((Mesh->GetTriCentroid(TriIdx) - BrushPos).SquaredLength() < RadiusSqr)
			{
				TriangleROI.Add(TriIdx);
			}
		});
	}
	else
	{
		if (Mesh->IsTriangle(CenterTID))
		{
			TArray<int32> StartROI;
			StartROI.Add(CenterTID);
			FMeshConnectedComponents::GrowToConnectedTriangles(Mesh, StartROI, TriangleROI, &TempROIBuffer,
				[&](int t1, int t2) 
			{ 
				if ((Mesh->GetTriCentroid(t2) - BrushPos).SquaredLength() < RadiusSqr)
				{
					return CheckEdgeCriteria(t1, t2);
				}
				return false;
			});
		}
	}
	if (bFill)
	{
		TArray<int32> StartROI;
		for (int32 tid : TriangleROI)
		{
			StartROI.Add(tid);
		}
		FMeshConnectedComponents::GrowToConnectedTriangles(Mesh, StartROI, TriangleROI, &TempROIBuffer,
														   [&](int t1, int t2)
		{
			return CheckEdgeCriteria(t1, t2);
		});
	}

	// construct ROI vertex set
	VertexSetBuffer.Reset();
	for (int32 tid : TriangleROI)
	{
		FIndex3i Tri = Mesh->GetTriangle(tid);
		VertexSetBuffer.Add(Tri.A);  VertexSetBuffer.Add(Tri.B);  VertexSetBuffer.Add(Tri.C);
	}
	
	// apply visibility filter
	if (FilterProperties->VisibilityFilter != EDataflowEditorWeightMapPaintVisibilityType::None)
	{
		TArray<int32> ResultBuffer;
		ApplyVisibilityFilter(VertexSetBuffer, TempROIBuffer, ResultBuffer);
	}

	VertexROI.SetNum(0, EAllowShrinking::No);
	//TODO: If we paint a 2D projection of UVs, these will need to be the 2D vertices not the 3D original mesh vertices
	BufferUtil::AppendElements(VertexROI, VertexSetBuffer);

	// construct ROI triangle and weight buffers
	ROITriangleBuffer.Reserve(TriangleROI.Num());
	ROITriangleBuffer.SetNum(0, EAllowShrinking::No);
	for (int32 tid : TriangleROI)
	{
		ROITriangleBuffer.Add(tid);
	}
	ROIWeightValueBuffer.SetNum(VertexROI.Num(), EAllowShrinking::No);
	SyncWeightBufferWithMesh(Mesh);
}

bool UDataflowEditorWeightMapPaintTool::UpdateStampPosition(const FRay& WorldRay)
{
	CalculateBrushRadius();

	TUniquePtr<FMeshSculptBrushOp>& UseBrushOp = GetActiveBrushOp();

	ESculptBrushOpTargetType TargetType = UseBrushOp->GetBrushTargetType();
	switch (TargetType)
	{
	case ESculptBrushOpTargetType::SculptMesh:
	case ESculptBrushOpTargetType::TargetMesh:
		UpdateBrushPositionOnSculptMesh(WorldRay, true);
		break;
	case ESculptBrushOpTargetType::ActivePlane:
		check(false);
		UpdateBrushPositionOnActivePlane(WorldRay);
		break;
	}

	const bool bVolumetric = (FilterProperties->BrushAreaMode == EMeshVertexPaintBrushAreaType::Volumetric);
	if (UseBrushOp->GetAlignStampToView() || bVolumetric)
	{
		AlignBrushToView();
	}

	CurrentStamp = LastStamp;
	CurrentStamp.DeltaTime = FMathd::Min((FDateTime::Now() - LastStamp.TimeStamp).GetTotalSeconds(), 1.0);
	CurrentStamp.WorldFrame = GetBrushFrameWorld();
	CurrentStamp.LocalFrame = GetBrushFrameLocal();
	CurrentStamp.Power = GetActivePressure() * GetCurrentBrushStrength();

	CurrentStamp.PrevLocalFrame = LastStamp.LocalFrame;
	CurrentStamp.PrevWorldFrame = LastStamp.WorldFrame;

	FVector3d MoveDelta = CurrentStamp.LocalFrame.Origin - CurrentStamp.PrevLocalFrame.Origin;

	if (UseBrushOp->IgnoreZeroMovements() && MoveDelta.SquaredLength() < 0.1 * CurrentBrushRadius)
	{
		return false;
	}

	return true;
}


bool UDataflowEditorWeightMapPaintTool::ApplyStamp()
{
	SCOPE_CYCLE_COUNTER(WeightMapPaintToolApplyStamp);

	TUniquePtr<FMeshSculptBrushOp>& UseBrushOp = GetActiveBrushOp();

	// yuck
	FDataflowMeshVertexWeightMapEditBrushOp* WeightBrushOp = (FDataflowMeshVertexWeightMapEditBrushOp*)UseBrushOp.Get();

	if (FilterProperties->SubToolType == EDataflowEditorWeightMapPaintInteractionType::Brush)
	{
		WeightBrushOp->bApplyRadiusLimit = true;
	}
	else
	{ 
		WeightBrushOp->bApplyRadiusLimit = false;
	}

	FDynamicMesh3* Mesh = GetSculptMesh();
	WeightBrushOp->ApplyStampByVertices(Mesh, CurrentStamp, VertexROI, ROIWeightValueBuffer);

	bool bUpdated = SyncMeshWithWeightBuffer(Mesh);

	LastStamp = CurrentStamp;
	LastStamp.TimeStamp = FDateTime::Now();

	return bUpdated;
}




bool UDataflowEditorWeightMapPaintTool::SyncMeshWithWeightBuffer(FDynamicMesh3* Mesh)
{
	int NumModified = 0;
	const int32 NumT = VertexROI.Num();
	if (ActiveWeightMap)
	{
		// change update could be async here if we collected array of <idx,orig,new> and dispatched independenlty
		for (int32 k = 0; k < NumT; ++k)
		{
			int VertIdx = VertexROI[k];
			double CurWeight = GetCurrentWeightValue(VertIdx);

			if (ROIWeightValueBuffer[k] != CurWeight)
			{
				if (bHaveDynamicMeshToWeightConversion)
				{
					for (const int32 Idx : WeightToDynamicMesh[DynamicMeshToWeight[VertIdx]])
					{
						ActiveWeightEditChangeTracker->SaveVertexOneRingTriangles(Idx, true);
						ActiveWeightMap->SetValue(Idx, &ROIWeightValueBuffer[k]);
					}
				}
				else
				{
				ActiveWeightEditChangeTracker->SaveVertexOneRingTriangles(VertIdx, true);
				ActiveWeightMap->SetValue(VertIdx, &ROIWeightValueBuffer[k]);
				}
				NumModified++;
			}
		}
	}
	return (NumModified > 0);
}

bool UDataflowEditorWeightMapPaintTool::SyncWeightBufferWithMesh(const FDynamicMesh3* Mesh)
{
	int NumModified = 0;
	const int32 NumT = VertexROI.Num();
	if (ActiveWeightMap)
	{
		// change update could be async here if we collected array of <idx,orig,new> and dispatched independenlty
		for (int32 k = 0; k < NumT; ++k)
		{
			int VertIdx = VertexROI[k];
			double CurWeight = GetCurrentWeightValue(VertIdx);
			if (ROIWeightValueBuffer[k] != CurWeight)
			{
				ROIWeightValueBuffer[k] = CurWeight;
				NumModified++;
			}
		}
	}
	return (NumModified > 0);
}

namespace UE::Dataflow
{
	template<typename RealType>
	static bool FindPolylineSelfIntersection(
		const TArray<UE::Math::TVector2<RealType>>& Polyline,
		UE::Math::TVector2<RealType>& IntersectionPointOut,
		FIndex2i& IntersectionIndexOut,
		bool bParallel = true)
	{
		int32 N = Polyline.Num();
		std::atomic<bool> bSelfIntersects(false);
		ParallelFor(N - 1, [&](int32 i)
			{
				TSegment2<RealType> SegA(Polyline[i], Polyline[i + 1]);
		for (int32 j = i + 2; j < N - 1 && bSelfIntersects == false; ++j)
		{
			TSegment2<RealType> SegB(Polyline[j], Polyline[j + 1]);
			if (SegA.Intersects(SegB) && bSelfIntersects == false)
			{
				bool ExpectedValue = false;
				if (std::atomic_compare_exchange_strong(&bSelfIntersects, &ExpectedValue, true))
				{
					TIntrSegment2Segment2<RealType> Intersection(SegA, SegB);
					Intersection.Find();
					IntersectionPointOut = Intersection.Point0;
					IntersectionIndexOut = FIndex2i(i, j);
					return;
				}
			}
		}
			}, (bParallel) ? EParallelForFlags::None : EParallelForFlags::ForceSingleThread);

		return bSelfIntersects;
	}



	template<typename RealType>
	static bool FindPolylineSegmentIntersection(
		const TArray<UE::Math::TVector2<RealType>>& Polyline,
		const TSegment2<RealType>& Segment,
		UE::Math::TVector2<RealType>& IntersectionPointOut,
		int& IntersectionIndexOut)
	{

		int32 N = Polyline.Num();
		for (int32 i = 0; i < N - 1; ++i)
		{
			TSegment2<RealType> PolySeg(Polyline[i], Polyline[i + 1]);
			if (Segment.Intersects(PolySeg))
			{
				TIntrSegment2Segment2<RealType> Intersection(Segment, PolySeg);
				Intersection.Find();
				IntersectionPointOut = Intersection.Point0;
				IntersectionIndexOut = i;
				return true;
			}
		}
		return false;
	}


	bool ApproxSelfClipPolyline(TArray<FVector2f>& Polyline)
	{
		int32 N = Polyline.Num();

		// handle already-closed polylines
		if (Distance(Polyline[0], Polyline[N - 1]) < 0.0001f)
		{
			return true;
		}

		FVector2f IntersectPoint;
		FIndex2i IntersectionIndex(-1, -1);
		bool bSelfIntersects = FindPolylineSelfIntersection(Polyline, IntersectPoint, IntersectionIndex);
		if (bSelfIntersects)
		{
			TArray<FVector2f> NewPolyline;
			NewPolyline.Add(IntersectPoint);
			for (int32 i = IntersectionIndex.A; i <= IntersectionIndex.B; ++i)
			{
				NewPolyline.Add(Polyline[i]);
			}
			NewPolyline.Add(IntersectPoint);
			Polyline = MoveTemp(NewPolyline);
			return true;
		}


		FVector2f StartDirOut = UE::Geometry::Normalized(Polyline[0] - Polyline[1]);
		FLine2f StartLine(Polyline[0], StartDirOut);
		FVector2f EndDirOut = UE::Geometry::Normalized(Polyline[N - 1] - Polyline[N - 2]);
		FLine2f EndLine(Polyline[N - 1], EndDirOut);
		FIntrLine2Line2f LineIntr(StartLine, EndLine);
		bool bIntersects = false;
		if (LineIntr.Find())
		{
			bIntersects = LineIntr.IsSimpleIntersection() && (LineIntr.Segment1Parameter > 0) && (LineIntr.Segment2Parameter > 0);
			if (bIntersects)
			{
				Polyline.Add(StartLine.PointAt(LineIntr.Segment1Parameter));
				Polyline.Add(StartLine.Origin);
				return true;
			}
		}


		FAxisAlignedBox2f Bounds;
		for (const FVector2f& P : Polyline)
		{
			Bounds.Contain(P);
		}
		float Size = Bounds.DiagonalLength();

		FVector2f StartPos = Polyline[0] + 0.001f * StartDirOut;
		if (FindPolylineSegmentIntersection(Polyline, FSegment2f(StartPos, StartPos + 2 * Size * StartDirOut), IntersectPoint, IntersectionIndex.A))
		{
			return true;
		}

		FVector2f EndPos = Polyline[N - 1] + 0.001f * EndDirOut;
		if (FindPolylineSegmentIntersection(Polyline, FSegment2f(EndPos, EndPos + 2 * Size * EndDirOut), IntersectPoint, IntersectionIndex.A))
		{
			return true;
		}

		return false;
	}
}


void UDataflowEditorWeightMapPaintTool::OnPolyLassoFinished(const FCameraPolyLasso& Lasso, bool bCanceled)
{
	// construct polyline
	TArray<FVector2f> Polyline;
	for (FVector2D Pos : Lasso.Polyline)
	{
		Polyline.Add((FVector2f)Pos);
	}
	int32 N = Polyline.Num();
	if (N < 2)
	{
		return;
	}

	// Try to clip polyline to be closed, or closed-enough for winding evaluation to work.
	// If that returns false, the polyline is "too open". In that case we will extend
	// outwards from the endpoints and then try to create a closed very large polygon
	if (UE::Dataflow::ApproxSelfClipPolyline(Polyline) == false)
	{
		FVector2f StartDirOut = UE::Geometry::Normalized(Polyline[0] - Polyline[1]);
		FLine2f StartLine(Polyline[0], StartDirOut);
		FVector2f EndDirOut = UE::Geometry::Normalized(Polyline[N-1] - Polyline[N-2]);
		FLine2f EndLine(Polyline[N-1], EndDirOut);

		// if we did not intersect, we are in ambiguous territory. Check if a segment along either end-direction
		// intersects the polyline. If it does, we have something like a spiral and will be OK. 
		// If not, make a closed polygon by interpolating outwards from each endpoint, and then in perp-directions.
		FPolygon2f Polygon(Polyline);
		float PerpSign = Polygon.IsClockwise() ? -1.0 : 1.0;

		Polyline.Insert(StartLine.PointAt(10000.0f), 0);
		Polyline.Insert(Polyline[0] + 1000 * PerpSign * UE::Geometry::PerpCW(StartDirOut), 0);

		Polyline.Add(EndLine.PointAt(10000.0f));
		Polyline.Add(Polyline.Last() + 1000 * PerpSign * UE::Geometry::PerpCW(EndDirOut));
		FVector2f StartPos = Polyline[0];
		Polyline.Add(StartPos);		// close polyline (cannot use Polyline[0] in case Add resizes!)
	}

	N = Polyline.Num();

	// project each mesh vertex to view plane and evaluate winding integral of polyline
	const FDynamicMesh3* Mesh = GetSculptMesh();
	TempROIBuffer.SetNum(Mesh->MaxVertexID());
	ParallelFor(Mesh->MaxVertexID(), [&](int32 VertexIdx)
	{
		if (Mesh->IsVertex(VertexIdx))
		{
			FVector3d WorldPos = CurTargetTransform.TransformPosition(Mesh->GetVertex(VertexIdx));
			FVector2f PlanePos = (FVector2f)Lasso.GetProjectedPoint((FVector)WorldPos);

			double WindingSum = 0;
			FVector2f a = Polyline[0] - PlanePos, b = FVector2f::Zero();
			for (int32 i = 1; i < N; ++i)
			{
				b = Polyline[i] - PlanePos;
				WindingSum += (double)FMathf::Atan2(a.X*b.Y - a.Y*b.X, a.X*b.X + a.Y*b.Y);
				a = b;
			}
			WindingSum /= FMathd::TwoPi;
			bool bInside = FMathd::Abs(WindingSum) > 0.3;
			TempROIBuffer[VertexIdx] = bInside ? 1 : 0;
		}
		else
		{
			TempROIBuffer[VertexIdx] = -1;
		}
	});

	// convert to vertex selection, and then select fully-enclosed faces
	FMeshVertexSelection VertexSelection(Mesh);
	VertexSelection.SelectByVertexID([&](int32 VertexIdx) { return TempROIBuffer[VertexIdx] == 1; });

	double SetWeightValue = GetInEraseStroke() ? 0.0 : FilterProperties->AttributeValue;
	SetVerticesToWeightMap(VertexSelection.AsSet(), SetWeightValue, GetInEraseStroke());
}


void UDataflowEditorWeightMapPaintTool::ComputeGradient()
{
	if (!ensure(ActiveWeightMap))
	{
		UE_LOGF(LogTemp, Warning, "No active weight map");
		return;
	}

	BeginChange();

	const FDynamicMesh3* const Mesh = DynamicMeshComponent->GetMesh();
	TempROIBuffer.SetNum(0, EAllowShrinking::No);
	for (int32 VertexIdx : Mesh->VertexIndicesItr())
	{
		TempROIBuffer.Add(VertexIdx);
	}

	if (bHaveDynamicMeshToWeightConversion)
	{
		for (int32 VertexIdx : TempROIBuffer)
		{
			for (const int32 Idx : WeightToDynamicMesh[DynamicMeshToWeight[VertexIdx]])
			{
				ActiveWeightEditChangeTracker->SaveVertexOneRingTriangles(Idx, true);
			}
		}
	}
	else
	{
		for (int32 VertexIdx : TempROIBuffer)
		{
			ActiveWeightEditChangeTracker->SaveVertexOneRingTriangles(VertexIdx, true);
		}
	}


	for (const int32 VertexIndex : TempROIBuffer)
	{
		const FVector3d Vert = Mesh->GetVertex(VertexIndex);

		// (Copied from FClothPaintTool_Gradient::ApplyGradient)

		// Get distances
		// TODO: Look into surface distance instead of 3D distance? May be necessary for some complex shapes
		float DistanceToLowSq = MAX_flt;
		for (const int32& LowIndex : LowValueGradientVertexSelection.SelectedCornerIDs)
		{
			const FVector3d LowPoint = Mesh->GetVertex(LowIndex);
			const float DistanceSq = (LowPoint - Vert).SizeSquared();
			if (DistanceSq < DistanceToLowSq)
			{
				DistanceToLowSq = DistanceSq;
			}
		}

		float DistanceToHighSq = MAX_flt;
		for (const int32& HighIndex : HighValueGradientVertexSelection.SelectedCornerIDs)
		{
			const FVector3d HighPoint = Mesh->GetVertex(HighIndex);
			const float DistanceSq = (HighPoint - Vert).SizeSquared();
			if (DistanceSq < DistanceToHighSq)
			{
				DistanceToHighSq = DistanceSq;
			}
		}

		const float Value = FMath::LerpStable(FilterProperties->GradientLowValue, FilterProperties->GradientHighValue, DistanceToLowSq / (DistanceToLowSq + DistanceToHighSq));
		if (bHaveDynamicMeshToWeightConversion)
		{
			for (const int32 Idx : WeightToDynamicMesh[DynamicMeshToWeight[VertexIndex]])
			{
				ActiveWeightMap->SetValue(Idx, &Value);
			}
		}
		else
		{
		ActiveWeightMap->SetValue(VertexIndex, &Value);
	}
	}

	// update colors
	UpdateVertexColorOverlay();
	DynamicMeshComponent->FastNotifyVertexAttributesUpdated(EMeshRenderAttributeFlags::VertexColors);
	GetToolManager()->PostInvalidation();
	EndChange();

}


void UDataflowEditorWeightMapPaintTool::OnSelectionModified()
{
	const bool bToolTypeIsGradient = (FilterProperties->SubToolType == EDataflowEditorWeightMapPaintInteractionType::Gradient);
	if (bToolTypeIsGradient && PolygonSelectionMechanic)
	{
		const FGroupTopologySelection NewSelection = PolygonSelectionMechanic->GetActiveSelection();

		const bool bSelectingLowValueGradientVertices = GetCtrlToggle();
		if (bSelectingLowValueGradientVertices)
		{
			LowValueGradientVertexSelection = NewSelection;
		}
		else
		{
			HighValueGradientVertexSelection = NewSelection;
		}

		if (LowValueGradientVertexSelection.SelectedCornerIDs.Num() > 0 && HighValueGradientVertexSelection.SelectedCornerIDs.Num() > 0)
		{
			ComputeGradient();
		}

		constexpr bool bBroadcast = false;
		PolygonSelectionMechanic->SetSelection(FGroupTopologySelection(), bBroadcast);
	}
}


void UDataflowEditorWeightMapPaintTool::SetVerticesToWeightMap(const TSet<int32>& Vertices, double WeightValue, bool bIsErase)
{
	BeginChange();

	TempROIBuffer.SetNum(0, EAllowShrinking::No);
	for (int32 VertexIdx : Vertices)
	{
		TempROIBuffer.Add(VertexIdx);
	}

	if (HaveVisibilityFilter())
	{
		TArray<int32> VisibleVertices;
		VisibleVertices.Reserve(TempROIBuffer.Num());
		ApplyVisibilityFilter(TempROIBuffer, VisibleVertices);
		TempROIBuffer = MoveTemp(VisibleVertices);
	}

	if (bHaveDynamicMeshToWeightConversion)
	{
		for (int32 VertexIdx : TempROIBuffer)
		{
			for (const int32 Idx : WeightToDynamicMesh[DynamicMeshToWeight[VertexIdx]])
			{
				ActiveWeightEditChangeTracker->SaveVertexOneRingTriangles(Idx, true);
				ActiveWeightMap->SetValue(Idx, &WeightValue);
			}
		}
	}
	else
	{
		for (int32 VertexIdx : TempROIBuffer)
		{
			ActiveWeightEditChangeTracker->SaveVertexOneRingTriangles(VertexIdx, true);
		}
		for (int32 VertexIdx : TempROIBuffer)
		{		
			ActiveWeightMap->SetValue(VertexIdx, &WeightValue);
		}
	}

	// update colors
	UpdateVertexColorOverlay();
	DynamicMeshComponent->FastNotifyVertexAttributesUpdated(EMeshRenderAttributeFlags::VertexColors);
	GetToolManager()->PostInvalidation();
	

	EndChange();
	
}



bool UDataflowEditorWeightMapPaintTool::HaveVisibilityFilter() const
{
	return FilterProperties->VisibilityFilter != EDataflowEditorWeightMapPaintVisibilityType::None;
}


void UDataflowEditorWeightMapPaintTool::ApplyVisibilityFilter(TSet<int32>& Vertices, TArray<int32>& ROIBuffer, TArray<int32>& OutputBuffer)
{
	ROIBuffer.SetNum(0, EAllowShrinking::No);
	ROIBuffer.Reserve(Vertices.Num());
	for (int32 VertexIdx : Vertices)
	{
		ROIBuffer.Add(VertexIdx);
	}
	
	OutputBuffer.Reset();
	ApplyVisibilityFilter(TempROIBuffer, OutputBuffer);

	Vertices.Reset();
	for (int32 VertexIdx : OutputBuffer)
	{
		Vertices.Add(VertexIdx);
	}
}

void UDataflowEditorWeightMapPaintTool::ApplyVisibilityFilter(const TArray<int32>& Vertices, TArray<int32>& VisibleVertices)
{
	if (!HaveVisibilityFilter())
	{
		VisibleVertices = Vertices;
		return;
	}

	FViewCameraState StateOut;
	GetToolManager()->GetContextQueriesAPI()->GetCurrentViewState(StateOut);
	FTransform3d LocalToWorld = UE::ToolTarget::GetLocalToWorldTransform(Target);
	FVector3d LocalEyePosition(LocalToWorld.InverseTransformPosition(StateOut.Position));

	const FDynamicMesh3* Mesh = GetSculptMesh();

	int32 NumVertices = Vertices.Num();

	VisibilityFilterBuffer.SetNum(NumVertices, EAllowShrinking::No);
	ParallelFor(NumVertices, [&](int32 idx)
	{
		VisibilityFilterBuffer[idx] = true;
		UE::Geometry::FVertexInfo VertexInfo;
		Mesh->GetVertex(Vertices[idx], VertexInfo, true, false, false);
		FVector3d Centroid = VertexInfo.Position;
		FVector3d FaceNormal = (FVector3d)VertexInfo.Normal;
		if (FaceNormal.Dot((Centroid - LocalEyePosition)) > 0)
		{
			VisibilityFilterBuffer[idx] = false;
		}
		if (FilterProperties->VisibilityFilter == EDataflowEditorWeightMapPaintVisibilityType::Unoccluded)
		{
			int32 HitTID = Octree.FindNearestHitObject(FRay3d(LocalEyePosition, UE::Geometry::Normalized(Centroid - LocalEyePosition)));
			if (HitTID != IndexConstants::InvalidID && Mesh->IsTriangle(HitTID))
			{
				// Check to see if our vertex has been occulded by another triangle.
				FIndex3i TriVertices = Mesh->GetTriangle(HitTID);
				if (TriVertices[0] != Vertices[idx] && TriVertices[1] != Vertices[idx] && TriVertices[2] != Vertices[idx])
				{
					VisibilityFilterBuffer[idx] = false;
				}
			}
		}
	});

	VisibleVertices.Reset();
	for (int32 k = 0; k < NumVertices; ++k)
	{
		if (VisibilityFilterBuffer[k])
		{
			VisibleVertices.Add(Vertices[k]);
		}
	}
}



int32 UDataflowEditorWeightMapPaintTool::FindHitSculptMeshTriangleConst(const FRay3d& LocalRay) const
{
	if (!IsInBrushSubMode())
	{
		return IndexConstants::InvalidID;
	}

	if (GetBrushCanHitBackFaces())
	{
		return Octree.FindNearestHitObject(LocalRay);
	}
	else
	{
		const FDynamicMesh3* Mesh = GetSculptMesh();

		FViewCameraState StateOut;
		GetToolManager()->GetContextQueriesAPI()->GetCurrentViewState(StateOut);
		FVector3d LocalEyePosition(CurTargetTransform.InverseTransformPosition((FVector3d)StateOut.Position));
		int HitTID = Octree.FindNearestHitObject(LocalRay,
			[this, Mesh, &LocalEyePosition](int TriangleID) {
			FVector3d Normal, Centroid;
			double Area;
			Mesh->GetTriInfo(TriangleID, Normal, Area, Centroid);
			return Normal.Dot((Centroid - LocalEyePosition)) < 0;
		});
		return HitTID;
	}
}

int32 UDataflowEditorWeightMapPaintTool::FindHitTargetMeshTriangleConst(const FRay3d& LocalRay) const
{
	check(false);
	return IndexConstants::InvalidID;
}

void UDataflowEditorWeightMapPaintTool::UpdateHitSculptMeshTriangle(int32 TriangleID, const FRay3d& LocalRay)
{
	// TODO: Figure out what the actual position on the triangle is when hit.
	CurrentBaryCentricCoords = FVector3d(1.0 / 3.0, 1.0 / 3.0, 1.0 / 3.0);
}

bool UDataflowEditorWeightMapPaintTool::UpdateBrushPosition(const FRay& WorldRay)
{
	TUniquePtr<FMeshSculptBrushOp>& UseBrushOp = GetActiveBrushOp();

	bool bHit = false; 
	ESculptBrushOpTargetType TargetType = UseBrushOp->GetBrushTargetType();
	switch (TargetType)
	{
	case ESculptBrushOpTargetType::SculptMesh:
	case ESculptBrushOpTargetType::TargetMesh:
		bHit = UpdateBrushPositionOnSculptMesh(WorldRay, false);  
		break;
	case ESculptBrushOpTargetType::ActivePlane:
		check(false);
		bHit = UpdateBrushPositionOnSculptMesh(WorldRay, false);
		break;
	}

	const bool bVolumetric = (FilterProperties->BrushAreaMode == EMeshVertexPaintBrushAreaType::Volumetric);
	if (bHit && (UseBrushOp->GetAlignStampToView() || bVolumetric))
	{
		AlignBrushToView();
	}

	return bHit;
}




bool UDataflowEditorWeightMapPaintTool::OnUpdateHover(const FInputDeviceRay& DevicePos)
{
	PendingStampType = FilterProperties->PrimaryBrushType;

	if(ensure(InStroke() == false))
	{
		UpdateBrushPosition(DevicePos.WorldRay);
	}
	return true;
}


void UDataflowEditorWeightMapPaintTool::DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI)
{
	if (PolyLassoMechanic)
	{
		// because the actual Weight change is deferred until mouse release, color the lasso to let the user know whether it will erase
		PolyLassoMechanic->LineColor = GetInEraseStroke() ? FLinearColor::Red : FLinearColor::Green;
		PolyLassoMechanic->DrawHUD(Canvas, RenderAPI);
	}

	if (PolygonSelectionMechanic)
	{
		PolygonSelectionMechanic->DrawHUD(Canvas, RenderAPI);
	}

	if (Canvas && RenderAPI && BrushIndicator)
	{
		// display the value under the brush
		const float BrushRadius = BrushIndicator->BrushRadius;
		const FVector Offset = (-RenderAPI->GetSceneView()->GetViewUp() * BrushRadius * 1.1f);
		const FVector BrushPos = BrushIndicator->BrushPosition + Offset;

		FVector4 ScreenBrushPos = RenderAPI->GetSceneView()->WorldToScreen(BrushPos);
		FVector2D ScreenBrushPos2D = FVector2D(ScreenBrushPos.X, ScreenBrushPos.Y);

		RenderAPI->GetSceneView()->ScreenToPixel(ScreenBrushPos, ScreenBrushPos2D);

		const bool bShouldShowValueAtBrush = IsInBrushSubMode();
		if (bShouldShowValueAtBrush)
		{
			const FText BaseText = LOCTEXT("DataflowWeightMapPaintTool_ValueAtBrush_TextFormat", "Value: {0}");
			const FText ValueAtBrushText = FText::Format(BaseText, FText::AsNumber(FilterProperties->ValueAtBrush));
			FCanvasTextItem TextItem(ScreenBrushPos2D, ValueAtBrushText, GEngine->GetLargeFont(), FLinearColor::White);
			TextItem.EnableShadow(FLinearColor::Black);
			TextItem.bCentreX = true;
			Canvas->DrawItem(TextItem);
		}
	}
}

void UDataflowEditorWeightMapPaintTool::Render(IToolsContextRenderAPI* RenderAPI)
{
	UMeshSculptToolBase::Render(RenderAPI);

	if (PolygonSelectionMechanic)
	{
		PolygonSelectionMechanic->RenderMarquee(RenderAPI);

		const FViewCameraState RenderCameraState = RenderAPI->GetCameraState();
		GradientSelectionRenderer.BeginFrame(RenderAPI, RenderCameraState);

		const FTransform Transform = DynamicMeshComponent->GetComponentTransform();
		GradientSelectionRenderer.SetTransform(Transform);
			
		GradientSelectionRenderer.SetPointParameters(FLinearColor::Red, 1.0);
		PolygonSelectionMechanic->UMeshTopologySelectionMechanic::GetTopologySelector()->DrawSelection(LowValueGradientVertexSelection, &GradientSelectionRenderer, &RenderCameraState);

		GradientSelectionRenderer.SetPointParameters(FLinearColor::Green, 1.0);
		PolygonSelectionMechanic->UMeshTopologySelectionMechanic::GetTopologySelector()->DrawSelection(HighValueGradientVertexSelection, &GradientSelectionRenderer, &RenderCameraState);

		// Now the current unsaved selection
		if (GetCtrlToggle())
		{
			GradientSelectionRenderer.SetPointParameters(FLinearColor::Red, 1.0);
		}
		else
		{
			GradientSelectionRenderer.SetPointParameters(FLinearColor::Green, 1.0);
		}

		PolygonSelectionMechanic->UMeshTopologySelectionMechanic::GetTopologySelector()->DrawSelection(PolygonSelectionMechanic->GetActiveSelection(), &GradientSelectionRenderer, &RenderCameraState);

		GradientSelectionRenderer.EndFrame();
	}
}


void UDataflowEditorWeightMapPaintTool::UpdateMaterialMode(EMeshEditingMaterialModes MaterialMode)
{
	if (MaterialMode == EMeshEditingMaterialModes::VertexColor)
	{
		constexpr bool bUseTwoSidedMaterial = true;
		ActiveOverrideMaterial = ToolSetupUtil::GetVertexColorMaterial(GetToolManager(), bUseTwoSidedMaterial);
		if (ensure(ActiveOverrideMaterial != nullptr))
		{
			GetSculptMeshComponent()->SetOverrideRenderMaterial(ActiveOverrideMaterial);
			ActiveOverrideMaterial->SetScalarParameterValue(TEXT("FlatShading"), (ViewProperties->bFlatShading) ? 1.0f : 0.0f);
		}
		GetSculptMeshComponent()->SetShadowsEnabled(false);
	}
	else
	{
		UMeshSculptToolBase::UpdateMaterialMode(MaterialMode);
	}
}

void UDataflowEditorWeightMapPaintTool::OnTick(float DeltaTime)
{
	UMeshSculptToolBase::OnTick(DeltaTime);
	MeshElementsDisplay->OnTick(DeltaTime);

	const bool bIsLasso = (FilterProperties->SubToolType == EDataflowEditorWeightMapPaintInteractionType::PolyLasso);
	PolyLassoMechanic->SetIsEnabled(bIsLasso);

	const bool bIsGradient = (FilterProperties->SubToolType == EDataflowEditorWeightMapPaintInteractionType::Gradient);
	PolygonSelectionMechanic->SetIsEnabled(bIsGradient);

	check(!(bIsLasso && bIsGradient));

	ConfigureIndicator(FilterProperties->BrushAreaMode == EMeshVertexPaintBrushAreaType::Volumetric);
	SetIndicatorVisibility(!bIsLasso && !bIsGradient);

	if (bHavePendingAction)
	{
		ApplyAction(PendingAction);
		bHavePendingAction = false;
		PendingAction = EDataflowEditorWeightMapPaintToolActions::NoAction;
	}

	SCOPE_CYCLE_COUNTER(WeightMapPaintToolTick);

	// process the undo update
	if (bUndoUpdatePending)
	{
		// wait for updates
		WaitForPendingUndoRedoUpdate();

		// post rendering update
		DynamicMeshComponent->FastNotifyTriangleVerticesUpdated(AccumulatedTriangleROI, EMeshRenderAttributeFlags::VertexColors);
		GetToolManager()->PostInvalidation();

		// ignore stamp and wait for next tick to do anything else
		bUndoUpdatePending = false;
		return;
	}

	// Get value at brush location
	const bool bShouldPickWeight = bPendingPickWeight && IsStampPending() == false;
	const bool bShouldUpdateValueAtBrush = IsInBrushSubMode();

	if (bShouldPickWeight || bShouldUpdateValueAtBrush)
	{
		if (GetSculptMesh()->IsTriangle(GetBrushTriangleID()))
		{
			const double HitWeightValue = GetCurrentWeightValueUnderBrush();

			if (bShouldPickWeight)
			{
				FilterProperties->AttributeValue = HitWeightValue;
				NotifyOfPropertyChangeByTool(FilterProperties);
			}

			if (bShouldUpdateValueAtBrush)
			{
				FilterProperties->ValueAtBrush = HitWeightValue;
			}
		}
		bPendingPickWeight = false;
	}

	auto ExecuteStampOperation = [this](int StampIndex, const FRay& StampRay)
		{
			SCOPE_CYCLE_COUNTER(WeightMapPaintTool_Tick_ApplyStampBlock);

			// update sculpt ROI
			UpdateROI(CurrentStamp);

			// append updated ROI to modified region (async)
			FDynamicMesh3* Mesh = GetSculptMesh();
			TFuture<void> AccumulateROI = Async(UE::Dataflow::Private::WeightPaintToolAsyncExecTarget, [&]()
				{
					UE::Geometry::VertexToTriangleOneRing(Mesh, VertexROI, AccumulatedTriangleROI);
				});

			// apply the stamp
			bool bWeightsModified = ApplyStamp();

			if (bWeightsModified)
			{
				SCOPE_CYCLE_COUNTER(WeightMapPaintTool_Tick_UpdateMeshBlock);
				UpdateVertexColorOverlay(&TriangleROI);
				DynamicMeshComponent->FastNotifyTriangleVerticesUpdated(TriangleROI, EMeshRenderAttributeFlags::VertexColors);
				GetToolManager()->PostInvalidation();
			}

			// we don't really need to wait for these to happen to end Tick()...
			AccumulateROI.Wait();
		};


	if (IsInBrushSubMode())
	{
		ProcessPerTickStamps(
			[this](const FRay& StampRay) -> bool {
				return UpdateStampPosition(StampRay);
			}, ExecuteStampOperation);
	}

}

bool UDataflowEditorWeightMapPaintTool::CanAccept() const
{
	return bAnyChangeMade || UpdateWeightMapProperties->Name != WeightMapNodeToUpdate->Name;
}



FColor UDataflowEditorWeightMapPaintTool::GetColorForWeightValue(double WeightValue)
{
	FColor MaxColor = LinearColors::White3b();
	FColor MinColor = LinearColors::Black3b();
	FColor Color;
	double ClampedValue = FMath::Clamp(WeightValue, 0.0, 1.0);
	Color.R = FMath::LerpStable(MinColor.R, MaxColor.R, ClampedValue);
	Color.G = FMath::LerpStable(MinColor.G, MaxColor.G, ClampedValue);
	Color.B = FMath::LerpStable(MinColor.B, MaxColor.B, ClampedValue);
	Color.A = 1.0;

	return Color;
}

void UDataflowEditorWeightMapPaintTool::FloodFillCurrentWeightAction()
{
	if (!ActiveWeightMap)
	{
		return;
	}

	BeginChange();

	const float SetWeightValue = FilterProperties->AttributeValue;
	const FDynamicMesh3* Mesh = DynamicMeshComponent->GetMesh();
	TempROIBuffer.SetNum(0, EAllowShrinking::No);
	for (int32 VertexIdx : Mesh->VertexIndicesItr())
	{
		TempROIBuffer.Add(VertexIdx);
	}

	if (bHaveDynamicMeshToWeightConversion)
	{
		for (int32 VertexIdx : TempROIBuffer)
		{
			for (const int32 Idx : WeightToDynamicMesh[DynamicMeshToWeight[VertexIdx]])
			{
				ActiveWeightEditChangeTracker->SaveVertexOneRingTriangles(Idx, true);
				ActiveWeightMap->SetValue(Idx, &SetWeightValue);
			}
		}
	}
	else
	{
		for (int32 VertexIdx : TempROIBuffer)
		{
			ActiveWeightEditChangeTracker->SaveVertexOneRingTriangles(VertexIdx, true);
		}
		for (int32 VertexIdx : TempROIBuffer)
		{
			ActiveWeightMap->SetValue(VertexIdx, &SetWeightValue);
		}
	}

	// update colors
	UpdateVertexColorOverlay();
	DynamicMeshComponent->FastNotifyVertexAttributesUpdated(EMeshRenderAttributeFlags::VertexColors);
	GetToolManager()->PostInvalidation();
	EndChange();
}

void UDataflowEditorWeightMapPaintTool::ClearAllWeightsAction()
{
	if (!ActiveWeightMap)
	{
		return;
	}

	BeginChange();

	float SetWeightValue = 0.0f;
	const FDynamicMesh3* Mesh = DynamicMeshComponent->GetMesh();
	TempROIBuffer.SetNum(0, EAllowShrinking::No);
	for (int32 VertexIdx : Mesh->VertexIndicesItr())
	{
		TempROIBuffer.Add(VertexIdx);
	}

	if (bHaveDynamicMeshToWeightConversion)
	{
		for (int32 VertexIdx : TempROIBuffer)
		{
			for (const int32 Idx : WeightToDynamicMesh[DynamicMeshToWeight[VertexIdx]])
			{
				ActiveWeightEditChangeTracker->SaveVertexOneRingTriangles(Idx, true);
				ActiveWeightMap->SetValue(Idx, &SetWeightValue);
			}
		}
	}
	else
	{
		for (int32 VertexIdx : TempROIBuffer)
		{
			ActiveWeightEditChangeTracker->SaveVertexOneRingTriangles(VertexIdx, true);
		}
		for (int32 VertexIdx : TempROIBuffer)
		{
			ActiveWeightMap->SetValue(VertexIdx, &SetWeightValue);
		}
	}

	// update colors
	UpdateVertexColorOverlay();
	DynamicMeshComponent->FastNotifyVertexAttributesUpdated(EMeshRenderAttributeFlags::VertexColors);
	GetToolManager()->PostInvalidation();
	EndChange();
}

void UDataflowEditorWeightMapPaintTool::InvertCurrentWeightAction(bool bInvertSurfaceOnly)
{
	if (!ActiveWeightMap)
	{
		return;
	}

	BeginChange();


	const FDynamicMesh3* Mesh = DynamicMeshComponent->GetMesh();
	TempROIBuffer.SetNum(0, EAllowShrinking::No);
	for (int32 VertexIdx : Mesh->VertexIndicesItr())
	{
		if (!bInvertSurfaceOnly || Mesh->IsReferencedVertex(VertexIdx))
		{
			TempROIBuffer.Add(VertexIdx);
		}
	}

	if (bHaveDynamicMeshToWeightConversion)
	{
		for (int32 VertexIdx : TempROIBuffer)
		{
			for (const int32 Idx : WeightToDynamicMesh[DynamicMeshToWeight[VertexIdx]])
			{
				ActiveWeightEditChangeTracker->SaveVertexOneRingTriangles(Idx, true);
				float CurrentValue;
				ActiveWeightMap->GetValue(Idx, &CurrentValue);
				float SetWeightValue = 1.f - CurrentValue;
				ActiveWeightMap->SetValue(Idx, &SetWeightValue);
			}
		}
	}
	else
	{
		for (int32 VertexIdx : TempROIBuffer)
		{
			ActiveWeightEditChangeTracker->SaveVertexOneRingTriangles(VertexIdx, true);
		}
		for (int32 VertexIdx : TempROIBuffer)
		{
			float CurrentValue;
			ActiveWeightMap->GetValue(VertexIdx, &CurrentValue);
			float SetWeightValue = 1.f - CurrentValue;
			ActiveWeightMap->SetValue(VertexIdx, &SetWeightValue);
		}
	}

	// update colors
	UpdateVertexColorOverlay();
	DynamicMeshComponent->FastNotifyVertexAttributesUpdated(EMeshRenderAttributeFlags::VertexColors);
	GetToolManager()->PostInvalidation();
	EndChange();
}

void UDataflowEditorWeightMapPaintTool::UpdateSelectedNode()
{
	check(ActiveWeightMap);
	TArray<float> CurrentWeights;
	GetCurrentWeightMap(CurrentWeights);

	check(WeightMapNodeToUpdate);

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (bHaveDynamicMeshToWeightConversion)
	{
		WeightMapNodeToUpdate->ReportVertexWeights(SetupWeights, CurrentWeights, DynamicMeshToWeight);
	}
	else
	{
		WeightMapNodeToUpdate->ReportVertexWeights(SetupWeights, CurrentWeights, {});
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	WeightMapNodeToUpdate->Name = UpdateWeightMapProperties->Name;

	WeightMapNodeToUpdate->Invalidate();
}


//
// Change Tracking
//

namespace UE::DataflowWeightPaintLocals
{

	/**
	 * A wrapper change that applies a given change to the unwrap canonical mesh of an input, and uses that
	 * to update the other views. Causes a broadcast of OnCanonicalModified.
	 */
	class  FDataflowWeightPaintMeshChange : public FToolCommandChange
	{
	public:
		FDataflowWeightPaintMeshChange(UDynamicMeshComponent* DynamicMeshComponentIn, TUniquePtr<FDynamicMeshChange> DynamicMeshChangeIn)
			: DynamicMeshComponent(DynamicMeshComponentIn)
			, DynamicMeshChange(MoveTemp(DynamicMeshChangeIn))
		{
			ensure(DynamicMeshComponentIn);
			ensure(DynamicMeshChange);
		};

		virtual void Apply(UObject* Object) override
		{
			DynamicMeshChange->Apply(DynamicMeshComponent->GetMesh(), false);			
		}

		virtual void Revert(UObject* Object) override
		{
			DynamicMeshChange->Apply(DynamicMeshComponent->GetMesh(), true);			
		}

		virtual bool HasExpired(UObject* Object) const override
		{
			return !(DynamicMeshComponent.IsValid() && DynamicMeshChange);
		}


		virtual FString ToString() const override
		{
			return TEXT("FDataflowWeightPaintMeshChange");
		}

	protected:
		TWeakObjectPtr<UDynamicMeshComponent> DynamicMeshComponent;
		TUniquePtr<FDynamicMeshChange> DynamicMeshChange;
	};
}

void UDataflowEditorWeightMapPaintTool::BeginChange()
{

	check(ActiveWeightEditChangeTracker == nullptr);
	
	ActiveWeightEditChangeTracker = MakeUnique<FDynamicMeshChangeTracker>(GetSculptMesh());
	ActiveWeightEditChangeTracker->BeginChange();
	LongTransactions.Open(LOCTEXT("WeightPaintChange", "Weight Stroke"), GetToolManager());
}

void UDataflowEditorWeightMapPaintTool::EndChange()
{
	check(ActiveWeightEditChangeTracker);

	bAnyChangeMade = true;

	TUniquePtr<FDynamicMeshChange> EditResult = ActiveWeightEditChangeTracker->EndChange();

	TUniquePtr<UE::DataflowWeightPaintLocals::FDataflowWeightPaintMeshChange> DataflowWeightPaintMeshChange =
		MakeUnique<UE::DataflowWeightPaintLocals::FDataflowWeightPaintMeshChange>(DynamicMeshComponent.Get(), MoveTemp(EditResult));
	ActiveWeightEditChangeTracker = nullptr;

	TUniquePtr<TWrappedToolCommandChange<UE::DataflowWeightPaintLocals::FDataflowWeightPaintMeshChange>> NewChange = MakeUnique<TWrappedToolCommandChange<UE::DataflowWeightPaintLocals::FDataflowWeightPaintMeshChange>>();
	NewChange->WrappedChange = MoveTemp(DataflowWeightPaintMeshChange);
	NewChange->BeforeModify = [this](bool bRevert)
	{
		this->WaitForPendingUndoRedoUpdate();
	};

	GetToolManager()->EmitObjectChange(DynamicMeshComponent, MoveTemp(NewChange), LOCTEXT("WeightPaintChange", "Weight Stroke"));
	LongTransactions.Close(GetToolManager());
}


void UDataflowEditorWeightMapPaintTool::WaitForPendingUndoRedoUpdate()
{
	if (bUndoUpdatePending)
	{
		bUndoUpdatePending = false;
	}
}

void UDataflowEditorWeightMapPaintTool::OnDynamicMeshComponentChanged(UDynamicMeshComponent* Component, const FMeshVertexChange* Change, bool bRevert)
{
	// update octree
	FDynamicMesh3* Mesh = GetSculptMesh();

	// make sure any previous async computations are done, and update the undo ROI
	if (bUndoUpdatePending)
	{
		// we should never hit this anymore, because of pre-change calling WaitForPendingUndoRedoUpdate()
		WaitForPendingUndoRedoUpdate();

		// this is not right because now we are going to do extra recomputation, but it's very messy otherwise...
		UE::Geometry::VertexToTriangleOneRing(Mesh, Change->Vertices, AccumulatedTriangleROI);
	}
	else
	{
		AccumulatedTriangleROI.Reset();
		UE::Geometry::VertexToTriangleOneRing(Mesh, Change->Vertices, AccumulatedTriangleROI);
	}

	// note that we have a pending update
	bUndoUpdatePending = true;
}


void UDataflowEditorWeightMapPaintTool::PrecomputeFilterData()
{
	const FDynamicMesh3* Mesh = GetSculptMesh();
	
	TriNormals.SetNum(Mesh->MaxTriangleID());
	ParallelFor(Mesh->MaxTriangleID(), [&](int32 tid)
	{
		if (Mesh->IsTriangle(tid))
		{
			TriNormals[tid] = Mesh->GetTriNormal(tid);
		}
	});

	const FDynamicMeshNormalOverlay* Normals = Mesh->Attributes()->PrimaryNormals();
	const FDynamicMeshUVOverlay* UVs = Mesh->Attributes()->PrimaryUV();
	UVSeamEdges.SetNum(Mesh->MaxEdgeID());
	NormalSeamEdges.SetNum(Mesh->MaxEdgeID());
	ParallelFor(Mesh->MaxEdgeID(), [&](int32 eid)
	{
		if (Mesh->IsEdge(eid))
		{
			UVSeamEdges[eid] = UVs ? UVs->IsSeamEdge(eid) : false;
			NormalSeamEdges[eid] = Normals ? Normals->IsSeamEdge(eid) : false;
		}
	});
}

double UDataflowEditorWeightMapPaintTool::GetCurrentWeightValue(int32 VertexId) const
{
	float WeightValue = 0.0;
	if (ActiveWeightMap && VertexId != IndexConstants::InvalidID)
	{
		ActiveWeightMap->GetValue(VertexId, &WeightValue);
	}
	return WeightValue;
}

double UDataflowEditorWeightMapPaintTool::GetCurrentWeightValueUnderBrush() const
{
	float WeightValue = 0.0;
	int32 VertexID = GetBrushNearestVertex();
	if (ActiveWeightMap && VertexID != IndexConstants::InvalidID)
	{
		ActiveWeightMap->GetValue(VertexID, &WeightValue);
	}
	return WeightValue;
}

int32 UDataflowEditorWeightMapPaintTool::GetBrushNearestVertex() const
{
	int TriangleVertex = 0;

	if (CurrentBaryCentricCoords.X >= CurrentBaryCentricCoords.Y && CurrentBaryCentricCoords.X >= CurrentBaryCentricCoords.Z)
	{
		TriangleVertex = 0;
	}
	else
	{
		if (CurrentBaryCentricCoords.Y >= CurrentBaryCentricCoords.X && CurrentBaryCentricCoords.Y >= CurrentBaryCentricCoords.Z)
		{
			TriangleVertex = 1;
		}
		else
		{
			TriangleVertex = 2;
		}
	}
	const FDynamicMesh3* Mesh = GetSculptMesh();

	int32 Tid = GetBrushTriangleID();
	if (Tid == IndexConstants::InvalidID)
	{
		return IndexConstants::InvalidID;
	}
	
	FIndex3i Vertices = Mesh->GetTriangle(Tid);
	return Vertices[TriangleVertex];
}

void UDataflowEditorWeightMapPaintTool::GetCurrentWeightMap(TArray<float>& OutWeights) const
{
	if (ActiveWeightMap)
	{
		const FDynamicMesh3* Mesh = GetSculptMesh();
		const int32 NumVertices = Mesh->VertexCount();
		OutWeights.SetNumUninitialized(NumVertices);
		for (int32 VertexID = 0; VertexID < NumVertices; ++VertexID)
		{
			ActiveWeightMap->GetValue(VertexID, &OutWeights[VertexID]);
		}
	}
}



void UDataflowEditorWeightMapPaintTool::UpdateSubToolType(EDataflowEditorWeightMapPaintInteractionType NewType)
{
	// Currenly we mirror base-brush properties in UDataflowEditorWeightMapPaintBrushFilterProperties, so we never
	// want to show both
	//bool bSculptPropsVisible = (NewType == EDataflowEditorWeightMapPaintInteractionType::Brush);
	//SetToolPropertySourceEnabled(UMeshSculptToolBase::BrushProperties, bSculptPropsVisible);
	SetToolPropertySourceEnabled(UMeshSculptToolBase::BrushProperties, false);

	SetToolPropertySourceEnabled(FilterProperties, true);
	SetBrushOpPropsVisibility(false);

	if (NewType != EDataflowEditorWeightMapPaintInteractionType::Gradient)
	{
		LowValueGradientVertexSelection.Clear();
		HighValueGradientVertexSelection.Clear();
	}
}


void UDataflowEditorWeightMapPaintTool::UpdateBrushType(EDataflowEditorWeightMapPaintBrushType BrushType)
{
	static const FText BaseMessage = LOCTEXT("OnStartTool", "Hold Shift to Erase. [/] and S/D change Size (+Shift to small-step)");
	FTextBuilder Builder;
	Builder.AppendLine(BaseMessage);

	SetActivePrimaryBrushType((int32)BrushType);

	SetToolPropertySourceEnabled(GizmoProperties, false);

	GetToolManager()->DisplayMessage(Builder.ToText(), EToolMessageLevel::UserNotification);
}




void UDataflowEditorWeightMapPaintTool::RequestAction(EDataflowEditorWeightMapPaintToolActions ActionType)
{
	if (!bHavePendingAction)
	{
		PendingAction = ActionType;
		bHavePendingAction = true;
	}
}


void UDataflowEditorWeightMapPaintTool::ApplyAction(EDataflowEditorWeightMapPaintToolActions ActionType)
{
	switch (ActionType)
	{
	case EDataflowEditorWeightMapPaintToolActions::FloodFillCurrent:
		FloodFillCurrentWeightAction();
		break;

	case EDataflowEditorWeightMapPaintToolActions::ClearAll:
		ClearAllWeightsAction();
		break;

	case EDataflowEditorWeightMapPaintToolActions::InvertCurrent:
		InvertCurrentWeightAction(false);
		break;

	case EDataflowEditorWeightMapPaintToolActions::InvertCurrentSurface:
		InvertCurrentWeightAction(true);
		break;
	}
}


void UDataflowEditorWeightMapPaintTool::UpdateVertexColorOverlay(const TSet<int>* TrianglesToUpdate)
{
	FDynamicMesh3* const Mesh = GetSculptMesh();
	check(Mesh->HasAttributes());
	check(Mesh->Attributes()->PrimaryColors());
	check(ActiveWeightMap);

	FDynamicMeshColorOverlay* const ColorOverlay = Mesh->Attributes()->PrimaryColors();

	auto SetColorsFromWeights = [&](int TriangleID)
	{
		const FIndex3i Tri = Mesh->GetTriangle(TriangleID);
		const FIndex3i ColorElementTri = ColorOverlay->GetTriangle(TriangleID);

		for (int TriVertIndex = 0; TriVertIndex < 3; ++TriVertIndex)
		{
			float VertexWeight;
			ActiveWeightMap->GetValue(Tri[TriVertIndex], &VertexWeight);

			const FVector4f NewColor = FVector4f(VertexWeight, VertexWeight, VertexWeight, 1.0f);
			ColorOverlay->SetElement(ColorElementTri[TriVertIndex], NewColor);
		}
	};

	if (TrianglesToUpdate)
	{
		for (const int TriangleID : *TrianglesToUpdate)
		{
			SetColorsFromWeights(TriangleID);
		}
	}
	else
	{
		for (const int TriangleID : Mesh->TriangleIndicesItr())
		{
			SetColorsFromWeights(TriangleID);
		}
	}
}

void UDataflowEditorWeightMapPaintTool::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	UDataflowEditorWeightMapPaintTool* This = CastChecked<UDataflowEditorWeightMapPaintTool>(InThis);
	Collector.AddReferencedObject(This->PreviewMeshActor);
	Collector.AddReferencedObject(This->DynamicMeshComponent);
	Collector.AddReferencedObject(This->MeshElementsDisplay);
	Collector.AddReferencedObject(This->DataflowEditorContextObject);
	Super::AddReferencedObjects(InThis, Collector);
}

#undef LOCTEXT_NAMESPACE
