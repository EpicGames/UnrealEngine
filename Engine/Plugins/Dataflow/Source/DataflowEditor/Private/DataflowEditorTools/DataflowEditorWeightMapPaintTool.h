// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "UObject/NoExportTypes.h"
#include "BaseTools/BaseBrushTool.h"
#include "BaseTools/MeshSurfacePointMeshEditingTool.h"
#include "Components/DynamicMeshComponent.h"
#include "PropertySets/WeightMapSetProperties.h"
#include "Mechanics/PolyLassoMarqueeMechanic.h"

#include "Sculpting/MeshSculptToolBase.h"
#include "Sculpting/MeshBrushOpBase.h"

#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "DynamicMesh/DynamicMeshOctree3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/DynamicMeshChangeTracker.h"
#include "DynamicMesh/MeshNormals.h"
#include "TransformTypes.h"
#include "ToolDataVisualizer.h"
#include "GroupTopology.h"
#include "MeshVertexPaintTool.h"
#include "DataflowEditorTools/DataflowEditorToolBuilder.h"

#include "DataflowEditorWeightMapPaintTool.generated.h"

#define UE_API DATAFLOWEDITOR_API

class UMeshElementsVisualizer;
class UDataflowWeightMapEraseBrushOpProps;
class UDataflowWeightMapPaintBrushOpProps;
class UDataflowWeightMapSmoothBrushOpProps;
class UPolygonSelectionMechanic;
class UDataflowContextObject;
struct FDataflowCollectionAddScalarVertexPropertyNode;
class UDataflowEditorMode;
class UInteractiveToolManager;

DECLARE_STATS_GROUP(TEXT("WeightMapPaintTool"), STATGROUP_WeightMapPaintTool, STATCAT_Advanced);
DECLARE_CYCLE_STAT(TEXT("WeightMapPaintTool_UpdateROI"), WeightMapPaintTool_UpdateROI, STATGROUP_WeightMapPaintTool);
DECLARE_CYCLE_STAT(TEXT("WeightMapPaintTool_ApplyStamp"), WeightMapPaintToolApplyStamp, STATGROUP_WeightMapPaintTool );
DECLARE_CYCLE_STAT(TEXT("WeightMapPaintTool_Tick"), WeightMapPaintToolTick, STATGROUP_WeightMapPaintTool);
DECLARE_CYCLE_STAT(TEXT("WeightMapPaintTool_Tick_ApplyStampBlock"), WeightMapPaintTool_Tick_ApplyStampBlock, STATGROUP_WeightMapPaintTool);
DECLARE_CYCLE_STAT(TEXT("WeightMapPaintTool_Tick_ApplyStamp_Remove"), WeightMapPaintTool_Tick_ApplyStamp_Remove, STATGROUP_WeightMapPaintTool);
DECLARE_CYCLE_STAT(TEXT("WeightMapPaintTool_Tick_ApplyStamp_Insert"), WeightMapPaintTool_Tick_ApplyStamp_Insert, STATGROUP_WeightMapPaintTool);
DECLARE_CYCLE_STAT(TEXT("WeightMapPaintTool_Tick_NormalsBlock"), WeightMapPaintTool_Tick_NormalsBlock, STATGROUP_WeightMapPaintTool);
DECLARE_CYCLE_STAT(TEXT("WeightMapPaintTool_Tick_UpdateMeshBlock"), WeightMapPaintTool_Tick_UpdateMeshBlock, STATGROUP_WeightMapPaintTool);
DECLARE_CYCLE_STAT(TEXT("WeightMapPaintTool_Tick_UpdateTargetBlock"), WeightMapPaintTool_Tick_UpdateTargetBlock, STATGROUP_WeightMapPaintTool);
DECLARE_CYCLE_STAT(TEXT("WeightMapPaintTool_Normals_Collect"), WeightMapPaintTool_Normals_Collect, STATGROUP_WeightMapPaintTool);
DECLARE_CYCLE_STAT(TEXT("WeightMapPaintTool_Normals_Compute"), WeightMapPaintTool_Normals_Compute, STATGROUP_WeightMapPaintTool);





/**
 * Tool Builder
 */
UCLASS(MinimalAPI)
class UDataflowEditorWeightMapPaintToolBuilder : public UMeshSurfacePointMeshEditingToolBuilder, public IDataflowEditorToolBuilder
{
	GENERATED_BODY()
public:
	UE_API void SetEditorMode(UDataflowEditorMode* InMode);

private:
	UE_API virtual void GetSupportedConstructionViewModes(const UDataflowContextObject& ContextObject, TArray<const UE::Dataflow::IDataflowConstructionViewMode*>& Modes) const override;
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual UMeshSurfacePointTool* CreateNewTool(const FToolBuilderState& SceneState) const override;
	virtual bool CanSetConstructionViewWireframeActive() const { return false; }

	// Per-editor mode lookup keyed by the editor's tool manager. This builder is held by the process-wide FDataflowToolRegistry singleton, 
	// so a single Mode pointer would alias across simultaneously-open Dataflow editors; the map keeps each editor's mode separate.
	TMap<TWeakObjectPtr<const UInteractiveToolManager>, TWeakObjectPtr<UDataflowEditorMode>> ModeForToolManager;
};




/** Mesh Sculpting Brush Types */
UENUM()
enum class EDataflowEditorWeightMapPaintInteractionType : uint8
{
	Brush,
	Fill,
	PolyLasso,
	Gradient,

	LastValue UMETA(Hidden)
};

/** Mesh Sculpting Brush Types */
UENUM()
enum class EDataflowEditorWeightMapPaintBrushType : uint8
{
	/** Paint weights */
	Paint UMETA(DisplayName = "Paint"),

	/** Smooth existing weights */
	Smooth UMETA(DisplayName = "Smooth"),

	/** Erase weights */
	Erase UMETA(Hidden, DisplayName = "Erase"),

	LastValue UMETA(Hidden)
};


/** Mesh Sculpting Brush Area Types */
UENUM()
enum class EDataflowEditorWeightMapPaintBrushAreaType : uint8
{
	Connected,
	Volumetric
};

/** Mesh Sculpting Brush Types */
UENUM()
enum class EDataflowEditorWeightMapPaintVisibilityType : uint8
{
	None,
	Unoccluded
};

// TODO: Look at EditConditions for all these properties. Which ones make sense for which SubToolType?

UCLASS(MinimalAPI)
class UDataflowEditorWeightMapPaintBrushFilterProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	
	UPROPERTY(EditAnywhere, Category = ActionType, meta = (DisplayName = "Action"))
	EDataflowEditorWeightMapPaintInteractionType SubToolType = EDataflowEditorWeightMapPaintInteractionType::Brush;

	UPROPERTY(EditAnywhere, Category = ActionType, meta = (DisplayName = "Brush Mode", 
		HideEditConditionToggle, EditConditionHides, EditCondition = "SubToolType == EDataflowEditorWeightMapPaintInteractionType::Brush"))
	EDataflowEditorWeightMapPaintBrushType PrimaryBrushType = EDataflowEditorWeightMapPaintBrushType::Paint;

	/** Relative size of brush */
	UPROPERTY(EditAnywhere, Category = Brush, meta = (DisplayName = "Brush Size", UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "10.0", 
		HideEditConditionToggle, EditConditionHides, EditCondition = "SubToolType == EDataflowEditorWeightMapPaintInteractionType::Brush"))
	float BrushSize = 0.25f;

	/** The new value to paint on the mesh */
	UPROPERTY(EditAnywhere, Category = Brush, meta = (UIMin = 0, ClampMin = 0, UIMax = 1, ClampMax = 1,
		HideEditConditionToggle, EditConditionHides, EditCondition = 
		"(SubToolType == EDataflowEditorWeightMapPaintInteractionType::Brush && PrimaryBrushType == EDataflowEditorWeightMapPaintBrushType::Paint) || SubToolType == EDataflowEditorWeightMapPaintInteractionType::Fill || SubToolType == EDataflowEditorWeightMapPaintInteractionType::PolyLasso"))
	double AttributeValue = 1;

	/** How quickly each brush stroke will drive mesh values towards the desired value */
	UPROPERTY(EditAnywhere, Category = Brush, meta = (UIMin = 0, ClampMin = 0, UIMax = 1, ClampMax = 1,
		HideEditConditionToggle, EditConditionHides, EditCondition = 
		"SubToolType == EDataflowEditorWeightMapPaintInteractionType::Brush || SubToolType == EDataflowEditorWeightMapPaintInteractionType::Fill"))
	double Strength = 0.5;

	/** The Gradient upper limit value */
	UPROPERTY(EditAnywhere, Category = Gradient, meta = (UIMin = 0, ClampMin = 0, UIMax = 1, ClampMax = 1,
		HideEditConditionToggle, EditConditionHides, EditCondition = "SubToolType == EDataflowEditorWeightMapPaintInteractionType::Gradient"))
	double GradientHighValue = 1.0;

	/** The Gradient lower limit value */
	UPROPERTY(EditAnywhere, Category = Gradient, meta = (UIMin = 0, ClampMin = 0, UIMax = 1, ClampMax = 1,
		HideEditConditionToggle, EditConditionHides, EditCondition = "SubToolType == EDataflowEditorWeightMapPaintInteractionType::Gradient"))
	double GradientLowValue = 0.0;

	/** Area Mode specifies the shape of the brush and which triangles will be included relative to the cursor */
	UPROPERTY(EditAnywhere, Category = Filters, meta = (DisplayName = "Brush Area Mode",
		HideEditConditionToggle, EditConditionHides, EditCondition =
		"SubToolType == EDataflowEditorWeightMapPaintInteractionType::Brush || SubToolType == EDataflowEditorWeightMapPaintInteractionType::Fill"))
	EMeshVertexPaintBrushAreaType BrushAreaMode = EMeshVertexPaintBrushAreaType::Connected;
	
	/** The Region affected by the current operation will be bounded by edge angles larger than this threshold */
	UPROPERTY(EditAnywhere, Category = Filters, meta = (UIMin = "0.0", ClampMin = "0.0", UIMax = "180.0", ClampMax = "180.0",
		HideEditConditionToggle, EditConditionHides, EditCondition = 
		"SubToolType == EDataflowEditorWeightMapPaintInteractionType::Brush || SubToolType == EDataflowEditorWeightMapPaintInteractionType::Fill"))
	float AngleThreshold = 180.0f;

	/** The Region affected by the current operation will be bounded by UV borders/seams */
	UPROPERTY(EditAnywhere, Category = Filters, meta = (HideEditConditionToggle, EditConditionHides, EditCondition = 
		"SubToolType == EDataflowEditorWeightMapPaintInteractionType::Brush || SubToolType == EDataflowEditorWeightMapPaintInteractionType::Fill"))
	bool bUVSeams = false;

	/** The Region affected by the current operation will be bounded by Hard Normal edges/seams */
	UPROPERTY(EditAnywhere, Category = Filters, meta = (HideEditConditionToggle, EditConditionHides, EditCondition = 
		"SubToolType == EDataflowEditorWeightMapPaintInteractionType::Brush || SubToolType == EDataflowEditorWeightMapPaintInteractionType::Fill"))
	bool bNormalSeams = false;

	/** Control which triangles can be affected by the current operation based on visibility. Applied after all other filters. */
	UPROPERTY(EditAnywhere, Category = Filters, meta = (HideEditConditionToggle, EditConditionHides, EditCondition =
		"SubToolType == EDataflowEditorWeightMapPaintInteractionType::Brush || SubToolType == EDataflowEditorWeightMapPaintInteractionType::Fill || SubToolType == EDataflowEditorWeightMapPaintInteractionType::PolyLasso"))
	EDataflowEditorWeightMapPaintVisibilityType VisibilityFilter = EDataflowEditorWeightMapPaintVisibilityType::None;

	/** The weight value at the brush indicator */
	UPROPERTY(VisibleAnywhere, Transient, Category = Query, meta = (NoResetToDefault, 
		HideEditConditionToggle, EditConditionHides, EditCondition = "SubToolType == EDataflowEditorWeightMapPaintInteractionType::Brush || SubToolType == EDataflowEditorWeightMapPaintInteractionType::Fill"))
	double ValueAtBrush = 0;

};

UENUM()
enum class EDataflowEditorWeightMapPaintToolActions
{
	NoAction,
	FloodFillCurrent,
	ClearAll,
	InvertCurrent,
	InvertCurrentSurface
};

UCLASS(MinimalAPI)
class UDataflowEditorMeshWeightMapPaintToolActions : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:

	TWeakObjectPtr<UDataflowEditorWeightMapPaintTool> ParentTool;

	void Initialize(UDataflowEditorWeightMapPaintTool* ParentToolIn) { ParentTool = ParentToolIn; }

	UE_API void PostAction(EDataflowEditorWeightMapPaintToolActions Action);

	UFUNCTION(CallInEditor, Category = Operations, meta = (DisplayPriority = 10))
	void ClearAll()
	{
		PostAction(EDataflowEditorWeightMapPaintToolActions::ClearAll);
	}

	UFUNCTION(CallInEditor, Category = Operations, meta = (DisplayPriority = 12))
	void FloodFillCurrent()
	{
		PostAction(EDataflowEditorWeightMapPaintToolActions::FloodFillCurrent);
	}

	/* Invert the values in range [0, 1] for the current selected geometry, including interior vertices*/
	UFUNCTION(CallInEditor, Category = Operations, meta = (DisplayPriority = 13))
	void InvertCurrent()
	{
		PostAction(EDataflowEditorWeightMapPaintToolActions::InvertCurrent);
	}

	/* Invert the values in range [0, 1] for the current selected surface*/
	UFUNCTION(CallInEditor, Category = Operations, meta = (DisplayPriority = 13))
	void InvertCurrentSurface()
	{
		PostAction(EDataflowEditorWeightMapPaintToolActions::InvertCurrentSurface);
	}
};

UCLASS(MinimalAPI)
class UDataflowEditorUpdateWeightMapProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:

	UPROPERTY()
	FString Name;

private:

	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
};

/**
 * Mesh Element Paint Tool Class
 */
UCLASS(MinimalAPI)
class UDataflowEditorWeightMapPaintTool : public UMeshSculptToolBase
{
	GENERATED_BODY()

public:
	UE_API virtual void RegisterActions(FInteractiveToolActionSet& ActionSet) override;

	UE_API virtual void Setup() override;
	UE_API virtual void Shutdown(EToolShutdownType ShutdownType) override;
	UE_API virtual void DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI) override;
	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;
	UE_API virtual void UpdateMaterialMode(EMeshEditingMaterialModes MaterialMode) override;
	UE_API virtual void OnTick(float DeltaTime) override;

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }
	UE_API virtual bool CanAccept() const override;

	UE_API virtual bool OnUpdateHover(const FInputDeviceRay& DevicePos) override;

	UE_API virtual void OnPropertyModified(UObject* PropertySet, FProperty* Property) override;

	UE_API bool IsInBrushSubMode() const;

	UE_API virtual void CommitResult(UBaseDynamicMeshComponent* Component, bool bModifiedTopology) override;

	UE_API void SetDataflowEditorContextObject(TObjectPtr<UDataflowContextObject> InDataflowEditorContextObject);
	void SetEditorMode(UDataflowEditorMode* InMode) { Mode = InMode; }

	virtual void SetupBrushEditBehaviorSetup(ULocalTwoAxisPropertyEditInputBehavior& OutBehavior) override;

	//~ UObject interface

	static UE_API void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);

public:


	/** Filters on paint brush */
	UPROPERTY()
	TObjectPtr<UDataflowEditorWeightMapPaintBrushFilterProperties> FilterProperties;


private:
	UPROPERTY()
	TObjectPtr<UDataflowWeightMapPaintBrushOpProps> PaintBrushOpProperties;

	UPROPERTY()
	TObjectPtr<UDataflowWeightMapSmoothBrushOpProps> SmoothBrushOpProperties;

	UPROPERTY()
	TObjectPtr<UDataflowWeightMapEraseBrushOpProps> EraseBrushOpProperties;

public:
	UE_API void FloodFillCurrentWeightAction();
	UE_API void ClearAllWeightsAction();
	UE_API void InvertCurrentWeightAction(bool bInvertSurfaceOnly = true);

	UE_API void SetVerticesToWeightMap(const TSet<int32>& Vertices, double WeightValue, bool bIsErase);

	UE_API bool HaveVisibilityFilter() const;
	UE_API void ApplyVisibilityFilter(const TArray<int32>& Vertices, TArray<int32>& VisibleVertices);
	UE_API void ApplyVisibilityFilter(TSet<int32>& Triangles, TArray<int32>& ROIBuffer, TArray<int32>& OutputBuffer);

	// we override these so we can update the separate BrushSize property added for this tool
	UE_API virtual void IncreaseBrushRadiusAction();
	UE_API virtual void DecreaseBrushRadiusAction();
	UE_API virtual void IncreaseBrushRadiusSmallStepAction();
	UE_API virtual void DecreaseBrushRadiusSmallStepAction();

protected:
	// UMeshSculptToolBase API
	virtual UBaseDynamicMeshComponent* GetSculptMeshComponent() { return DynamicMeshComponent; }
	virtual FDynamicMesh3* GetBaseMesh() { check(false); return nullptr; }
	virtual const FDynamicMesh3* GetBaseMesh() const { check(false); return nullptr; }

	UE_API virtual int32 FindHitSculptMeshTriangleConst(const FRay3d& LocalRay) const override;
	UE_API virtual int32 FindHitTargetMeshTriangleConst(const FRay3d& LocalRay) const override;
	UE_API virtual void UpdateHitSculptMeshTriangle(int32 TriangleID, const FRay3d& LocalRay) override;

	UE_API virtual void OnBeginStroke(const FRay& WorldRay) override;
	UE_API virtual void OnEndStroke() override;

	UE_API virtual TUniquePtr<FMeshSculptBrushOp>& GetActiveBrushOp();

	virtual bool SharesBrushPropertiesChanges() const override { return false; }
	// end UMeshSculptToolBase API



	//
	// Action support
	//

public:
	UE_API virtual void RequestAction(EDataflowEditorWeightMapPaintToolActions ActionType);
	
	UPROPERTY()
	TObjectPtr<UDataflowEditorMeshWeightMapPaintToolActions> ActionsProps;

	UPROPERTY()
	TObjectPtr<UDataflowEditorUpdateWeightMapProperties> UpdateWeightMapProperties;

protected:
	bool bHavePendingAction = false;
	EDataflowEditorWeightMapPaintToolActions PendingAction;
	UE_API virtual void ApplyAction(EDataflowEditorWeightMapPaintToolActions ActionType);



	//
	// Marquee Support
	//
public:
	UPROPERTY()
	TObjectPtr<UPolyLassoMarqueeMechanic> PolyLassoMechanic;

protected:
	UE_API void OnPolyLassoFinished(const FCameraPolyLasso& Lasso, bool bCanceled);


	// 
	// Gradient Support
	//
	UPROPERTY()
	TObjectPtr<UPolygonSelectionMechanic> PolygonSelectionMechanic;

	TUniquePtr<UE::Geometry::FDynamicMeshAABBTree3> MeshSpatial = nullptr;

	TUniquePtr<UE::Geometry::FTriangleGroupTopology> GradientSelectionTopology = nullptr;

	FToolDataVisualizer GradientSelectionRenderer;

	UE::Geometry::FGroupTopologySelection LowValueGradientVertexSelection;
	UE::Geometry::FGroupTopologySelection HighValueGradientVertexSelection;

	UE_API void ComputeGradient();
	UE_API void OnSelectionModified();

	//
	// Internals
	//

protected:

	UPROPERTY()
	TObjectPtr<AInternalToolFrameworkActor> PreviewMeshActor = nullptr;

	UPROPERTY()
	TObjectPtr<UDynamicMeshComponent> DynamicMeshComponent = nullptr;

	UPROPERTY()
	TObjectPtr<UMeshElementsVisualizer> MeshElementsDisplay = nullptr;

	UPROPERTY()
	TObjectPtr<UDataflowContextObject> DataflowEditorContextObject = nullptr;

	// realtime visualization
	UE_API void OnDynamicMeshComponentChanged(UDynamicMeshComponent* Component, const FMeshVertexChange* Change, bool bRevert);
	FDelegateHandle OnDynamicMeshComponentChangedHandle;

	UE::Geometry::FDynamicMeshWeightAttribute* ActiveWeightMap;
	UE_API double GetCurrentWeightValue(int32 VertexId) const;
	UE_API double GetCurrentWeightValueUnderBrush() const;
	FVector3d CurrentBaryCentricCoords;
	UE_API int32 GetBrushNearestVertex() const;

	UE_API void GetCurrentWeightMap(TArray<float>& OutWeights) const;

	UE_API void UpdateSubToolType(EDataflowEditorWeightMapPaintInteractionType NewType);

	UE_API void UpdateBrushType(EDataflowEditorWeightMapPaintBrushType BrushType);

	TSet<int32> AccumulatedTriangleROI;
	bool bUndoUpdatePending = false;
	TArray<int> NormalsBuffer;
	UE_API void WaitForPendingUndoRedoUpdate();

	TArray<int> TempROIBuffer;
	TArray<int> VertexROI;
	TArray<bool> VisibilityFilterBuffer;
	TSet<int> VertexSetBuffer;
	TSet<int> TriangleROI;
	UE_API void UpdateROI(const FSculptBrushStamp& CurrentStamp);

	EDataflowEditorWeightMapPaintBrushType PendingStampType = EDataflowEditorWeightMapPaintBrushType::Paint;

	UE_API bool UpdateStampPosition(const FRay& WorldRay);
	UE_API bool ApplyStamp();

	UE::Geometry::FDynamicMeshOctree3 Octree;

	UE_API bool UpdateBrushPosition(const FRay& WorldRay);

	bool GetInEraseStroke()
	{
		// Re-use the smoothing stroke key (shift) for erase stroke in the weight paint tool
		return GetInSmoothingStroke();
	}


	bool bPendingPickWeight = false;

	TArray<int32> ROITriangleBuffer;
	TArray<double> ROIWeightValueBuffer;
	UE_API bool SyncMeshWithWeightBuffer(FDynamicMesh3* Mesh);
	UE_API bool SyncWeightBufferWithMesh(const FDynamicMesh3* Mesh);

	TUniquePtr<UE::Geometry::FDynamicMeshChangeTracker> ActiveWeightEditChangeTracker;
	UE_API void BeginChange();
	UE_API void EndChange();

	UE_API FColor GetColorForWeightValue(double WeightValue);

	TArray<FVector3d> TriNormals;
	TArray<int32> UVSeamEdges;
	TArray<int32> NormalSeamEdges;
	UE_API void PrecomputeFilterData();

	// DynamicMesh might be unwelded mesh, but weights are on the welded mesh.
	bool bHaveDynamicMeshToWeightConversion = false;
	TArray<int32> DynamicMeshToWeight; 
	TArray<TArray<int32>> WeightToDynamicMesh;

	/** Setup weights used to store the initial values */
	TArray<float> SetupWeights;

protected:
	virtual bool ShowWorkPlane() const override { return false; }

	friend class UDataflowEditorWeightMapPaintToolBuilder;

	bool bAnyChangeMade = false;

	// Node graph editor support

	FDataflowCollectionAddScalarVertexPropertyNode* WeightMapNodeToUpdate = nullptr;

	UE_API void UpdateSelectedNode();

	UE_API void UpdateVertexColorOverlay(const TSet<int>* TrianglesToUpdate = nullptr);

	UDataflowEditorMode* Mode = nullptr;
};



#undef UE_API
