// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EditorGizmos/TransformGizmoInterfaces.h"
#include "UObject/Object.h"
#include "DataflowGizmoModeSource.generated.h"

/**
 * Per-instance ITransformGizmoSource that locks the new editor TRS gizmo to a single
 * EGizmoTransformMode. Used when a Dataflow property opts into a Translate/Rotate/Scale-only
 * gizmo and must not follow the editor mode toolbar.
 */
UCLASS(MinimalAPI, Transient)
class UDataflowGizmoModeSource : public UObject, public ITransformGizmoSource
{
	GENERATED_BODY()

public:
	EGizmoTransformMode FixedMode = EGizmoTransformMode::Translate;

	virtual EGizmoTransformMode GetGizmoMode() const override
	{
		return FixedMode;
	}

	virtual EAxisList::Type GetGizmoAxisToDraw(EGizmoTransformMode InGizmoMode) const override
	{
		return InGizmoMode == FixedMode ? EAxisList::All : EAxisList::None;
	}

	virtual EToolContextCoordinateSystem GetGizmoCoordSystemSpace() const override
	{
		return EToolContextCoordinateSystem::Local;
	}

	virtual float GetGizmoScale() const override
	{
		return 1.0f;
	}

	virtual bool GetVisible(const EViewportContext InViewportContext = EViewportContext::Focused) const override
	{
		return true;
	}

	virtual bool CanInteract(const EViewportContext InViewportContext = EViewportContext::Focused) const override
	{
		return true;
	}

	virtual EGizmoTransformScaleType GetScaleType() const override
	{
		return EGizmoTransformScaleType::Default;
	}

	virtual const FRotationContext& GetRotationContext() const override
	{
		static const FRotationContext DefaultContext;
		return DefaultContext;
	}
};
