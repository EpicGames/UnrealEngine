// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataflowEditorTools/DataflowMeshSelectionTool.h"

#include "CanvasItem.h"
#include "CanvasTypes.h"
#include "InteractiveToolManager.h"
#include "Materials/MaterialInterface.h"
#include "ModelingToolTargetUtil.h"
#include "PreviewMesh.h"
#include "Selection/PolygonSelectionMechanic.h"
#include "ToolMeshSelector.h"
#include "ToolSetupUtil.h"
#include "UnrealEngine.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowMeshSelectionTool)

#define LOCTEXT_NAMESPACE "DataflowMeshSelectionTool"


///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

#include "Dataflow/DataflowSelectionToolNode.h"

void UDataflowSelectionToolNodeDataProvider::BindToNode(FDataflowSelectionToolNode& InNode, UE::Dataflow::FContext& Context)
{
	WeakNode = StaticCastWeakPtr<FDataflowSelectionToolNode>(InNode.AsWeak());
	InNode.LoadData(Context, NodeData);
}

void UDataflowSelectionToolNodeDataProvider::Init(const UE::Geometry::FDynamicMesh3& Mesh)
{
	MeshMapping.Init(Mesh);
}

void UDataflowSelectionToolNodeDataProvider::Shutdown()
{
	MeshMapping.Reset();
}

void UDataflowSelectionToolNodeDataProvider::GetVertexSelection(TSet<int32>& OutSelection) const
{
	using namespace UE::Dataflow;
	TSet<int32> SelectedVertices;
	NodeData.GetSelectedVertices(SelectedVertices);
	MeshMapping.RemapVertices(SelectedVertices, OutSelection, FDynamicMeshMapping::ERemapDirection::SourceToDynamicMesh);
}

void UDataflowSelectionToolNodeDataProvider::SetVertexSelection(const TSet<int32>& InSelection)
{
	using namespace UE::Dataflow;
	if (TSharedPtr<FDataflowSelectionToolNode> Node = WeakNode.Pin())
	{
		TSet<int32> SourceVerticesToSelect;
		MeshMapping.RemapVertices(InSelection, SourceVerticesToSelect, FDynamicMeshMapping::ERemapDirection::DynamicMeshToSource);
		const TArray<int32>& MappedSourceVertices = MeshMapping.GetMappedSourceVertices();
		if (MappedSourceVertices.IsEmpty())
		{
			NodeData.DeselectAllVertices();
		}
		else
		{
			NodeData.DeselectVertices(MeshMapping.GetMappedSourceVertices());
		}
		NodeData.SelectVertices(SourceVerticesToSelect);

		Node->SaveData(NodeData);
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

#include "Dataflow/DataflowRenderingViewMode.h"
#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"
#include "ToolTargetManager.h"
#include "Dataflow/DataflowContent.h"
#include "ToolContextInterfaces.h"
#include "ContextObjectStore.h"

void UDataflowMeshSelectionToolBuilder::GetSupportedConstructionViewModes(const UDataflowContextObject& ContextObject, TArray<const UE::Dataflow::IDataflowConstructionViewMode*>& Modes) const
{
	// matches UE::Dataflow::FDataflowConstruction3DViewMode::Name - but we cannot include the module to get it directly 
	static const FName DataflowConstruction3DViewModeName = TEXT("3DView");
	static const FName Cloth3DSimView = TEXT("Cloth3DSimView");
	static const FName ClothRenderView = TEXT("ClothRenderView");

	TArray<FName> ViewModeNames;
	// TODO : get the views from the node 
	ViewModeNames.Add(DataflowConstruction3DViewModeName);
	ViewModeNames.Add(Cloth3DSimView);
	ViewModeNames.Add(ClothRenderView);

	const UE::Dataflow::FRenderingViewModeFactory& Factory = UE::Dataflow::FRenderingViewModeFactory::GetInstance();
	for (FName ViewModeName : ViewModeNames)
	{
		if (const UE::Dataflow::IDataflowConstructionViewMode* ViewMode = Factory.GetViewMode(ViewModeName))
		{
			Modes.Add(ViewMode);
		}
	}
}

bool UDataflowMeshSelectionToolBuilder::CanSceneStateChange(const UInteractiveTool* ActiveTool, const FToolBuilderState& SceneState) const
{
	return ActiveTool->IsA<UDataflowMeshSelectionTool>();
}

void UDataflowMeshSelectionToolBuilder::SceneStateChanged(UInteractiveTool* ActiveTool, const FToolBuilderState& SceneState)
{
	check(CanSceneStateChange(ActiveTool, SceneState));

	if (UDataflowMeshSelectionTool* const SelectionTool = Cast<UDataflowMeshSelectionTool>(ActiveTool))
	{
		UToolTarget* const Target = SceneState.TargetManager->BuildFirstSelectedTargetable(SceneState, GetTargetRequirements());
		check(Target);
		check(Target->IsValid());
		SelectionTool->SetTarget(Target);
		check(SelectionTool->GetTargetWorld() == SceneState.World);
		if (UDataflowSelectionToolNodeDataProvider* DataProvider = NewObject<UDataflowSelectionToolNodeDataProvider>(SelectionTool))
		{
			if (UDataflowBaseContent* ContextObject = SceneState.ToolManager->GetContextObjectStore()->FindContext<UDataflowBaseContent>())
			{
				if (FDataflowSelectionToolNode* Node = ContextObject->GetSelectedNodeOfType<FDataflowSelectionToolNode>())
				{
					if (TSharedPtr<UE::Dataflow::FEngineContext> DataflowContext = ContextObject->GetDataflowContext())
					{
						DataProvider->BindToNode(*Node, *DataflowContext);
						SelectionTool->DataProvider = DataProvider;
					}
				}
			}
		}
		SelectionTool->NotifyTargetChanged();

		// These are likely to be empty functions but are called here for completeness (see UInteractiveToolManager::ActivateToolInternal())
		PostBuildTool(ActiveTool, SceneState);
		PostSetupTool(ActiveTool, SceneState);
	}

}

const FToolTargetTypeRequirements& UDataflowMeshSelectionToolBuilder::GetTargetRequirements() const
{
	static FToolTargetTypeRequirements TypeRequirements(UPrimitiveComponentBackedTarget::StaticClass());
	return TypeRequirements;
}

bool UDataflowMeshSelectionToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	if (SceneState.SelectedComponents.Num() == 1)
	{
		const bool bIsADynamicMesh = SceneState.SelectedComponents[0]->IsA<UDynamicMeshComponent>();
		return bIsADynamicMesh;
	}
	return false;
	// TODO do a check on the node type
	//if (UDataflowContextObject* const DataflowContextObject = SceneState.ToolManager->GetContextObjectStore()->FindContext<UDataflowContextObject>())
	//{
	//	return DataflowContextObject->GetSelectedNodeOfType<FChaosClothAssetSelectionNode_v2>() != nullptr && (SceneState.TargetManager->CountSelectedAndTargetable(SceneState, GetTargetRequirements()) == 1);
	//}
	//return false;
}

UInteractiveTool* UDataflowMeshSelectionToolBuilder::BuildTool(const FToolBuilderState& SceneState) const
{
	UDataflowMeshSelectionTool* const SelectionTool = NewObject<UDataflowMeshSelectionTool>(SceneState.ToolManager);

	UToolTarget* const Target = SceneState.TargetManager->BuildFirstSelectedTargetable(SceneState, GetTargetRequirements());
	SelectionTool->SetTarget(Target);
	SelectionTool->SetWorld(SceneState.World);

	if (UDataflowSelectionToolNodeDataProvider* DataProvider = NewObject<UDataflowSelectionToolNodeDataProvider>(SelectionTool))
	{
		if (UDataflowBaseContent* ContextObject = SceneState.ToolManager->GetContextObjectStore()->FindContext<UDataflowBaseContent>())
		{
			if (FDataflowSelectionToolNode* Node = ContextObject->GetSelectedNodeOfType<FDataflowSelectionToolNode>())
			{
				if (TSharedPtr<UE::Dataflow::FEngineContext> DataflowContext = ContextObject->GetDataflowContext())
				{
					DataProvider->BindToNode(*Node, *DataflowContext);
					SelectionTool->DataProvider = DataProvider;
				}
			}
		}
	}

	//if (UDataflowContextObject* const DataflowContextObject = SceneState.ToolManager->GetContextObjectStore()->FindContext<UDataflowContextObject>())
	//{
	//	NewTool->SetDataflowContextObject(DataflowContextObject);
	//}

	// TODO : need to set instanciate and set the data provider 
	// the data provider holder a weak ref to the node and contains various cached data 
	// it can also liste to node changes if needed 

	return SelectionTool;
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

void UDataflowMeshSelectionTool::InitializeSculptMeshFromTarget()
{
	using namespace UE::Geometry;

	// (Re-)Create the preview mesh
	if (PreviewMesh)
	{
		PreviewMesh->Disconnect();
	}
	
	PreviewMesh = NewObject<UPreviewMesh>(this, TEXT("PreviewMesh"));
	if (!PreviewMesh)
	{
		return;
	}

	PreviewMesh->CreateInWorld(GetTargetWorld(), FTransform::Identity);
	ToolSetupUtil::ApplyRenderingConfigurationToPreview(PreviewMesh, Target);

	// We will use the preview mesh's spatial data structure
	PreviewMesh->bBuildSpatialDataStructure = true;

	// set materials
	FComponentMaterialSet MaterialSet = UE::ToolTarget::GetMaterialSet(Target);
	PreviewMesh->SetMaterials(MaterialSet.Materials);

	// configure secondary render material for selected triangles
	// NOTE: the material returned by ToolSetupUtil::GetSelectionMaterial has a checkerboard pattern on back faces which makes it hard to use
	UMaterialInterface* Material = LoadObject<UMaterialInterface>(nullptr, TEXT("/MeshModelingToolsetExp/Materials/SculptMaterial"));
	if (Material != nullptr)
	{
		if (UMaterialInstanceDynamic* MatInstance = UMaterialInstanceDynamic::Create(Material, GetToolManager()))
		{
			MatInstance->SetVectorParameterValue(TEXT("Color"), FLinearColor::Yellow);
			PreviewMesh->SetSecondaryRenderMaterial(MatInstance);
		}
	}

	PreviewMesh->SetTangentsMode(EDynamicMeshComponentTangentsMode::AutoCalculated);
	PreviewMesh->UpdatePreview(UE::ToolTarget::GetDynamicMeshCopy(Target));
	PreviewMesh->SetVisible(true);

	// Hide input target mesh
	UE::ToolTarget::HideSourceObject(Target);

	if (DataProvider && PreviewMesh->GetMesh())
	{
		DataProvider->Init(*PreviewMesh->GetMesh());
	}
}

void UDataflowMeshSelectionTool::InitializeMeshSelector()
{
	// Setup mesh selector
	MeshSelector = NewObject<UToolMeshSelector>(this);
	if (MeshSelector)
	{
		auto OnSelectionChangedLambda = [this]() { OnSelectionModified(); };
		MeshSelector->InitialSetup(TargetWorld.Get(), this, OnSelectionChangedLambda);
		MeshSelector->SetMesh(PreviewMesh, FTransform::Identity);
		MeshSelector->SetComponentSelectionMode(EComponentSelectionMode::Vertices);
		MeshSelector->SetSelectionTool(EMeshSelectorTool::Marquee);
		MeshSelector->SetIsEnabled(true);

		if (UPolygonSelectionMechanic* SelectionMechanic = MeshSelector->GetSelectionMechanic())
		{
			SelectionMechanic->SetSelectionDragToolUpdateType(ESelectionDragToolUpdateType::OnTickAndRelease);
			if (SelectionMechanic->Properties)
			{
				SelectionMechanic->Properties->RestoreProperties(this);
				SelectionMechanic->Properties->bCanSelectEdges = false;
				SelectionMechanic->Properties->bSelectVertices = true;
				SelectionMechanic->Properties->bSelectFaces = false;
				SelectionMechanic->Properties->bDisplaySelectionDragToolsControls = true;
				SelectionMechanic->Properties->bDisplayPolygroupReliantControls = false;
				AddToolPropertySource(SelectionMechanic->Properties);

				// make sure we can toggle between them 
				SelectionMechanic->Properties->WatchProperty(SelectionMechanic->Properties->bSelectVertices, [this, SelectionMechanic = SelectionMechanic](bool bSelectVertices) {
					if (bSelectVertices && SelectionMechanic->Properties)
					{
						if (SelectionMechanic->Properties->bSelectFaces == true)
						{
							SelectionMechanic->Properties->bSelectFaces = false;
							SelectionMechanic->SetShowSelectableCorners(true);
							SelectionMechanic->SetShowEdges(false);
							ConvertFaceToVertexSelection();
						}
					}
					});
				SelectionMechanic->Properties->WatchProperty(SelectionMechanic->Properties->bSelectFaces, [this, SelectionMechanic = SelectionMechanic](bool bSelectFaces) {
					if (bSelectFaces && SelectionMechanic->Properties)
					{
						if (SelectionMechanic->Properties->bSelectVertices == true)
						{
							SelectionMechanic->Properties->bSelectVertices = false;
							SelectionMechanic->SetShowSelectableCorners(false);
							SelectionMechanic->SetShowEdges(true);
							ConvertVertexToFaceSelection();
						}
					}
					});
			}
		}
	}
}

void UDataflowMeshSelectionTool::ConvertFaceToVertexSelection()
{
	if (PreviewMesh && MeshSelector)
	{
		if (const UE::Geometry::FDynamicMesh3* Mesh = PreviewMesh->GetMesh())
		{
			if (UPolygonSelectionMechanic* SelectionMechanic = MeshSelector->GetSelectionMechanic())
			{
				FGroupTopologySelection NewSelection;
				const FGroupTopologySelection& Selection = SelectionMechanic->GetActiveSelection();
				for (const int32 GroupId : Selection.SelectedGroupIDs)
				{
					if (Mesh->IsTriangle(GroupId))
					{
						UE::Geometry::FIndex3i Indices = Mesh->GetTriangle(GroupId);
						NewSelection.SelectedCornerIDs.Add(Indices[0]);
						NewSelection.SelectedCornerIDs.Add(Indices[1]);
						NewSelection.SelectedCornerIDs.Add(Indices[2]);
					}
				}
				SelectionMechanic->SetSelection(NewSelection);
			}
		}
	}
}

void UDataflowMeshSelectionTool::ConvertVertexToFaceSelection()
{
	if (PreviewMesh && MeshSelector)
	{
		if (const UE::Geometry::FDynamicMesh3* Mesh = PreviewMesh->GetMesh())
		{
			if (UPolygonSelectionMechanic* SelectionMechanic = MeshSelector->GetSelectionMechanic())
			{
				FGroupTopologySelection NewSelection;
				const FGroupTopologySelection& Selection = SelectionMechanic->GetActiveSelection();
				for (const int32 TriangleId : Mesh->TriangleIndicesItr())
				{
					if (Mesh->IsTriangle(TriangleId))
					{
						UE::Geometry::FIndex3i Indices = Mesh->GetTriangle(TriangleId);
						if (Selection.SelectedCornerIDs.Contains(Indices[0]) &&
							Selection.SelectedCornerIDs.Contains(Indices[1]) &&
							Selection.SelectedCornerIDs.Contains(Indices[2]))
						{
							NewSelection.SelectedGroupIDs.Add(TriangleId);
						}
					}
				}
				SelectionMechanic->SetSelection(NewSelection);
			}
		}
	}
}

void UDataflowMeshSelectionTool::Setup()
{
	using namespace UE::Geometry;

	InitializeSculptMeshFromTarget();

	InitializeMeshSelector();

	LoadFromDataProvider();

	bAnyChangeMade = false;
}

void UDataflowMeshSelectionTool::OnSelectionModified()
{
	bAnyChangeMade = true;
}

void UDataflowMeshSelectionTool::OnShutdown(EToolShutdownType ShutdownType)
{
	if (ShutdownType == EToolShutdownType::Accept && CanAccept())
	{
		SaveToDataProvider();
	}

	if (DataProvider)
	{
		DataProvider->Shutdown();
	}

	if (MeshSelector)
	{
		if (UPolygonSelectionMechanic* SelectionMechanic = MeshSelector->GetSelectionMechanic())
		{
			if (SelectionMechanic->Properties)
			{
				SelectionMechanic->Properties->SaveProperties(this);
			}
		}

		MeshSelector->Shutdown();
		MeshSelector = nullptr;
	}

	if (PreviewMesh != nullptr)
	{
		UE::ToolTarget::ShowSourceObject(Target);
		PreviewMesh->Disconnect();
		PreviewMesh = nullptr;
	}
}

void UDataflowMeshSelectionTool::LoadFromDataProvider()
{
	if (MeshSelector && DataProvider)
	{
		if (UPolygonSelectionMechanic* SelectionMechanic = MeshSelector->GetSelectionMechanic())
		{
			FGroupTopologySelection Selection;
			DataProvider->GetVertexSelection(Selection.SelectedCornerIDs);

			constexpr bool bBroadcastChange = false;
			SelectionMechanic->SetSelection(Selection, bBroadcastChange);
		}
	}
}

void UDataflowMeshSelectionTool::SaveToDataProvider()
{
	if (MeshSelector && DataProvider)
	{
		if (UPolygonSelectionMechanic* SelectionMechanic = MeshSelector->GetSelectionMechanic())
		{
			GetToolManager()->BeginUndoTransaction(LOCTEXT("SelectionToolTransactionName", "Mesh Selection"));

			const FGroupTopologySelection& Selection = SelectionMechanic->GetActiveSelection();
			DataProvider->SetVertexSelection(Selection.SelectedCornerIDs);
			GetToolManager()->EndUndoTransaction();
		}
	}
}

void UDataflowMeshSelectionTool::Render(IToolsContextRenderAPI* RenderAPI)
{
	USingleSelectionMeshEditingTool::Render(RenderAPI);
	if (MeshSelector)
	{
		MeshSelector->Render(RenderAPI);
	}
}

void UDataflowMeshSelectionTool::DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI)
{
	USingleSelectionMeshEditingTool::DrawHUD(Canvas, RenderAPI);
	if (MeshSelector)
	{
		MeshSelector->DrawHUD(Canvas, RenderAPI);

		if (Canvas && RenderAPI)
		{
			if (UPolygonSelectionMechanic* SelectionMechanic = MeshSelector->GetSelectionMechanic())
			{
				const UE::Geometry::FGroupTopologySelection& Selection = SelectionMechanic->GetActiveSelection();

				const UE::Geometry::FDynamicMesh3* Mesh = PreviewMesh ? PreviewMesh->GetPreviewDynamicMesh() : nullptr;

				const FText BaseText = LOCTEXT("UClothMeshSelectionTool_Info", "Selected : Vertices: {0} / {1} - Triangles: {2} / {3}");
				const FText InfoText = FText::Format(BaseText,
					FText::AsNumber(Selection.SelectedCornerIDs.Num()),
					FText::AsNumber(Mesh ? Mesh->VertexCount() : 0),
					FText::AsNumber(Selection.SelectedGroupIDs.Num()),
					FText::AsNumber(Mesh ? Mesh->TriangleCount() : 0)
				);
				const FVector2D TextPosition = { 0.5f * Canvas->GetViewRect().Width() * Canvas->GetDPIScale(), 10 };
				FCanvasTextItem TextItem(TextPosition, InfoText, GEngine->GetLargeFont(), FLinearColor::White);
				TextItem.EnableShadow(FLinearColor::Black);
				TextItem.bCentreX = true;
				TextItem.Scale = FVector2D(1.2);
				Canvas->DrawItem(TextItem);
			}
		}
	}
}

void UDataflowMeshSelectionTool::OnTick(float DeltaTime)
{
	USingleSelectionMeshEditingTool::OnTick(DeltaTime);

	if (MeshSelector)
	{
		MeshSelector->Tick(DeltaTime);
	}

	//if (bHavePendingAction)
	//{
	//	ApplyAction(PendingAction);
	//	bHavePendingAction = false;
	//	PendingAction = EClothMeshSelectionToolActions::NoAction;
	//}
}

bool UDataflowMeshSelectionTool::CanAccept() const
{
	return bAnyChangeMade;
}

FBox UDataflowMeshSelectionTool::GetWorldSpaceFocusBox()
{
	if (MeshSelector)
	{
		if (UPolygonSelectionMechanic* SelectionMechanic = MeshSelector->GetSelectionMechanic())
		{
			static constexpr bool bWorld = true;
			return FBox(SelectionMechanic->GetSelectionBounds(bWorld));
		}
	}
	return FBox(EForceInit::ForceInitToZero);
}


void UDataflowMeshSelectionTool::NotifyTargetChanged()
{
	InitializeSculptMeshFromTarget();

	if (MeshSelector)
	{
		MeshSelector->SetMesh(PreviewMesh, FTransform::Identity);
	}
	GetToolManager()->PostInvalidation();
}
//
#undef LOCTEXT_NAMESPACE
