// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseTools/SingleSelectionMeshEditingTool.h"
#include "DataflowEditorTools/DataflowDynamicMeshMapping.h"
#include "Dataflow/DataflowSelectionToolNode.h"
#include "UObject/ObjectPtr.h"

// builder includes
#include "DataflowEditorTools/DataflowEditorToolBuilder.h"
#include "InteractiveToolBuilder.h"

#include "DataflowMeshSelectionTool.generated.h"

#define UE_API DATAFLOWEDITOR_API

class UPreviewMesh;
class UToolMeshSelector;

struct FDataflowSelectionToolNode;

namespace UE::Dataflow { class FContext; }

UINTERFACE(MinimalAPI)
class UDataflowMeshSelectionToolDataProviderInterface : public UInterface
{
	GENERATED_BODY()
};

class IDataflowMeshSelectionToolDataProviderInterface
{
	GENERATED_BODY()

public:
	virtual void Init(const UE::Geometry::FDynamicMesh3& Mesh) = 0;
	virtual void Shutdown() = 0;

	// Face and edge selection are infered from the vertex selection
	virtual void GetVertexSelection(TSet<int32>& OutSelection) const = 0;
	virtual void SetVertexSelection(const TSet<int32>& InSelection) = 0;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

UCLASS(MinimalAPI)
class UDataflowSelectionToolNodeDataProvider
	: public UObject
	, public IDataflowMeshSelectionToolDataProviderInterface
{
	GENERATED_BODY()

public:
	void BindToNode(FDataflowSelectionToolNode& InNode, UE::Dataflow::FContext& Context);

	//~ Begin IDataflowMeshSelectionToolDataProviderInterface 
	virtual void Init(const UE::Geometry::FDynamicMesh3& Mesh) override;
	virtual void Shutdown() override;
	virtual void GetVertexSelection(TSet<int32>& OutSelection) const override;
	virtual void SetVertexSelection(const TSet<int32>& InSelection) override;
	//~ End IDataflowMeshSelectionToolDataProviderInterface 

private:
	TWeakPtr<FDataflowSelectionToolNode> WeakNode;
	UE::Dataflow::FDynamicMeshMapping MeshMapping;
	FDataflowSelectionToolNodeData NodeData;
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////

UCLASS(MinimalAPI)
class UDataflowToolNodeSnapshotProvider	: public UObject
{
	GENERATED_BODY()

public:
	void BindToNode(FDataflowToolNode& InNode);

	TConstArrayView<FDataflowToolNodeSnapshot> GetSnapshots() const;
	void RemoveSnapshot();
	void MoveSnapshot(int32 FromPosition, int32 ToPosition);

private:
	TWeakPtr<FDataflowToolNode> WeakNode;
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////

class UDataflowContextObject;

namespace UE::Dataflow
{
	class IDataflowConstructionViewMode;
}

UCLASS(MinimalAPI)
class UDataflowMeshSelectionToolBuilder : public UInteractiveToolWithToolTargetsBuilder, public IDataflowEditorToolBuilder
{
	GENERATED_BODY()

private:
	// IDataflowEditorToolBuilder
	UE_API virtual void GetSupportedConstructionViewModes(const UDataflowContextObject& ContextObject, TArray<const UE::Dataflow::IDataflowConstructionViewMode*>& Modes) const override;
	UE_API virtual bool CanSceneStateChange(const UInteractiveTool* ActiveTool, const FToolBuilderState& SceneState) const override;
	UE_API virtual void SceneStateChanged(UInteractiveTool* ActiveTool, const FToolBuilderState& SceneState) override;

	// UInteractiveToolWithToolTargetsBuilder
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual UInteractiveTool* BuildTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual const FToolTargetTypeRequirements& GetTargetRequirements() const override;
};

UCLASS(MinimalAPI)
class UDataflowMeshSelectionTool : public USingleSelectionMeshEditingTool
{
	GENERATED_BODY()

private:

	friend class UDataflowMeshSelectionToolBuilder;

	//~ Begin UInteractiveTool interface
	virtual void Setup() override;
	virtual void OnShutdown(EToolShutdownType ShutdownType) override;
	
	virtual void Render(IToolsContextRenderAPI* RenderAPI) override;
	virtual void DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI) override;
	virtual void OnTick(float DeltaTime) override;

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }
	virtual bool CanAccept() const override;
	//~ End UInteractiveTool interface

	//~ Begin IInteractiveToolCameraFocusAPI interface
	virtual FBox GetWorldSpaceFocusBox() override;
	//~ End IInteractiveToolCameraFocusAPI interface

	void OnSelectionModified();

public:
	UE_API void NotifyTargetChanged();

	UPROPERTY()
	TObjectPtr<UPreviewMesh> PreviewMesh = nullptr;

	UPROPERTY()
	TObjectPtr<UToolMeshSelector> MeshSelector = nullptr;

	UPROPERTY(Transient)
	TScriptInterface<IDataflowMeshSelectionToolDataProviderInterface> DataProvider = nullptr;

private:
	void InitializeSculptMeshFromTarget();
	void InitializeMeshSelector();

	void LoadFromDataProvider();
	void SaveToDataProvider();

	void ConvertFaceToVertexSelection();
	void ConvertVertexToFaceSelection();

	bool bAnyChangeMade = false;
};

#undef UE_API
