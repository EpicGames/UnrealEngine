// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataflowEditorTools/DataflowPropertyGizmoManager.h"

#include "BaseGizmos/CombinedTransformGizmo.h"
#include "BaseGizmos/GizmoElementHitTargets.h"
#include "BaseGizmos/TransformGizmoUtil.h"
#include "BaseGizmos/TransformProxy.h"
#include "Dataflow/DataflowEdNode.h"
#include "Dataflow/DataflowNode.h"
#include "DataflowEditorTools/DataflowGizmoModeSource.h"
#include "EditConditionEvaluator.h"
#include "EditorGizmos/EditorTransformGizmoUtil.h"
#include "EditorGizmos/TransformGizmo.h"
#include "EditorInteractiveGizmoManager.h"
#include "Engine/Canvas.h"
#include "Engine/Engine.h"
#include "InteractiveGizmoManager.h"
#include "InteractiveToolManager.h"
#include "SceneView.h"
#include "UObject/Class.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowPropertyGizmoManager)

//------------------------------------------------------------------------------
// Setup / Teardown
//------------------------------------------------------------------------------

void UDataflowPropertyGizmoManager::Setup(UInteractiveToolManager* InToolManager, UDataflowEdNode* InEdNode)
{
	if (!ensure(InToolManager) || !ensure(InEdNode))
	{
		return;
	}

	TrackedEdNode = InEdNode;

	TSharedPtr<FDataflowNode> DataflowNode = InEdNode->GetDataflowNode();
	if (!DataflowNode.IsValid())
	{
		return;
	}

	const UScriptStruct* ScriptStruct = DataflowNode->TypedScriptStruct();
	if (!ScriptStruct)
	{
		return;
	}

	void* const NodeMemory = DataflowNode.Get();

	for (TFieldIterator<FProperty> PropIt(ScriptStruct, EFieldIteratorFlags::IncludeSuper); PropIt; ++PropIt)
	{
		FProperty* Property = *PropIt;
		if (!Property->HasMetaData(TEXT("GizmoType")))
		{
			continue;
		}

		if (!IsPropertyEditable(*DataflowNode, *Property))
		{
			continue;
		}

		// set a potential new entry
		const int32 EntryIndex = TransformProxies.Num();
		FPropertyGizmoEntry Entry;
		Entry.Property = Property;
		Entry.ArrayIndex = EntryIndex;

		const FString GizmoTypeStr = Property->GetMetaData(TEXT("GizmoType"));
		Entry.GizmoType = GizmoTypeFromString(GizmoTypeStr);
		const ETransformGizmoSubElements SubElements = GizmoTypeToSubElements(Entry.GizmoType);
		if (SubElements == ETransformGizmoSubElements::None)
		{
			continue;
		}

		Entry.PositionProperty = nullptr;
		if (Entry.GizmoType != EGizmoType::Transform && Entry.GizmoType != EGizmoType::Translate)
		{
			const FString GizmoPositionStr = Property->GetMetaData(TEXT("GizmoPosition"));
			if (!GizmoPositionStr.IsEmpty())
			{
				Entry.PositionProperty = ScriptStruct->FindPropertyByName(FName(GizmoPositionStr));
			}
		}

		const FTransform InitialTransform = ReadPropertyAsTransform(Entry, NodeMemory);
		ensureAlways(!InitialTransform.ContainsNaN());

		UTransformProxy* Proxy = NewObject<UTransformProxy>(this);
		Proxy->SetTransform(InitialTransform);
		Proxy->OnEndTransformEdit.AddUObject(this, &UDataflowPropertyGizmoManager::OnEndTransformEdit);
		TransformProxies.Add(Proxy);

		UCombinedTransformGizmo*  LegacyGizmo = nullptr;
		UTransformGizmo*          TRSGizmo    = nullptr;
		UDataflowGizmoModeSource* ModeSource  = nullptr;

		const bool bUseNewTRS = UEditorInteractiveGizmoManager::UsesNewTRSGizmos();
		if (bUseNewTRS)
		{
			TRSGizmo = UE::EditorTransformGizmoUtil::CreateTransformGizmo(InToolManager, FString(), this);
		}

		if (TRSGizmo)
		{
			// Single-mode gizmos lock to their EGizmoTransformMode; "Transform" gizmos follow the editor mode toolbar.
			if (Entry.GizmoType != EGizmoType::Transform)
			{
				ModeSource = NewObject<UDataflowGizmoModeSource>(this);
				ModeSource->FixedMode = GizmoTypeToTransformMode(Entry.GizmoType);
				TRSGizmo->TransformGizmoSource = ModeSource;
			}
			else
			{
				TRSGizmo->TransformGizmoSource = nullptr;
			}

			TRSGizmo->SetActiveTarget(Proxy, InToolManager->GetPairedGizmoManager());
			if (UGizmoElementHitMultiTarget* HitMultiTarget = Cast<UGizmoElementHitMultiTarget>(TRSGizmo->HitTarget))
			{
				HitMultiTarget->GizmoTransformProxy = Proxy;
			}
			TRSGizmo->SetVisibility(true);
		}
		else
		{
			LegacyGizmo = UE::TransformGizmoUtil::CreateCustomTransformGizmo(InToolManager, SubElements, this);
			LegacyGizmo->SetActiveTarget(Proxy, InToolManager->GetPairedGizmoManager());
			LegacyGizmo->ReinitializeGizmoTransform(InitialTransform, /*bKeepGizmoUnscaled=*/true);
			LegacyGizmo->SetVisibility(true);
			LegacyGizmo->ActiveGizmoMode = EToolContextTransformGizmoMode::Combined;
			LegacyGizmo->bUseContextCoordinateSystem = false;
			LegacyGizmo->bUseContextGizmoMode = false;
			LegacyGizmo->CurrentCoordinateSystem = EToolContextCoordinateSystem::Local;
		}

		TransformGizmos.Add(LegacyGizmo);
		TRSGizmos.Add(TRSGizmo);
		GizmoModeSources.Add(ModeSource);

		PropertyEntries.Add(Entry);
	}
}

void UDataflowPropertyGizmoManager::Teardown(UInteractiveToolManager* InToolManager)
{
	for (UTransformProxy* Proxy : TransformProxies)
	{
		if (Proxy)
		{
			Proxy->OnEndTransformEdit.RemoveAll(this);
		}
	}

	if (ensure(InToolManager))
	{
		InToolManager->GetPairedGizmoManager()->DestroyAllGizmosByOwner(this);
	}

	TransformGizmos.Reset();
	TRSGizmos.Reset();
	GizmoModeSources.Reset();
	TransformProxies.Reset();
	PropertyEntries.Reset();
	TrackedEdNode.Reset();
}

//------------------------------------------------------------------------------
// Sync from external property changes
//------------------------------------------------------------------------------

void UDataflowPropertyGizmoManager::RefreshGizmo(const FPropertyGizmoEntry& Entry, const TSharedPtr<const FDataflowNode>& Node)
{
	if (!TransformProxies.IsValidIndex(Entry.ArrayIndex) || !TransformProxies[Entry.ArrayIndex])
	{
		return;
	}

	const FTransform NewTransform = ReadPropertyAsTransform(Entry, Node.Get());
	ensureAlways(!NewTransform.ContainsNaN());
	TransformProxies[Entry.ArrayIndex]->SetTransform(NewTransform);

	if (TRSGizmos.IsValidIndex(Entry.ArrayIndex) && TRSGizmos[Entry.ArrayIndex])
	{
		TRSGizmos[Entry.ArrayIndex]->SetVisibility(true);
	}
	else if (TransformGizmos.IsValidIndex(Entry.ArrayIndex) && TransformGizmos[Entry.ArrayIndex])
	{
		TransformGizmos[Entry.ArrayIndex]->SetVisibility(true);
		TransformGizmos[Entry.ArrayIndex]->ActiveGizmoMode = EToolContextTransformGizmoMode::Combined;
		TransformGizmos[Entry.ArrayIndex]->ReinitializeGizmoTransform(NewTransform, /*bKeepGizmoUnscaled=*/true);
	}
}

void UDataflowPropertyGizmoManager::SyncGizmosFromNodeProperties()
{
	const UDataflowEdNode* EdNode = TrackedEdNode.Get();
	if (!EdNode)
	{
		return;
	}

	TSharedPtr<const FDataflowNode> DataflowNode = EdNode->GetDataflowNode();
	if (!DataflowNode.IsValid())
	{
		return;
	}

	for (const FPropertyGizmoEntry& Entry : PropertyEntries)
	{
		RefreshGizmo(Entry, DataflowNode);
	}
}

//------------------------------------------------------------------------------
// Gizmo delegates
//------------------------------------------------------------------------------
void UDataflowPropertyGizmoManager::OnEndTransformEdit(UTransformProxy* Proxy)
{
	UDataflowEdNode* EdNode = TrackedEdNode.Get();
	if (!EdNode)
	{
		return;
	}

	TSharedPtr<FDataflowNode> DataflowNode = EdNode->GetDataflowNode();
	if (!DataflowNode.IsValid())
	{
		return;
	}

	for (const FPropertyGizmoEntry& Entry : PropertyEntries)
	{
		if (TransformProxies.IsValidIndex(Entry.ArrayIndex) && TransformProxies[Entry.ArrayIndex] == Proxy)
		{
			EdNode->Modify();
			ensureAlways(!Proxy->GetTransform().ContainsNaN());
			WriteTransformToProperty(Entry, DataflowNode.Get(), Proxy->GetTransform());

			// make sure we update the dependent gizmos 
			for (const FPropertyGizmoEntry& DependentEntry : PropertyEntries)
			{
				if (DependentEntry.PositionProperty == Entry.Property)
				{
					RefreshGizmo(DependentEntry, DataflowNode);
				}
			}

			DataflowNode->Invalidate();
			break;
		}
	}
}

//------------------------------------------------------------------------------
// Property/FTransform conversion helpers
//------------------------------------------------------------------------------
FTransform UDataflowPropertyGizmoManager::ReadPropertyAsTransform(const FPropertyGizmoEntry& Entry, const void* NodeMemory) const
{
	FTransform Transform = ReadPropertyAsTransform(Entry.Property, Entry.GizmoType, NodeMemory);
	ensureAlways(!Transform.ContainsNaN());

	if (Entry.PositionProperty && Entry.GizmoType != EGizmoType::Transform && Entry.GizmoType != EGizmoType::Translate)
	{
		FPropertyGizmoEntry PositionEntry{ Entry };
		PositionEntry.Property = Entry.PositionProperty;
		PositionEntry.PositionProperty = nullptr;
		PositionEntry.GizmoType = EGizmoType::Translate;
		FTransform PositionTransform = ReadPropertyAsTransform(PositionEntry, NodeMemory);
		Transform.SetTranslation(PositionTransform.GetTranslation());
		ensureAlways(!Transform.ContainsNaN());
	}
	return Transform;
}

FTransform UDataflowPropertyGizmoManager::ReadPropertyAsTransform(const FProperty* Property, const EGizmoType GizmoType, const void* NodeMemory) const
{
	if (const FStructProperty* StructProp = CastField<const FStructProperty>(Property))
	{
		const void* PropAddr = StructProp->ContainerPtrToValuePtr<void>(NodeMemory);

		if (StructProp->Struct == TBaseStructure<FTransform>::Get())
		{
			const FTransform Value = *static_cast<const FTransform*>(PropAddr);
			if (GizmoType == EGizmoType::Transform)
			{
				return Value;
			}
			else if (GizmoType == EGizmoType::Translate)
			{
				return FTransform(FQuat::Identity, Value.GetTranslation(), FVector::OneVector);
			}
			else if (GizmoType == EGizmoType::Rotate)
			{
				return FTransform(Value.GetRotation(), FVector::ZeroVector, FVector::OneVector);
			}
			else if (GizmoType == EGizmoType::Scale)
			{
				return FTransform(FQuat::Identity, FVector::ZeroVector, Value.GetScale3D());
			}
		}
		if (StructProp->Struct == TBaseStructure<FVector>::Get())
		{
			const FVector Value = *static_cast<const FVector*>(PropAddr);
			if (GizmoType == EGizmoType::Translate)
			{
				return FTransform(FQuat::Identity, Value, FVector::OneVector);
			}
			else if (GizmoType == EGizmoType::Rotate)
			{
				return FTransform(FQuat::MakeFromEuler(Value), FVector::ZeroVector, FVector::OneVector);
			}
			else if (GizmoType == EGizmoType::Scale)
			{
				return FTransform(FQuat::Identity, FVector::ZeroVector, Value);
			}
		}
		if (StructProp->Struct == TVariantStructure<FVector3f>::Get())
		{
			const FVector Value = FVector(*static_cast<const FVector3f*>(PropAddr));
			if (GizmoType == EGizmoType::Translate)
			{
				return FTransform(FQuat::Identity, Value, FVector::OneVector);
			}
			else if (GizmoType == EGizmoType::Rotate)
			{
				return FTransform(FQuat::MakeFromEuler(Value), FVector::ZeroVector, FVector::OneVector);
			}
			else if (GizmoType == EGizmoType::Scale)
			{
				return FTransform(FQuat::Identity, FVector::ZeroVector, Value);
			}
		}
		if (StructProp->Struct == TBaseStructure<FQuat>::Get())
		{
			if (GizmoType == EGizmoType::Rotate)
			{
				return FTransform(*static_cast<const FQuat*>(PropAddr));
			}
		}
		if (StructProp->Struct == TBaseStructure<FRotator>::Get())
		{
			if (GizmoType == EGizmoType::Rotate)
			{
				return FTransform(static_cast<const FRotator*>(PropAddr)->Quaternion());
			}
		}
	}
	else if (const FFloatProperty* FloatProp = CastField<const FFloatProperty>(Property))
	{
		if (GizmoType == EGizmoType::Scale)
		{
			const float Scale = *FloatProp->ContainerPtrToValuePtr<float>(NodeMemory);
			return FTransform(FQuat::Identity, FVector::ZeroVector, FVector(Scale));
		}
	}
	else if (const FDoubleProperty* DoubleProp = CastField<const FDoubleProperty>(Property))
	{
		if (GizmoType == EGizmoType::Scale)
		{
			const double Scale = *DoubleProp->ContainerPtrToValuePtr<double>(NodeMemory);
			return FTransform(FQuat::Identity, FVector::ZeroVector, FVector(Scale));
		}
	}
	return FTransform::Identity;
}

void UDataflowPropertyGizmoManager::WriteTransformToProperty(const FPropertyGizmoEntry& Entry, void* NodeMemory, const FTransform& NewTransform) const
{
	if (const FStructProperty* StructProp = CastField<const FStructProperty>(Entry.Property))
	{
		void* PropAddr = StructProp->ContainerPtrToValuePtr<void>(NodeMemory);

		if (StructProp->Struct == TBaseStructure<FTransform>::Get())
		{
			FTransform& ValueToSet = *static_cast<FTransform*>(PropAddr);
			if (Entry.GizmoType == EGizmoType::Transform)
			{
				ValueToSet = NewTransform;
			}
			else if (Entry.GizmoType == EGizmoType::Translate)
			{
				ValueToSet.SetTranslation(NewTransform.GetTranslation());
			}
			else if (Entry.GizmoType == EGizmoType::Rotate)
			{
				ValueToSet.SetRotation(NewTransform.GetRotation());
			}
			else if (Entry.GizmoType == EGizmoType::Scale)
			{
				ValueToSet.SetScale3D(NewTransform.GetScale3D());
			}
		}
		else if (StructProp->Struct == TBaseStructure<FVector>::Get())
		{
			FVector& ValueToSet = *static_cast<FVector*>(PropAddr);
			if (Entry.GizmoType == EGizmoType::Translate)
			{
				ValueToSet = NewTransform.GetTranslation();
			}
			else if (Entry.GizmoType == EGizmoType::Rotate)
			{
				ValueToSet = NewTransform.GetRotation().Rotator().Euler();
			}
			else if (Entry.GizmoType == EGizmoType::Scale)
			{
				ValueToSet = NewTransform.GetScale3D();
			}
		}
		else if (StructProp->Struct == TVariantStructure<FVector3f>::Get())
		{
			FVector3f& ValueToSet = *static_cast<FVector3f*>(PropAddr);
			if (Entry.GizmoType == EGizmoType::Translate)
			{
				ValueToSet = FVector3f(NewTransform.GetTranslation());
			}
			if (Entry.GizmoType == EGizmoType::Rotate)
			{
				ValueToSet = FVector3f(NewTransform.GetRotation().Rotator().Euler());
			}
			if (Entry.GizmoType == EGizmoType::Scale)
			{
				ValueToSet = FVector3f(NewTransform.GetScale3D());
			}
		}
		else if (StructProp->Struct == TBaseStructure<FQuat>::Get())
		{
			if (Entry.GizmoType == EGizmoType::Rotate)
			{
				*static_cast<FQuat*>(PropAddr) = NewTransform.GetRotation();
			}
		}
		else if (StructProp->Struct == TBaseStructure<FRotator>::Get())
		{
			if (Entry.GizmoType == EGizmoType::Rotate)
			{
				*static_cast<FRotator*>(PropAddr) = NewTransform.GetRotation().Rotator();
			}
		}
	}
	else if (const FFloatProperty* FloatProp = CastField<const FFloatProperty>(Entry.Property))
	{
		if (Entry.GizmoType == EGizmoType::Scale)
		{
			*FloatProp->ContainerPtrToValuePtr<float>(NodeMemory) = (float)NewTransform.GetScale3D().X;
		}
	}
	else if (const FDoubleProperty* DoubleProp = CastField<const FDoubleProperty>(Entry.Property))
	{
		if (Entry.GizmoType == EGizmoType::Scale)
		{
			*DoubleProp->ContainerPtrToValuePtr<double>(NodeMemory) = NewTransform.GetScale3D().X;
		}
	}
}

//------------------------------------------------------------------------------
// Canvas text overlay
//------------------------------------------------------------------------------

void UDataflowPropertyGizmoManager::DrawPropertyLabels(FCanvas* Canvas, const FSceneView* SceneView) const
{
#if WITH_EDITOR
	if (!Canvas || !SceneView)
	{
		return;
	}
	const float DPIScale = Canvas->GetDPIScale();
	for (const FPropertyGizmoEntry& Entry : PropertyEntries)
	{
		if (!TransformProxies.IsValidIndex(Entry.ArrayIndex) || !TransformProxies[Entry.ArrayIndex])
		{
			continue;
		}
		const FVector Origin = TransformProxies[Entry.ArrayIndex]->GetTransform().GetTranslation();
		FVector2D PixelLocation;
		if (SceneView->WorldToPixel(Origin, PixelLocation))
		{
			const FString LabelText = Entry.Property->GetDisplayNameText().ToString();
			FCanvasTextItem TextItem(PixelLocation / DPIScale, FText::FromString(LabelText), GEngine->GetSmallFont(), FLinearColor::White);
			TextItem.Scale = FVector2D::UnitVector;
			TextItem.EnableShadow(FLinearColor::Black);
			TextItem.Draw(Canvas);
		}
	}
#endif
}

/*static*/ bool UDataflowPropertyGizmoManager::IsPropertyEditable(const FDataflowNode& DataflowNode, const FProperty& Property)
{
	// connected inputs are not be hand editable, because they are driven by upstreamn node output
	if (const FDataflowInput* Input = DataflowNode.FindInput(Property.GetFName()))
	{
		if (Input->IsConnected())
		{
			return false;
		}
	}

	return EditConditionEvaluator::IsPropertyEditable(&Property, DataflowNode.TypedScriptStruct(), &DataflowNode);
}

/*static*/ UDataflowPropertyGizmoManager::EGizmoType UDataflowPropertyGizmoManager::GizmoTypeFromString(const FString& GizmoTypeStr)
{
	if (GizmoTypeStr == TEXT("Translate"))
	{
		return EGizmoType::Translate;
	}
	if (GizmoTypeStr == TEXT("Rotate"))
	{
		return EGizmoType::Rotate;
	}
	if (GizmoTypeStr == TEXT("Scale"))
	{
		return EGizmoType::Scale;
	}
	if (GizmoTypeStr == TEXT("Transform"))
	{
		return EGizmoType::Transform;
	}
	return EGizmoType::None;
}

/*static*/ EGizmoTransformMode UDataflowPropertyGizmoManager::GizmoTypeToTransformMode(const EGizmoType GizmoType)
{
	switch (GizmoType)
	{
	case EGizmoType::Translate: return EGizmoTransformMode::Translate;
	case EGizmoType::Rotate:    return EGizmoTransformMode::Rotate;
	case EGizmoType::Scale:     return EGizmoTransformMode::Scale;
	default:                    return EGizmoTransformMode::None;
	}
}

/*static*/ ETransformGizmoSubElements UDataflowPropertyGizmoManager::GizmoTypeToSubElements(const EGizmoType GizmoType)
{
	switch (GizmoType)
	{
	case EGizmoType::Translate:
	{
		return ETransformGizmoSubElements::TranslateAllAxes | ETransformGizmoSubElements::TranslateAllPlanes;
	}
	case EGizmoType::Rotate:
	{
		return ETransformGizmoSubElements::RotateAllAxes | ETransformGizmoSubElements::FreeRotate;
	}
	case EGizmoType::Scale:
	{
		return ETransformGizmoSubElements::ScaleAllAxes | ETransformGizmoSubElements::ScaleAllPlanes | ETransformGizmoSubElements::ScaleUniform;
	}
	case EGizmoType::Transform:
	{
		return ETransformGizmoSubElements::TranslateAllAxes | ETransformGizmoSubElements::TranslateAllPlanes
			| ETransformGizmoSubElements::RotateAllAxes
			| ETransformGizmoSubElements::ScaleAllAxes | ETransformGizmoSubElements::ScaleAllPlanes | ETransformGizmoSubElements::ScaleUniform;
	}
	}
	return ETransformGizmoSubElements::None;
}
