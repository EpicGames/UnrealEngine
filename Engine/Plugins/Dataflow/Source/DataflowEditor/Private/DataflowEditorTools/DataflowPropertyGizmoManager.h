// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InteractiveGizmo.h"
#include "DataflowPropertyGizmoManager.generated.h"

#define UE_API DATAFLOWEDITOR_API

struct FDataflowNode;

enum class EGizmoTransformMode : uint8;

class FCanvas;
class FSceneView;
class UCombinedTransformGizmo;
class UDataflowEdNode;
class UDataflowGizmoModeSource;
class UInteractiveToolManager;
class UTransformGizmo;
class UTransformProxy;

/**
 * Manages viewport gizmos for UPROPERTY-annotated properties on a selected dataflow node.
 *
 * Node authors opt in by adding meta=(GizmoType="Translate"|"Rotate"|"Scale"|"Transform") to a UPROPERTY.
 * When a node is selected in the graph, Setup() creates one gizmo per annotated property.
 * Gizmo manipulations write back to the node property and invalidate the node for re-evaluation.
 *
 * Example usage in a node struct:
 *   UPROPERTY(EditAnywhere, Category="Transform", meta=(GizmoType="Translate"))
 *   FVector Origin = FVector::ZeroVector;
 *
 *   UPROPERTY(EditAnywhere, Category="Transform", meta=(GizmoType="Rotate"))
 *   FQuat Orientation = FQuat::Identity;
 */
UCLASS(MinimalAPI)
class UDataflowPropertyGizmoManager : public UObject
{
	GENERATED_BODY()

	enum class EGizmoType
	{
		None,
		Translate,
		Rotate,
		Scale,
		Transform,
	};

public:

	/** Create gizmos for all GizmoType-annotated properties on the given node. Call when a node is selected. */
	UE_API void Setup(UInteractiveToolManager* ToolManager, UDataflowEdNode* EdNode);

	/** Destroy all active gizmos. Call when the node is deselected or the editor closes. */
	UE_API void Teardown(UInteractiveToolManager* ToolManager);

	/** Sync gizmo world transforms to match the current node property values.
	 *  Call after an external property edit (e.g. via the details panel). */
	UE_API void SyncGizmosFromNodeProperties();

	bool HasActiveGizmos() const { return TransformProxies.Num() > 0; }

	/** Draw the property name as a text overlay at each gizmo's world-space origin. Call from viewport DrawCanvas. */
	UE_API void DrawPropertyLabels(FCanvas* Canvas, const FSceneView* SceneView) const;

private:

	/** Parallel-array entry tying a reflected property to its gizmo/proxy indices. */
	struct FPropertyGizmoEntry
	{
		EGizmoType GizmoType = EGizmoType::None;
		FProperty* Property = nullptr;
		FProperty* PositionProperty = nullptr;
		int32 ArrayIndex = INDEX_NONE;
	};

	FTransform ReadPropertyAsTransform(const FPropertyGizmoEntry& Entry, const void* NodeMemory) const;
	FTransform ReadPropertyAsTransform(const FProperty* Property, const EGizmoType GizmoType, const void* NodeMemory) const;
	void WriteTransformToProperty(const FPropertyGizmoEntry& Entry, void* NodeMemory, const FTransform& NewTransform) const;

	void RefreshGizmo(const FPropertyGizmoEntry& Entry, const TSharedPtr<const FDataflowNode>& Node);

	static EGizmoType GizmoTypeFromString(const FString& GizmoTypeStr);
	static ETransformGizmoSubElements GizmoTypeToSubElements(const EGizmoType GizmoTypeStr);
	static EGizmoTransformMode GizmoTypeToTransformMode(const EGizmoType GizmoType);
	static bool IsPropertyEditable(const FDataflowNode& DataflowNode, const FProperty& Property);

	void OnEndTransformEdit(UTransformProxy* Proxy);

	TArray<FPropertyGizmoEntry> PropertyEntries;

	UPROPERTY(Transient)
	TArray<TObjectPtr<UTransformProxy>> TransformProxies;

	UPROPERTY(Transient)
	TArray<TObjectPtr<UCombinedTransformGizmo>> TransformGizmos;

	UPROPERTY(Transient)
	TArray<TObjectPtr<UTransformGizmo>> TRSGizmos;

	UPROPERTY(Transient)
	TArray<TObjectPtr<UDataflowGizmoModeSource>> GizmoModeSources;

	TWeakObjectPtr<UDataflowEdNode> TrackedEdNode;
};

#undef UE_API
