// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataflowRendering/DataflowBoxRenderableType.h"
#include "DataflowRendering/DataflowRenderableTypeUtils.h"

#include "Components/BoxComponent.h"
#include "Components/InstancedStaticMeshComponent.h"

#include "Dataflow/DataflowRenderingViewMode.h"
#include "Dataflow/DataflowEngineUtil.h"

#include "DataflowRendering/DataflowRenderableComponents.h"
#include "DataflowRendering/DataflowRenderableType.h"
#include "DataflowRendering/DataflowRenderableTypeInstance.h"
#include "DataflowRendering/DataflowRenderableTypeRegistry.h"
#include "DataflowRendering/DataflowRenderableTypeUtils.h"

#include "Engine/StaticMesh.h"

#include "UObject/ObjectPtr.h"

#include "Materials/MaterialInstanceDynamic.h"

namespace UE::Dataflow::Private
{
	///////////////////////////////////////////////////////////////////////////////////////////////////////////////

	class FBoxSurfaceRenderableType : public IRenderableType
	{
		UE_DATAFLOW_IRENDERABLE_PRIMARY_TYPE(FBox, Box);
		UE_DATAFLOW_IRENDERABLE_RENDER_GROUP(Surface);
		UE_DATAFLOW_IRENDERABLE_VIEW_MODE(FDataflowConstruction3DViewMode);
		UE_DATAFLOW_IRENDERABLE_SETTINGS(UDataflowBoxRenderSettings);

		virtual void GetPrimitiveComponents(const FRenderableTypeInstance& Instance, FRenderableComponents& OutComponents) const override
		{
			const FBox Box = GetBox(Instance, {});

			const FName BoxComponentName = Instance.GetComponentName(TEXT("Box"));

			UStaticMesh* BoxMesh = UE::Dataflow::RenderGeometry::GetDataflowBox();
			if (BoxMesh)
			{
				if (UStaticMeshComponent* Component = OutComponents.AddNewComponent<UStaticMeshComponent>(BoxComponentName))
				{
					Component->SetStaticMesh(BoxMesh);

					FTransform BoxTransform;
					BoxTransform.SetTranslation(Box.GetCenter());
					BoxTransform.SetScale3D(Box.GetExtent() / 50.f);

					Component->SetWorldTransform(BoxTransform);
					Component->SetCollisionEnabled(ECollisionEnabled::NoCollision);

					const UDataflowBoxRenderSettings* Settings = Instance.GetTypedRenderSettings<UDataflowBoxRenderSettings>();
					if (Settings)
					{
						const FVector BoxExtent = Box.GetExtent();
						FLinearColor BoxColor = Settings->ColorSettings.Color;

						if (BoxExtent.X < 0.0 || BoxExtent.Y < 0.0 || BoxExtent.Z < 0.0)
						{
							BoxColor = FLinearColor::Red;
						}

						UMaterialInterface* Material = Settings->ColorSettings.bWireframe
							? UE::Dataflow::RenderMaterial::GetDataflowColorWireframeMaterial()
							: UE::Dataflow::RenderMaterial::GetDataflowColorMaterial();
						if (Material)
						{
							UMaterialInstanceDynamic* MaterialInstance = UMaterialInstanceDynamic::Create(Material, Component);
							if (MaterialInstance)
							{
								Component->SetMaterial(0, MaterialInstance);

								MaterialInstance->SetVectorParameterValue(FName("Color"), BoxColor);
								MaterialInstance->SetScalarParameterValue(FName("Transparency"), Settings->ColorSettings.Transparency);
							}
						}
					}
				}
			}
		}
	};

	///////////////////////////////////////////////////////////////////////////////////////////////////////////////

	class FBoxArraySurfaceRenderableType : public IRenderableType
	{
		UE_DATAFLOW_IRENDERABLE_PRIMARY_TYPE(TArray<FBox>, BoxArray);
		UE_DATAFLOW_IRENDERABLE_RENDER_GROUP(Surface);
		UE_DATAFLOW_IRENDERABLE_VIEW_MODE(FDataflowConstruction3DViewMode);
		UE_DATAFLOW_IRENDERABLE_SETTINGS(UDataflowBoxArrayRenderSettings);

		virtual void GetPrimitiveComponents(const FRenderableTypeInstance& Instance, FRenderableComponents& OutComponents) const override
		{
			UStaticMesh* BoxMesh = UE::Dataflow::RenderGeometry::GetDataflowBox();
			if (BoxMesh)
			{
				const TArray<FBox>& BoxArray = GetBoxArray(Instance, {});

				if (BoxArray.Num())
				{
					const FName BoxArrayComponentName = Instance.GetComponentName(TEXT("BoxArray"));

					const int32 NumBoxes = BoxArray.Num();

					UInstancedStaticMeshComponent* ISMComponent = OutComponents.AddNewComponent<UInstancedStaticMeshComponent>(BoxArrayComponentName);
					if (ISMComponent)
					{
						ISMComponent->PreAllocateInstancesMemory(NumBoxes);
						ISMComponent->SetStaticMesh(BoxMesh);

						ISMComponent->SetCollisionEnabled(ECollisionEnabled::NoCollision);

						FBox ComponentBoundingBox;
						for (const FBox& Box : BoxArray)
						{
							ComponentBoundingBox += Box;

							FTransform BoxTransform;
							BoxTransform.SetTranslation(Box.GetCenter());
							BoxTransform.SetScale3D(Box.GetExtent() / 50.f);

							ISMComponent->AddInstance(BoxTransform);
						}

						ISMComponent->SetNumCustomDataFloats(3); // RGB

						const UDataflowBoxArrayRenderSettings* Settings = Instance.GetTypedRenderSettings<UDataflowBoxArrayRenderSettings>();
						if (Settings)
						{
							UMaterialInterface* Material = Settings->ColorSettings.bWireframe
								? UE::Dataflow::RenderMaterial::GetDataflowColorWireframeMaterial()
								: UE::Dataflow::RenderMaterial::GetDataflowColorMaterial();
							if (Material)
							{
								UMaterialInstanceDynamic* MaterialInstance = UMaterialInstanceDynamic::Create(Material, ISMComponent);
								if (MaterialInstance)
								{
									ISMComponent->SetMaterial(0, MaterialInstance);

									MaterialInstance->SetScalarParameterValue(FName("Transparency"), Settings->ColorSettings.Transparency);

									TArray<float> Colors; // NumInstances * 3
									Colors.Init(1.0, BoxArray.Num() * 3);

									int32 Idx = 0;
									for (const FBox& Box : BoxArray)
									{
										FLinearColor Color = UE::Dataflow::Rendering::GetColor(Idx,
											ComponentBoundingBox,
											Box.GetCenter(),
											2.0 * Box.GetExtent().GetMax(),
											Settings->ColorSettings);

										Colors[3 * Idx + 0] = Color.R;
										Colors[3 * Idx + 1] = Color.G;
										Colors[3 * Idx + 2] = Color.B;

										Idx++;
									}

									ISMComponent->SetCustomData(0, BoxArray.Num() - 1, Colors);
								}
							}
						}
					}
				}
			}
		}
	};
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////

namespace UE::Dataflow::Private
{
	void RegisterBoxRenderableTypes()
	{
		UE_DATAFLOW_REGISTER_RENDERABLE_TYPE(FBoxSurfaceRenderableType);
		UE_DATAFLOW_REGISTER_RENDERABLE_TYPE(FBoxArraySurfaceRenderableType);
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////
