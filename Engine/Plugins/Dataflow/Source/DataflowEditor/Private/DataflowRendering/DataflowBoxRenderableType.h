// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataflowRendering/DataflowRenderableTypeSettings.h"
#include "DataflowRendering/DataflowRenderableTypeUtils.h"

#include "DataflowBoxRenderableType.generated.h"

UCLASS(MinimalAPI)
class UDataflowBoxRenderSettings : public UDataflowRenderableTypeSettings
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = "Surface", meta = (ShowOnlyInnerProperties))
	FDataflowSimpleColorCommonRenderSettings ColorSettings;
};

UCLASS(MinimalAPI, AutoExpandCategories = "Surface|ConstantColor, Surface|RandomColor, Surface|ColorBySize")
class UDataflowBoxArrayRenderSettings : public UDataflowRenderableTypeSettings
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = "Surface", meta = (ShowOnlyInnerProperties))
	FDataflowColorCommonRenderSettings ColorSettings;
};

namespace UE::Dataflow::Private
{
	void RegisterBoxRenderableTypes();
}
