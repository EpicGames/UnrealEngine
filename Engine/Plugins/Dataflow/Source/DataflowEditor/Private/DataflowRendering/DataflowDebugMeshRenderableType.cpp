// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataflowRendering/DataflowDebugMeshRenderableType.h"

#include "Components/DynamicMeshComponent.h"
#include "Dataflow/DataflowDebugDrawComponent.h"
#include "DataflowRendering/DataflowRenderableComponents.h"
#include "Debug/DebugDrawComponent.h"
#include "UObject/ObjectPtr.h"
#include "Dataflow/DataflowEngineUtil.h"
#include "Selections/MeshConnectedComponents.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Materials/MaterialInstanceDynamic.h"

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

UDataflowDebugMeshRenderSettings::UDataflowDebugMeshRenderSettings(const FObjectInitializer& ObjectInitializer)
{
	VertexSettings.IDsColor = FColor::Cyan;
	VertexSettings.NormalsColor = FColor::Blue;

	FaceSettings.IDsColor = FColor::Orange;
	FaceSettings.NormalsColor = FColor::Red;

	UMaterialInterface* Material = UE::Dataflow::RenderMaterial::GetDataflowCheckerBoardMaterial();
	if (Material)
	{
		UVSettings.UVMaterial = Material;
	}

	UVSettings.UVColor = FLinearColor(0.178f, 0.287f, 1.0f).ToFColor(true);
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

class FDataflowDebugMeshSceneProxy final : public FDebugRenderSceneProxy
{
public:

	FDataflowDebugMeshSceneProxy(const UPrimitiveComponent* InComponent)
		: FDebugRenderSceneProxy(InComponent)
	{
	}
private:
	virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override
	{
		FPrimitiveViewRelevance Result;
		Result.bDrawRelevance = IsShown(View);
		Result.bDynamicRelevance = true;
		// ideally the TranslucencyRelevance should be filled out by the material, here we do it conservative
		Result.bSeparateTranslucency = Result.bNormalTranslucency = true;
		return Result;
	}
};

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FDebugRenderSceneProxy* UDataflowDebugMeshComponent::CreateDebugSceneProxy()
{
	FDataflowDebugMeshSceneProxy* NewProxy = new FDataflowDebugMeshSceneProxy(this);
	if (NewProxy)
	{
		NewProxy->Texts = IDs;
		NewProxy->Lines = Normals;

		GetDebugDrawDelegateHelper().InitDelegateHelper(NewProxy);
	}
	GetDebugDrawDelegateHelper().ProcessDeferredRegister();
	return NewProxy;
}

FBoxSphereBounds UDataflowDebugMeshComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	FBoxSphereBounds::Builder BoundsBuilder;
	BoundsBuilder += LocalToWorld.GetLocation();

	for (const UDataflowDebugMeshComponent::FLine& Line : Normals)
	{
		BoundsBuilder += LocalToWorld.TransformPosition(Line.Start);
		BoundsBuilder += LocalToWorld.TransformPosition(Line.End);
	}
	for (const UDataflowDebugMeshComponent::FText3d& Text : IDs)
	{
		BoundsBuilder += FSphere(LocalToWorld.TransformPosition(Text.Location), 10.0f); // Approximation
	}

	FBoxSphereBounds ReturnBounds(BoundsBuilder);
	ReturnBounds = ReturnBounds.ExpandBy(5);
	return ReturnBounds;
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

namespace UE::Dataflow::Private
{
	static void UpdateDebugDrawComponentFromDynamicMesh(UDataflowDebugMeshComponent& DebugDrawComponent, const UE::Geometry::FDynamicMesh3& Mesh, const UDataflowDebugMeshRenderSettings& Settings)
	{
		if (Settings.VertexSettings.bShowIDs)
		{
			DebugDrawComponent.IDs.Reserve(DebugDrawComponent.IDs.Num() + Mesh.VertexCount());
			for (const int32 VertexId : Mesh.VertexIndicesItr())
			{
				if (Mesh.IsVertex(VertexId))
				{
					UDataflowDebugMeshComponent::FText3d Text;
					Text.Color = Settings.VertexSettings.IDsColor;
					Text.Location = Mesh.GetVertex(VertexId);
					Text.Text = FString::FromInt(VertexId);
					DebugDrawComponent.IDs.Add(MoveTemp(Text));
				}
			}
		}
		if (Settings.VertexSettings.bShowNormals)
		{
			DebugDrawComponent.Normals.Reserve(DebugDrawComponent.Normals.Num() + Mesh.VertexCount());
			if (Mesh.HasVertexNormals())
			{
				for (const int32 VertexId : Mesh.VertexIndicesItr())
				{
					if (Mesh.IsVertex(VertexId))
					{
						const FVector Position = Mesh.GetVertex(VertexId);
						UDataflowDebugMeshComponent::FLine Line(
							/* Start */ Position,
							/* End   */ Position + FVector(Mesh.GetVertexNormal(VertexId) * Settings.VertexSettings.NormalsLength),
							/* Color */ Settings.VertexSettings.NormalsColor,
							/* Thickness */ Settings.VertexSettings.NormalsThickness
						);
						DebugDrawComponent.Normals.Add(MoveTemp(Line));
					}
				}
			}
			else if (Mesh.HasAttributes() && Mesh.Attributes()->PrimaryNormals())
			{
				const UE::Geometry::FDynamicMeshNormalOverlay* const NormalOverlay = Mesh.Attributes()->PrimaryNormals();

				TArray<int> OverlayElements;
				for (const int32 VertexId : Mesh.VertexIndicesItr())
				{
					if (Mesh.IsVertex(VertexId))
					{
						OverlayElements.Reset();
						NormalOverlay->GetVertexElements(VertexId, OverlayElements);

						FVector3f AvgNormal(0.0f, 0.0f, 0.0f);
						if (OverlayElements.Num() > 0)
						{
							for (int32 ElementID : OverlayElements)
							{
								AvgNormal += NormalOverlay->GetElement(ElementID);
							}
							AvgNormal /= (float)OverlayElements.Num();
						}

						const FVector Position = Mesh.GetVertex(VertexId);
						UDataflowDebugMeshComponent::FLine Line(
							/* Start */ Position,
							/* End   */ Position + FVector(AvgNormal * Settings.VertexSettings.NormalsLength),
							/* Color */ Settings.VertexSettings.NormalsColor,
							/* Thickness */ Settings.VertexSettings.NormalsThickness
						);
						DebugDrawComponent.Normals.Add(MoveTemp(Line));
					}
				}
			}
		}
		if (Settings.FaceSettings.bShowIDs)
		{
			DebugDrawComponent.IDs.Reserve(DebugDrawComponent.IDs.Num() + Mesh.VertexCount());
			for (const int32 TriangleId : Mesh.TriangleIndicesItr())
			{
				if (Mesh.IsTriangle(TriangleId))
				{
					const UE::Geometry::FIndex3i Triangle = Mesh.GetTriangle(TriangleId);
					const FVector Vtx0 = Mesh.GetVertex(Triangle[0]);
					const FVector Vtx1 = Mesh.GetVertex(Triangle[1]);
					const FVector Vtx2 = Mesh.GetVertex(Triangle[2]);
					const FVector Center = (Vtx0 + Vtx1 + Vtx2) / 3.0;

					UDataflowDebugMeshComponent::FText3d Text;
					Text.Color = Settings.FaceSettings.IDsColor;
					Text.Location = Center;
					Text.Text = FString::FromInt(TriangleId);
					DebugDrawComponent.IDs.Add(MoveTemp(Text));
				}
			}
		}
		if (Settings.FaceSettings.bShowNormals)
		{
			DebugDrawComponent.Normals.Reserve(DebugDrawComponent.Normals.Num() + Mesh.VertexCount());
			for (const int32 TriangleId : Mesh.TriangleIndicesItr())
			{
				if (Mesh.IsTriangle(TriangleId))
				{
					const UE::Geometry::FIndex3i Triangle = Mesh.GetTriangle(TriangleId);
					const FVector Vtx0 = Mesh.GetVertex(Triangle[0]);
					const FVector Vtx1 = Mesh.GetVertex(Triangle[1]);
					const FVector Vtx2 = Mesh.GetVertex(Triangle[2]);
					const FVector Center = (Vtx0 + Vtx1 + Vtx2) / 3.0;

					UDataflowDebugMeshComponent::FLine Line(
						/* Start */ Center,
						/* End   */ Center + FVector(Mesh.GetTriNormal(TriangleId) * Settings.FaceSettings.NormalsLength),
						/* Color */ Settings.FaceSettings.NormalsColor,
						/* Thickness */ Settings.FaceSettings.NormalsThickness
					);
					DebugDrawComponent.Normals.Add(MoveTemp(Line));
				}
			}
		}
		if (Settings.UVSettings.bShowUVs)
		{
			TArray<FVector2f> UVs;
			const bool bFoundUVs = UE::Dataflow::Mesh::GetMeshUVs(&Mesh, UVs, Settings.UVSettings.UVChannel);

			DebugDrawComponent.IDs.Reserve(DebugDrawComponent.IDs.Num() + Mesh.VertexCount());
			for (const int32 VertexId : Mesh.VertexIndicesItr())
			{
				if (Mesh.IsVertex(VertexId))
				{
					UDataflowDebugMeshComponent::FText3d Text;
					Text.Color = Settings.UVSettings.UVColor;
					Text.Location = Mesh.GetVertex(VertexId);
					if (bFoundUVs)
					{
						Text.Text = FString::Printf(TEXT("(%.2f,%.2f)"), UVs[VertexId].X, UVs[VertexId].Y);
					}
					else
					{
						Text.Text = TEXT("No UVs");
					}
					DebugDrawComponent.IDs.Add(MoveTemp(Text));
				}
			}
		}
	}
	
	static void SetUVIslandsVertexColors(FDynamicMesh3& Mesh, const UDataflowDebugMeshRenderSettings& Settings)
	{
		if (Mesh.HasAttributes())
		{
			if (UE::Geometry::FDynamicMeshUVOverlay* UVOverlay = Mesh.Attributes()->GetUVLayer(Settings.UVSettings.UVChannel))
			{
				UE::Geometry::FMeshConnectedComponents UVIslands(&Mesh);

				UVIslands.FindConnectedTriangles([&](int32 Triangle0, int32 Triangle1)
					{
						// Check Mesh if triangles connected
						if (!Mesh.IsTriangle(Triangle0) || !Mesh.IsTriangle(Triangle1))
							return false;

						UE::Geometry::FIndex3i EdgesTriangle0 = Mesh.GetTriEdges(Triangle0);

						for (int32 EdgeIdx = 0; EdgeIdx < 3; ++EdgeIdx)
						{
							int32 EdgeID = EdgesTriangle0[EdgeIdx];

							UE::Geometry::FIndex2i ConnectedTris = Mesh.GetEdgeT(EdgeID);

							if (ConnectedTris.A == Triangle1 || (Mesh.IsEdge(ConnectedTris.B) && ConnectedTris.B == Triangle1))
							{
								return true;
							}
						}

						return false;
					});

				if (!Mesh.HasAttributes())
				{
					Mesh.EnableAttributes();
				}
				if (!Mesh.Attributes()->HasPrimaryColors())
				{
					Mesh.Attributes()->EnablePrimaryColors();
				}
				UE::Geometry::FDynamicMeshColorOverlay* ColorOverlay = Mesh.Attributes()->PrimaryColors();
				if (ColorOverlay)
				{
					ColorOverlay->ClearElements();

					ColorOverlay->CreateFromPredicate([](int ParentVID, int TriIDA, int TriIDB)
						{
							return false; // make sure we have a value per corner
						}
					, 0.0f);

					int32 UVIslandCounter = 0;
					for (const UE::Geometry::FMeshConnectedComponents::FComponent& Island : UVIslands.Components)
					{
						FLinearColor RandomColor = FLinearColor::MakeRandomSeededColor(Settings.UVSettings.ColorRandomSeed + UVIslandCounter);
						//						int32 ColorIndex = ColorOverlay->AppendElement(RandomColor);

						for (int32 TriangleID : Island.Indices)
						{
							UE::Geometry::FIndex3i AttrTri;
							if (ColorOverlay->GetTriangleIfValid(TriangleID, AttrTri))
							{
								ColorOverlay->SetElement(AttrTri[0], (FVector4f)RandomColor);
								ColorOverlay->SetElement(AttrTri[1], (FVector4f)RandomColor);
								ColorOverlay->SetElement(AttrTri[2], (FVector4f)RandomColor);
							}
						}

						UVIslandCounter++;
					}
				}
			}
		}
	}

	void AddDebugComponents(FRenderableComponents& OutComponents, const UDataflowDebugMeshRenderSettings& Settings)
	{
		static FName DebugMeshComponentBaseName = TEXT("DebugInfo");

		const TArray<UPrimitiveComponent*> InputComponents = OutComponents.GetComponents();
		for (UPrimitiveComponent* InComponent : InputComponents)
		{
			if (UDynamicMeshComponent* DynamicMeshComponent = Cast<UDynamicMeshComponent>(InComponent))
			{
				if (Settings.UVSettings.bColorUVIslands)
				{
					UMaterialInterface* Material = UE::Dataflow::RenderMaterial::GetDataflowVertexColorMaterial();
					if (Material)
					{
						DynamicMeshComponent->SetOverrideRenderMaterial(Material);
					}

					DynamicMeshComponent->EditMesh([&](FDynamicMesh3& Mesh)
					{
						SetUVIslandsVertexColors(Mesh, Settings);
					});
				}
				
				if (UE::Geometry::FDynamicMesh3* Mesh = DynamicMeshComponent->GetMesh())
				{
					if (UDataflowDebugMeshComponent* DebugDrawComponent = OutComponents.AddNewComponent<UDataflowDebugMeshComponent>(DebugMeshComponentBaseName, DynamicMeshComponent))
					{
						UpdateDebugDrawComponentFromDynamicMesh(*DebugDrawComponent, *Mesh, Settings);
					}
				}
			}

			if (Settings.UVSettings.bUseUVMaterial)
			{
				UMaterialInterface* Material = UE::Dataflow::RenderMaterial::GetDataflowCheckerBoardMaterial(Settings.UVSettings.bColorUVIslands);
				if (Material)
				{
					UMaterialInstanceDynamic* MaterialInstance = UMaterialInstanceDynamic::Create(Material, InComponent);
					if (MaterialInstance)
					{
						if (UDynamicMeshComponent* DynamicMeshComponent = Cast<UDynamicMeshComponent>(InComponent))
						{
							DynamicMeshComponent->SetOverrideRenderMaterial(MaterialInstance);
						}
						else
						{
							for (int32 MaterialIdx = 0; MaterialIdx < InComponent->GetNumMaterials(); ++MaterialIdx)
							{
								InComponent->SetMaterial(MaterialIdx, MaterialInstance);
							}
						}
						MaterialInstance->SetScalarParameterValue(FName("UVChannel"), Settings.UVSettings.UVChannel);
					}
				}
			}
		}
	}
}