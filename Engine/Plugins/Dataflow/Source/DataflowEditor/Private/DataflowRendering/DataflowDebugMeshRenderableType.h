// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataflowRendering/DataflowRenderableTypeSettings.h"
#include "Debug/DebugDrawComponent.h"
#include "Curves/LinearColorRamp.h"
#include "UObject/ObjectPtr.h"
#include "DataflowRendering/DataflowDebugMeshComponent.h"

#include "DataflowDebugMeshRenderableType.generated.h"

USTRUCT()
struct FDataflowDebugMeshRenderElementSettings
{
	GENERATED_BODY()

	/** Show Ids */
	UPROPERTY(EditAnywhere, Category = "IDs", meta = (InlineEditConditionToggle))
	bool bShowIDs = false;

	/** Color of the IDs text */
	UPROPERTY(EditAnywhere, Category = "IDs", meta = (EditCondition = bShowIDs))
	FColor IDsColor = FColor::Blue;

	/** Show normals */
	UPROPERTY(EditAnywhere, Category = "Normals")
	bool bShowNormals = false;

	/** Length of the normals */
	UPROPERTY(EditAnywhere, Category = "Normals", meta = (EditCondition = bShowNormals, Units = cm))
	float NormalsLength = 2.0f;

	/** Thickness of the normals */
	UPROPERTY(EditAnywhere, Category = "Normals", meta = (EditCondition = bShowNormals))
	float NormalsThickness = 1.f;

	/** Color of the Normals */
	UPROPERTY(EditAnywhere, Category = "Normals", meta = (EditCondition = bShowNormals))
	FColor NormalsColor = FColor::Blue;
};

USTRUCT()
struct FDataflowDebugMeshUVSettings
{
	GENERATED_BODY()

	/** Show UVs */
	UPROPERTY(EditAnywhere, Category = "UVs")
	bool bShowUVs = false;

	/** UV channel index */
	UPROPERTY(EditAnywhere, Category = "UVs", meta = (UIMin = 0, UIClampMin = 0, UIMax = 7, UIClampMax = 7, EditCondition = bShowUVs))
	int32 UVChannel = 0;

	/** Color of the UVs text */
	UPROPERTY(EditAnywhere, Category = "UVs", meta = (EditCondition = bShowUVs))
	FColor UVColor = FColor::Yellow;

	/** Assign UV material */
	UPROPERTY(EditAnywhere, Category = "Material")
	bool bUseUVMaterial = false;

	/** Material for meshes to display UVs */
	UPROPERTY(VisibleAnywhere, Category = "Material")
	TObjectPtr<UMaterialInterface> UVMaterial = nullptr;

	/** Color UV islands */
	UPROPERTY(EditAnywhere, Category = "UV Islands")
	bool bColorUVIslands = false;

	/** Color UV islands */
	UPROPERTY(EditAnywhere, Category = "UV Islands")
	int32 ColorRandomSeed = 0;
};

UCLASS(MinimalAPI, AutoExpandCategories = "MeshDebug")
class UDataflowDebugMeshRenderSettings : public UDataflowRenderableTypeSettings
{
	GENERATED_BODY()

	UDataflowDebugMeshRenderSettings(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

public:
	UPROPERTY(EditAnywhere, Category = "MeshDebug")
	FDataflowDebugMeshRenderElementSettings VertexSettings;

	UPROPERTY(EditAnywhere, Category = "MeshDebug")
	FDataflowDebugMeshRenderElementSettings FaceSettings;

	UPROPERTY(EditAnywhere, Category = "MeshDebug", DisplayName="UV Settings")
	FDataflowDebugMeshUVSettings UVSettings;
};

namespace UE::Dataflow
{
	struct FRenderableComponents;

	namespace Private
	{
		void AddDebugComponents(FRenderableComponents& OutComponents, const UDataflowDebugMeshRenderSettings& Settings);
	}
}
