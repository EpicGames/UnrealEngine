// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataflowRendering/DataflowDynamicMeshRenderableType.h"

#include "Components/DynamicMeshComponent.h"

#include "Dataflow/DataflowMesh.h"
#include "Dataflow/DataflowRenderingViewMode.h"

#include "DataflowRendering/DataflowRenderableComponents.h"
#include "DataflowRendering/DataflowRenderableType.h"
#include "DataflowRendering/DataflowRenderableTypeInstance.h"
#include "DataflowRendering/DataflowRenderableTypeRegistry.h"
#include "DataflowRendering/DataflowRenderableTypeSettings.h"
#include "DataflowRendering/DataflowRenderableTypeUtils.h"

#include "UObject/ObjectPtr.h"

UE_DATAFLOW_POLICY_DECLARE_TYPENAME(TObjectPtr<UMaterialInterface>);

namespace UE::Dataflow::Private
{
	///////////////////////////////////////////////////////////////////////////////////////////////////////////////

	class FDynamicMeshSurfaceRenderableType : public IRenderableType
	{
		UE_DATAFLOW_IRENDERABLE_PRIMARY_TYPE(TObjectPtr<UDynamicMesh>, DynamicMesh);
		UE_DATAFLOW_IRENDERABLE_RENDER_GROUP(Surface);
		UE_DATAFLOW_IRENDERABLE_VIEW_MODE(FDataflowConstruction3DViewMode);

		virtual void GetPrimitiveComponents(const FRenderableTypeInstance& Instance, FRenderableComponents& OutComponents) const override
		{
			if (const TObjectPtr<UDynamicMesh> DynamicMesh = GetDynamicMesh(Instance, nullptr))
			{
				const FName ComponentName = Instance.GetComponentName(TEXT("Mesh"));

				if (UDynamicMeshComponent* Component = OutComponents.AddNewComponent<UDynamicMeshComponent>(DynamicMesh->GetFName()))
				{
					Component->SetDynamicMesh(DynamicMesh);

					using FMaterialArray = TArray<TObjectPtr<UMaterialInterface>>;
					const FMaterialArray EmptyArray;
					const FMaterialArray& Materials = Instance.GetOutputValueByType<FMaterialArray>(EmptyArray);
					for (int32 Index = 0; Index < Materials.Num(); ++Index)
					{
						Component->SetMaterial(Index, Materials[Index]);
					}
					// make sure we have material
					if (Materials.IsEmpty())
					{
						Component->SetMaterial(0, nullptr);
					}
				}
			}
		}
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

	class FDataflowMeshSurfaceRenderableType : public IRenderableType
	{
		UE_DATAFLOW_IRENDERABLE_PRIMARY_TYPE(TObjectPtr<UDataflowMesh>, DataflowMesh);
		UE_DATAFLOW_IRENDERABLE_RENDER_GROUP(Surface);
		UE_DATAFLOW_IRENDERABLE_VIEW_MODE(FDataflowConstruction3DViewMode);

		virtual void GetPrimitiveComponents(const FRenderableTypeInstance& Instance, FRenderableComponents& OutComponents) const override
		{
			if (const TObjectPtr<UDataflowMesh> DataflowMesh = GetDataflowMesh(Instance, nullptr))
			{
				if (const UE::Geometry::FDynamicMesh3* DynamicMesh = DataflowMesh->GetDynamicMesh())
				{
					const FName ComponentName = Instance.GetComponentName(TEXT("Mesh"));

					if (UDynamicMeshComponent* Component = OutComponents.AddNewComponent<UDynamicMeshComponent>(DataflowMesh->GetFName()))
					{
						UE::Geometry::FDynamicMesh3 MeshToSet = *DynamicMesh;
						Component->SetMesh(MoveTemp(MeshToSet));

						const TArray<TObjectPtr<UMaterialInterface>>& Materials = DataflowMesh->GetMaterials();
						for (int32 Index = 0; Index < Materials.Num(); ++Index)
						{
							Component->SetMaterial(Index, Materials[Index]);
						}
					}
				}
			}
		}
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
	
	class FDynamicMeshArraySurfaceRenderableType : public IRenderableType
	{
		UE_DATAFLOW_IRENDERABLE_PRIMARY_TYPE(TArray<TObjectPtr<UDynamicMesh>>, DynamicMeshArray);
		UE_DATAFLOW_IRENDERABLE_RENDER_GROUP(Surface);
		UE_DATAFLOW_IRENDERABLE_VIEW_MODE(FDataflowConstruction3DViewMode);

		virtual void GetPrimitiveComponents(const FRenderableTypeInstance& Instance, FRenderableComponents& OutComponents) const override
		{
			const TArray<TObjectPtr<UDynamicMesh>>& DynamicMeshArray = GetDynamicMeshArray(Instance, {});

			for (const TObjectPtr<UDynamicMesh>& DynamicMesh : DynamicMeshArray)
			{
				if (DynamicMesh)
				{
					const FName ComponentName = Instance.GetComponentName(DynamicMesh->GetFName());

					if (UDynamicMeshComponent* Component = OutComponents.AddNewComponent<UDynamicMeshComponent>(ComponentName))
					{
						Component->SetDynamicMesh(DynamicMesh);
					}
				}
			}
		}
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

	class FDynamicMeshUvsRenderableType : public IRenderableType
	{
		UE_DATAFLOW_IRENDERABLE_PRIMARY_TYPE(TObjectPtr<UDynamicMesh>, DynamicMesh);
		UE_DATAFLOW_IRENDERABLE_RENDER_GROUP(Uvs);
		UE_DATAFLOW_IRENDERABLE_VIEW_MODE(FDataflowConstructionUVViewMode);
		UE_DATAFLOW_IRENDERABLE_SETTINGS(UDataflowUVsRenderSettings);

		virtual void GetPrimitiveComponents(const FRenderableTypeInstance& Instance, FRenderableComponents& OutComponents) const override
		{
			const UDataflowUVsRenderSettings* Settings = Instance.GetTypedRenderSettings<UDataflowUVsRenderSettings>();

			if (const TObjectPtr<UDynamicMesh> DynamicMesh = GetDynamicMesh(Instance, nullptr))
			{
				const int32 UVChannel = Settings
					? Settings->GetUVChannel(Rendering::GetUVChannelFromInstance(Instance))
					: 0;
				Rendering::AddUvDynamicMeshComponent(DynamicMesh->GetMeshRef(), UVChannel, OutComponents);
			}
		}
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

	class FDataflowMeshUvsRenderableType : public IRenderableType
	{
		UE_DATAFLOW_IRENDERABLE_PRIMARY_TYPE(TObjectPtr<UDataflowMesh>, DataflowMesh);
		UE_DATAFLOW_IRENDERABLE_RENDER_GROUP(Uvs);
		UE_DATAFLOW_IRENDERABLE_VIEW_MODE(FDataflowConstructionUVViewMode);
		UE_DATAFLOW_IRENDERABLE_SETTINGS(UDataflowUVsRenderSettings);

		virtual void GetPrimitiveComponents(const FRenderableTypeInstance& Instance, FRenderableComponents& OutComponents) const override
		{
			const UDataflowUVsRenderSettings* Settings = Instance.GetTypedRenderSettings<UDataflowUVsRenderSettings>();

			if (const TObjectPtr<UDataflowMesh> DataflowMesh = GetDataflowMesh(Instance, nullptr))
			{
				if (const UE::Geometry::FDynamicMesh3* DynamicMesh = DataflowMesh->GetDynamicMesh())
				{
					const int32 UVChannel = Settings
						? Settings->GetUVChannel(Rendering::GetUVChannelFromInstance(Instance))
						: 0;
					Rendering::AddUvDynamicMeshComponent(*DynamicMesh, UVChannel, OutComponents);
				}
			}
		}
	};

	///////////////////////////////////////////////////////////////////////////////////////////////////////////////

	void RegisterDynamicMeshRenderableTypes()
	{
		UE_DATAFLOW_REGISTER_RENDERABLE_TYPE(FDynamicMeshSurfaceRenderableType);
		UE_DATAFLOW_REGISTER_RENDERABLE_TYPE(FDataflowMeshSurfaceRenderableType);
		UE_DATAFLOW_REGISTER_RENDERABLE_TYPE(FDynamicMeshArraySurfaceRenderableType);
		UE_DATAFLOW_REGISTER_RENDERABLE_TYPE(FDynamicMeshUvsRenderableType);
		UE_DATAFLOW_REGISTER_RENDERABLE_TYPE(FDataflowMeshUvsRenderableType);
	}

	///////////////////////////////////////////////////////////////////////////////////////////////////////////////

}