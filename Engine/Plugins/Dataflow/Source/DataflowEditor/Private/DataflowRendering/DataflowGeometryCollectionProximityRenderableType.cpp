// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataflowRendering/DataflowGeometryCollectionProximityRenderableType.h"

#include "Drawing/LineSetComponent.h"
#include "Drawing/PointSetComponent.h"

#include "Dataflow/DataflowRenderingViewMode.h"

#include "DataflowRendering/DataflowRenderableComponents.h"
#include "DataflowRendering/DataflowRenderableType.h"
#include "DataflowRendering/DataflowRenderableTypeInstance.h"
#include "DataflowRendering/DataflowRenderableTypeRegistry.h"
#include "DataflowRendering/DataflowRenderableTypeUtils.h"

#include "GeometryCollection/Facades/CollectionBoundsFacade.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "GeometryCollection/ManagedArrayCollection.h"

#include "DataflowRendering/DataflowGeometryCollectionRenderableType.h"

#include "Materials/MaterialInterface.h"
#include "Materials/Material.h"

#include "UObject/ObjectPtr.h"

namespace UE::Dataflow::Private
{
	////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

	class FGeometryCollectionProximityRenderableType : public IRenderableType
	{
		UE_DATAFLOW_IRENDERABLE_PRIMARY_TYPE(FManagedArrayCollection, Collection);
		UE_DATAFLOW_IRENDERABLE_RENDER_GROUP(Proximity);
		UE_DATAFLOW_IRENDERABLE_VIEW_MODE(FDataflowConstruction3DViewMode);

		virtual void GetSettingsClasses(TArray<const UClass*>& OutArray) const
		{
			OutArray.Add(UDataflowExplodedViewRenderSettings::StaticClass());
			OutArray.Add(UDataflowGeometryCollectionProximityRenderSettings::StaticClass());
		}

	public:
		FGeometryCollectionProximityRenderableType()
		{
			constexpr bool bDepthTested = false;
			Material = (bDepthTested) 
				? LoadObject<UMaterial>(nullptr, TEXT("/MeshModelingToolsetExp/Materials/PointSetComponentMaterial"))
				: LoadObject<UMaterial>(nullptr, TEXT("/MeshModelingToolsetExp/Materials/PointSetOverlaidComponentMaterial"));
		}

	private:
		virtual bool CanRender(const FRenderableTypeInstance& Instance) const
		{
			const FManagedArrayCollection& Collection = GetCollection(Instance);
			return Collection.HasAttribute(ProximityAttributeName, FGeometryCollection::GeometryGroup)
				&& Collection.HasAttribute(BoundingBoxAttributeName, FGeometryCollection::GeometryGroup);
		}

		virtual void GetPrimitiveComponents(const FRenderableTypeInstance& Instance, FRenderableComponents& OutComponents) const override
		{
			const UDataflowGeometryCollectionProximityRenderSettings* Settings = Instance.GetTypedRenderSettings<UDataflowGeometryCollectionProximityRenderSettings>();
			const UDataflowExplodedViewRenderSettings* ExplodedViewSettings = Instance.GetTypedRenderSettings<UDataflowExplodedViewRenderSettings>();

			const bool bShowCenters = Settings ? Settings->bShowCenters : true;
			const bool bShowConnections = Settings ? Settings->bShowConnections : true;

			const FManagedArrayCollection& Collection = GetCollection(Instance);

			if (Collection.HasAttribute(ProximityAttributeName, FGeometryCollection::GeometryGroup) &&
				Collection.HasAttribute(BoundingBoxAttributeName, FGeometryCollection::GeometryGroup))
			{
				const GeometryCollection::Facades::FBoundsFacade BoundsFacade(Collection);
				const TArray<FVector> CollectionSpaceCentroids = BoundsFacade.GetTransformCentroidsInCollectionSpace();

				TArray<FVector> PerTransformOffsets;
				if (ExplodedViewSettings)
				{
					ExplodedViewSettings->ComputePerTransformExplodedVectors(BoundsFacade, PerTransformOffsets);
				}

				const TManagedArray<TSet<int32>>& Proximity = Collection.GetAttribute<TSet<int32>>(ProximityAttributeName, FGeometryCollection::GeometryGroup);
				const TManagedArray<int32>* GeometryToTransformIndex = Collection.FindAttributeTyped<int32>(FGeometryCollection::TransformIndexAttribute, FGeometryCollection::GeometryGroup);

				const int32 NumGeometry = Collection.NumElements(FGeometryCollection::GeometryGroup);
				TArray<FVector> Centers;
				if (bShowCenters)
				{
					Centers.Reserve(CollectionSpaceCentroids.Num());
				}
				TArray<FVector> LineStarts;
				TArray<FVector> LineEnds;
				if (bShowConnections)
				{
					// rough estimate of number of connection
					LineStarts.Reserve(CollectionSpaceCentroids.Num() * 2);
					LineEnds.Reserve(CollectionSpaceCentroids.Num() * 2);
				}

				for (int32 GeoIndex = 0; GeoIndex < NumGeometry; ++GeoIndex)
				{
					if (GeometryToTransformIndex && GeometryToTransformIndex->IsValidIndex(GeoIndex) && Proximity[GeoIndex].Num() > 0)
					{
						const int32 TransformIndex = (*GeometryToTransformIndex)[GeoIndex];
						const FVector Offset = PerTransformOffsets.IsValidIndex(TransformIndex)? PerTransformOffsets[TransformIndex] : FVector::ZeroVector;
						const FVector Center = Offset + (CollectionSpaceCentroids.IsValidIndex(TransformIndex) ? CollectionSpaceCentroids[TransformIndex] : FVector::ZeroVector);

						if (bShowCenters)
						{
							Centers.Add(Center);
						}

						if (bShowConnections)
						{
							for (const int32 OtherGeoIndex : Proximity[GeoIndex])
							{
								if (OtherGeoIndex > GeoIndex && GeometryToTransformIndex->IsValidIndex(OtherGeoIndex))
								{
									const int32 OtherTransformIndex = (*GeometryToTransformIndex)[OtherGeoIndex];
									const FVector OtherOffset = PerTransformOffsets.IsValidIndex(OtherTransformIndex) ? PerTransformOffsets[OtherTransformIndex] : FVector::ZeroVector;
									const FVector OtherCenter = OtherOffset + (CollectionSpaceCentroids.IsValidIndex(OtherTransformIndex) ? CollectionSpaceCentroids[OtherTransformIndex] : FVector::ZeroVector);

									LineStarts.Add(Center);
									LineEnds.Add(OtherCenter);

								}
							}
						}
					}
				}

				if (bShowCenters && Centers.Num())
				{
					const FName PointComponentName = Instance.GetComponentName(TEXT("Connectivity_Points"));

					if (UPointSetComponent* PointComponent = OutComponents.AddNewComponent<UPointSetComponent>(PointComponentName))
					{
						const float PointSize = Settings ? Settings->PointSize : 15.0f;
						const FColor PointColor = Settings ? Settings->PointColor : FColor::Blue;

						PointComponent->AddPoints(Centers, PointColor, PointSize);
						PointComponent->SetPointMaterial(Material);
					}
				}

				if (bShowConnections && LineStarts.Num() && LineStarts.Num() == LineEnds.Num())
				{
					const FName LineComponentName = Instance.GetComponentName(TEXT("Connectivity_Lines"));

					if (ULineSetComponent* LineComponent = OutComponents.AddNewComponent<ULineSetComponent>(LineComponentName))
					{
						const float LineThickness = Settings? Settings->LineThickness: 4.0f;
						const FColor LineColor = Settings ? Settings->LineColor: FColor::Yellow;

						LineComponent->AddLines(LineStarts, LineEnds, LineColor, LineThickness);
						LineComponent->SetLineMaterial(Material);
					}
				}
			}
		}

		static inline const FName ProximityAttributeName = TEXT("Proximity");
		static inline const FName BoundingBoxAttributeName = TEXT("BoundingBox");

		UMaterialInterface* Material = nullptr;
	};


	///////////////////////////////////////////////////////////////////////////////////////////////////////////////

	void RegisterGeometryCollectionProximityRenderableTypes()
	{
		UE_DATAFLOW_REGISTER_RENDERABLE_TYPE(FGeometryCollectionProximityRenderableType);
	}
}