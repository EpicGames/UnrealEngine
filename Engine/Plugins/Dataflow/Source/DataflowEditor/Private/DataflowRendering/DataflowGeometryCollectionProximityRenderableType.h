// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataflowRendering/DataflowRenderableTypeSettings.h"
#include "Math/Color.h"

#include "DataflowGeometryCollectionProximityRenderableType.generated.h"

UCLASS(MinimalAPI, AutoExpandCategories="Proximity|Centers, Proximity|Connections")
class UDataflowGeometryCollectionProximityRenderSettings : public UDataflowRenderableTypeSettings
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, Category = "Proximity|Centers")
	bool bShowCenters = true;

	UPROPERTY(EditAnywhere, Category = "Proximity|Centers")
	float PointSize = 15.0f;

	UPROPERTY(EditAnywhere, Category = "Proximity|Centers")
	FColor PointColor = FColor::Blue;

	UPROPERTY(EditAnywhere, Category = "Proximity|Connections")
	bool bShowConnections = true;

	UPROPERTY(EditAnywhere, Category = "Proximity|Connections")
	float LineThickness = 4.0f;

	UPROPERTY(EditAnywhere, Category = "Proximity|Connections")
	FColor LineColor = FColor::Yellow;
};

namespace UE::Dataflow::Private
{
	void RegisterGeometryCollectionProximityRenderableTypes();
}
