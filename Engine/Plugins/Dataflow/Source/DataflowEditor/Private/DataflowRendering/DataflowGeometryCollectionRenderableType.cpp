// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataflowRendering/DataflowGeometryCollectionRenderableType.h"

#include "Components/DynamicMeshComponent.h"
#include "Drawing/PointSetComponent.h"
#include "Drawing/LineSetComponent.h"
#include "Drawing/TriangleSetComponent.h"

#include "Dataflow/DataflowRenderingViewMode.h"
#include "DataflowRendering/DataflowRenderableComponents.h"
#include "DataflowRendering/DataflowRenderableType.h"
#include "DataflowRendering/DataflowRenderableTypeInstance.h"
#include "DataflowRendering/DataflowRenderableTypeRegistry.h"
#include "DataflowRendering/DataflowRenderableTypeUtils.h"

#include "DynamicMesh/NonManifoldMappingSupport.h"
#include "DynamicMesh/DynamicMeshBulkEdit.h"

#include "GeometryCollection/Facades/CollectionMeshFacade.h"
#include "GeometryCollection/GeometryCollectionUtility.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "GeometryCollection/ManagedArrayCollection.h"

#include "GeometryCollection/Facades/CollectionCurveFacade.h"
#include "GeometryCollection/Facades/CollectionTransformFacade.h"
#include "GeometryCollection/Facades/CollectionTransformSelectionFacade.h"
#include "GeometryCollection/Facades/CollectionBoundsFacade.h"
#include "GeometryCollection/GeometryCollectionComponent.h"
#include "GeometryCollection/GeometryCollectionObject.h"

#include "UObject/ObjectPtr.h"


#include "PlanarCut.h" // for ConvertGeometryCollectionToDynamicMesh
#include "Dataflow/DataflowEditorStyle.h"

#define LOCTEXT_NAMESPACE "DataflowGeometryCollectionRenderableType"

namespace GeometryCollectionSurfaceRenderableType::Private
{
	bool bFastDynamicMeshGeneration = false;
	FAutoConsoleVariableRef CVarFastDynamicMeshGeneration(
		TEXT("p.Dataflow.FastDynamicMeshGeneration"),
		bFastDynamicMeshGeneration,
		TEXT("When on, use a faster way to generate dynamic meshes - very experimental for now")
	);

	static const TCHAR* DefaultDataflowSelectionMaterialName = TEXT("/Engine/EditorMaterials/Dataflow/DataflowSelection");

	static void AddLeavesToSelection(int32 TransformIndex, const TManagedArray<TSet<int32>>& ChildrenAttribute, FDataflowSelection& OutSelection)
	{
		const TSet<int32>& Children = ChildrenAttribute[TransformIndex];
		if (Children.IsEmpty())
		{
			OutSelection.SetSelected(TransformIndex);
		}
		else
		{
			for (const int32 Child : Children)
			{
				AddLeavesToSelection(Child, ChildrenAttribute, OutSelection);
			}
		}
	}

	static void AddLeavesToSelection(const FManagedArrayCollection& Collection, const FDataflowSelection& InSelection, FDataflowSelection& OutSelection)
	{
		if (const TManagedArray<TSet<int32>>* ChildrenAttributePtr = Collection.FindAttribute<TSet<int32>>(FTransformCollection::ChildrenAttribute, FTransformCollection::TransformGroup))
		{
			const int32 NumTransforms = ChildrenAttributePtr->Num();
			for (int32 TransformIndex = 0; TransformIndex < NumTransforms; ++TransformIndex)
			{
				if (InSelection.IsSelected(TransformIndex))
				{
					AddLeavesToSelection(TransformIndex, *ChildrenAttributePtr, OutSelection);
				}
			}
		}
	}

	static void GeometryCollectionToDynamicMesh(const FManagedArrayCollection& Collection, int32 TransformIndex, FDynamicMesh3& OutMesh)
	{
		GeometryCollection::Facades::FCollectionMeshFacade MeshFacade(Collection);
		if (!MeshFacade.IsValid())
		{
			return; // invalid facade - not a geometry collection
		}

		const int32 GeometryIndex = MeshFacade.TransformToGeometryIndexAttribute.IsValidIndex(TransformIndex)
			? MeshFacade.TransformToGeometryIndexAttribute[TransformIndex]
			: INDEX_NONE;

		if (GeometryIndex == INDEX_NONE)
		{
			return; // no geometry to convert 
		}

		const int32 VtxStart = MeshFacade.VertexStartAttribute.IsValidIndex(GeometryIndex)
			? MeshFacade.VertexStartAttribute[GeometryIndex]
			: 0;
		const int32 VtxCount = MeshFacade.VertexCountAttribute.IsValidIndex(GeometryIndex)
			? MeshFacade.VertexCountAttribute[GeometryIndex]
			: 0;

		if (VtxCount == 0)
		{
			return; // no vertices
		}

		const int32 TriStart = MeshFacade.FaceStartAttribute.IsValidIndex(GeometryIndex)
			? MeshFacade.FaceStartAttribute[GeometryIndex]
			: 0;
		const int32 TriCount = MeshFacade.FaceCountAttribute.IsValidIndex(GeometryIndex)
			? MeshFacade.FaceCountAttribute[GeometryIndex]
			: 0;

		if (TriCount == 0)
		{
			return; // no triangles
		}

		int32 VisibleTriCount = TriCount;
		if (MeshFacade.VisibleAttribute.Num() >= TriStart + TriCount)
		{
			int32 NumVisible = 0;
			for (int32 Index = 0; Index < TriCount; ++Index)
			{
				if (MeshFacade.VisibleAttribute.Get()[TriStart + Index])
				{
					++NumVisible;
				}
			}
			VisibleTriCount = NumVisible;
		}

		if (VisibleTriCount == 0)
		{
			return; // no visible triangles
		}

		TArrayView<const FIntVector> InTriangles = MeshFacade.GetTriangles(TransformIndex);

		UE::Geometry::FDynamicMeshBulkEdit BulkEdit(OutMesh);

		// Edge count is left to zero so it will be autogenerated ( faster than providing it in our case ) 
		BulkEdit.Initialize(VtxCount, VisibleTriCount, /*EdgeCount*/0);

		// Vertices
		constexpr int VertexBatchSize = 1000;
		TArrayView<const FVector3f> InPositions = MeshFacade.GetVertexPositions(TransformIndex);
		ParallelFor(TEXT("CopyCollectionVerticesToDynamicMesh"), InPositions.Num(), VertexBatchSize,
			[&BulkEdit, &InPositions](int32 VertexIndex)
			{
				BulkEdit.SetVertex(VertexIndex, FVector(InPositions[VertexIndex]));
			});

		// Triangles - because of visible tri attribute we cannot parralelize that to get the right index
		int32 TriangleToAdd = 0;
		for (int32 TriangleIndex = 0; TriangleIndex < TriCount; ++TriangleIndex)
		{
			const UE::Geometry::FIndex3i LocalTriangle(
				InTriangles[TriangleIndex].X - VtxStart,
				InTriangles[TriangleIndex].Y - VtxStart,
				InTriangles[TriangleIndex].Z - VtxStart
			);
			const int32 GlobalTriIndex = TriStart + TriangleIndex;
			bool bIsVisible = true;
			if (MeshFacade.VisibleAttribute.IsValidIndex(GlobalTriIndex))
			{
				bIsVisible = MeshFacade.VisibleAttribute.Get()[GlobalTriIndex];
			}
			if (bIsVisible)
			{
				BulkEdit.SetTriVertexIndices(TriangleToAdd, LocalTriangle);
				++TriangleToAdd;
			}
		};
		
		OutMesh.EnableAttributes();

		TArray<UE::Tasks::FTask> OtherTasks;

		// Normals
		if (MeshFacade.NormalAttribute.Num() >= (VtxStart + VtxCount))
		{
			OtherTasks.Add(
				UE::Tasks::Launch(TEXT("CopyCollectionNormalsToDynamicMesh"),
					[&OutMesh, &VtxStart, &VtxCount, &InNormals = MeshFacade.NormalAttribute]()
					{
						OutMesh.EnableVertexNormals(FVector3f(FLinearColor::White));
						for (int32 VertexIndex = 0; VertexIndex < VtxCount; ++VertexIndex)
						{
							OutMesh.SetVertexNormal(VertexIndex, InNormals[VtxStart+VertexIndex]);
						}
					}));
		}

		// Colors
		if (MeshFacade.ColorAttribute.Num() >= (VtxStart + VtxCount))
		{
			OtherTasks.Add(
				UE::Tasks::Launch(TEXT("CopyCollectionNormalsToDynamicMesh"),
					[&OutMesh, &VtxStart, &VtxCount, &InColors = MeshFacade.ColorAttribute]()
					{
						OutMesh.Attributes()->EnablePrimaryColors();
						if (UE::Geometry::FDynamicMeshColorOverlay* ColorOverlay = OutMesh.Attributes()->PrimaryColors())
						{
							ColorOverlay->CreatePerVertex(0.f);
							for (int32 VertexIndex = 0; VertexIndex < VtxCount; ++VertexIndex)
							{
								ColorOverlay->SetElement(VertexIndex, FVector4f(InColors[VtxStart+VertexIndex]));
							}
						}
					}));
		}

		BulkEdit.Finalize();

		UE::Tasks::Wait(OtherTasks);

	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
UDataflowGeometryCollectionSurfaceRenderSettings::UDataflowGeometryCollectionSurfaceRenderSettings(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	SelectionMaterial = LoadObject<UMaterialInterface>(nullptr, GeometryCollectionSurfaceRenderableType::Private::DefaultDataflowSelectionMaterialName);

	constexpr bool bOnlyRGB = true;
	ColorRamp.SetColorAtTime(0.0f, FLinearColor::Black, bOnlyRGB);
	ColorRamp.SetColorAtTime(1.0f, FLinearColor::White, bOnlyRGB);

#if WITH_EDITOR
	ColorRamp.OnColorCurveChangedDelegate.AddWeakLambda(this, [this](const TArray<FRichCurve*>& CHangedCurves)
		{
			FPropertyChangedEvent PropertyChangedEvent(UDataflowGeometryCollectionSurfaceRenderSettings::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UDataflowGeometryCollectionSurfaceRenderSettings, ColorRamp)));
			PostEditChangeProperty(PropertyChangedEvent);
		});
#endif
}

const int32 UDataflowGeometryCollectionSurfaceRenderSettings::GetHashFromProperties() const
{
	int32 TypeHash = 0;
	TypeHash = HashCombine(TypeHash, GetTypeHash(bShowSelection));
	TypeHash = HashCombine(TypeHash, GetTypeHash(SelectionMaterial));
	TypeHash = HashCombine(TypeHash, GetTypeHash(bShowVertexAttributeAsVertexColor));
	TypeHash = HashCombine(TypeHash, GetTypeHash(bUseVertexAttributeOverride));
	TypeHash = HashCombine(TypeHash, GetTypeHash(VertexAttributeOverride));
	TypeHash = HashCombine(TypeHash, GetTypeHash(bUseColorRamp));

	const TArray<FRichCurveEditInfoConst> Curves = ColorRamp.GetCurves();
	for (const FRichCurveEditInfoConst& Curve : Curves)
	{
		TypeHash = HashCombine(TypeHash, GetTypeHash(Curve));
	}

	TypeHash = HashCombine(TypeHash, GetTypeHash(MaxCurveVertices));

	return TypeHash;
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
FVector UDataflowExplodedViewRenderSettings::ComputeExplodedVector(const FVector& Point, const FVector& Center) const
{
	FVector ExplodedVector = Point - Center - CenterOffset;
	ExplodedVector.Normalize();
	ExplodedVector *= Amount;
	return ExplodedVector;
}

void UDataflowExplodedViewRenderSettings::ComputePerTransformExplodedVectors(const GeometryCollection::Facades::FBoundsFacade& BoundsFacade, TArray<FVector>& OutExplodedVectors) const
{
	if (bExplodedView)
	{
		const TArray<FVector> CollectionSpaceCentroids = BoundsFacade.GetTransformCentroidsInCollectionSpace();
		const FBox CollectionSpaceBounds = BoundsFacade.GetBoundingBoxInCollectionSpace();

		const int32 NumTransforms = CollectionSpaceCentroids.Num();

		OutExplodedVectors.Init(FVector::ZeroVector, NumTransforms);
		for (int32 TransformIndex = 0; TransformIndex < NumTransforms; ++TransformIndex)
		{
			const FVector TransformCentroid = CollectionSpaceCentroids[TransformIndex];
			OutExplodedVectors[TransformIndex] = ComputeExplodedVector(TransformCentroid, CollectionSpaceBounds.GetCenter());
		}
	}
}

void UDataflowExplodedViewRenderSettings::ComputePerTransformExplodedVectors(const FManagedArrayCollection& Collection, TArray<FVector>& OutExplodedVectors) const
{
	const GeometryCollection::Facades::FBoundsFacade BoundsFacade(Collection);
	ComputePerTransformExplodedVectors(BoundsFacade, OutExplodedVectors);
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

namespace UE::Dataflow::Private
{
	////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

	class FGeometryCollectionSurfaceRenderableType : public IRenderableType
	{
		UE_DATAFLOW_IRENDERABLE_PRIMARY_TYPE(FManagedArrayCollection, Collection);
		UE_DATAFLOW_IRENDERABLE_RENDER_GROUP(Surface);
		UE_DATAFLOW_IRENDERABLE_VIEW_MODE(FDataflowConstruction3DViewMode);
		
		virtual void GetSettingsClasses(TArray<const UClass*>& OutArray) const
		{
			OutArray.Add(UDataflowExplodedViewRenderSettings::StaticClass());
			OutArray.Add(UDataflowGeometryCollectionSurfaceRenderSettings::StaticClass());
		}

		struct FSelection
		{
			FDataflowTransformSelection TransformSelection;
			bool bValidTransformSelection = false;

			FDataflowVertexSelection VertexSelection;
			bool bValidVertexSelection = false;

			FDataflowFaceSelection FaceSelection;
			bool bValidFaceSelection = false;

			FDataflowCurveSelection CurveSelection;
			bool bValidCurveSelection = false;
		};

		virtual bool CanRender(const FRenderableTypeInstance& Instance) const
		{
			const FManagedArrayCollection& Collection = GetCollection(Instance);
			return Collection.HasGroup(FGeometryCollection::GeometryGroup);
		}

		virtual void GetPrimitiveComponents(const FRenderableTypeInstance& Instance, FRenderableComponents& OutComponents) const override
		{
			// Save the  number of component we start with to known which ones were added during this method execution
			const int32 NumComponentsAtStart = OutComponents.GetComponents().Num();

			const UDataflowGeometryCollectionSurfaceRenderSettings* Settings = Instance.GetTypedRenderSettings<UDataflowGeometryCollectionSurfaceRenderSettings>();
			const UDataflowExplodedViewRenderSettings* ExplodedViewSettings = Instance.GetTypedRenderSettings<UDataflowExplodedViewRenderSettings>();

			const bool bShowSelection = Settings ? Settings->bShowSelection : true;

			const FManagedArrayCollection& Collection = GetCollection(Instance);

			// TODO(dataflow) : use a facade or a more generic way to convert a collection to a dynamic mesh
			const TManagedArray<FTransform3f>* TransformAttribute = Collection.FindAttribute<FTransform3f>(FTransformCollection::TransformAttribute, FTransformCollection::TransformGroup);
			const TManagedArray<int32>* TransformIndexAttribute = Collection.FindAttribute<int32>(FGeometryCollection::TransformIndexAttribute, FGeometryCollection::GeometryGroup);
			const TManagedArray<FString>* BoneNameAttribute = Collection.FindAttribute<FString>(FTransformCollection::BoneNameAttribute, FTransformCollection::TransformGroup);

			static const FName MeshArrayCacheName = TEXT("MeshArray");
			static const FName SettingsHashCacheName = TEXT("SettingsHash");
			if (TransformAttribute && TransformIndexAttribute)
			{
				TArray<FDynamicMesh3> DynamicMeshes;

				// first check if the setting shave changes
				const int32 CachedSettingsHash = Instance.GetCachedValue<int32>(SettingsHashCacheName);
				const int32 CurrentSettingsHash = Settings ? Settings->GetHashFromProperties() : 0;
				if (CachedSettingsHash == CurrentSettingsHash)
				{
					// now check if we have precached meshes
					if (Instance.HasUptoDateCachedValue(MeshArrayCacheName))
					{
						const TArray<FDynamicMesh3>& CachedDynamicMeshes = Instance.GetCachedValue<TArray<FDynamicMesh3>>(MeshArrayCacheName);
						DynamicMeshes.SetNum(CachedDynamicMeshes.Num());

						// copying dynamic meshes can be expensive let's parallelize it 
						ParallelFor(DynamicMeshes.Num(),
							[&CachedDynamicMeshes, &DynamicMeshes](int32 Index)
							{
								DynamicMeshes[Index] = CachedDynamicMeshes[Index];
							});
					}
				}
				if (DynamicMeshes.Num() != TransformAttribute->Num())
				{
					// TODO(dataflow) : need a version of ConvertGeometryCollectionToDynamicMesh that only takes a managed array collection
					if (TUniquePtr<FGeometryCollection> GeomCollection = TUniquePtr<FGeometryCollection>(Collection.NewCopy<FGeometryCollection>()))
					{
						CullCurvesIfNeeded(*GeomCollection, Settings ? Settings->MaxCurveVertices : -1);

						// prepare the dynamic meshes
						DynamicMeshes.SetNum(TransformAttribute->Num());

						TArrayView<const FTransform3f> BoneTransforms = MakeArrayView(TransformAttribute->GetConstArray());

						// Override vertex color if we have an attribute to visualize
						const bool bShowAttributeIfAvailable = Settings ? Settings->bShowVertexAttributeAsVertexColor : true;
						if (bShowAttributeIfAvailable)
						{
							FDataflowNode::FAttributeKey AttributeKey;
							if (Settings && Settings->bUseVertexAttributeOverride)
							{
								AttributeKey.AttributeName = Settings ? FName(Settings->VertexAttributeOverride) : NAME_None;
								AttributeKey.GroupName = FGeometryCollection::VerticesGroup;
							}
							if (AttributeKey.AttributeName.IsNone())
							{
								AttributeKey = Instance.GetVertexAttributeToVisualize();
							}
							TransferAttributeToVisualizeToVertexColor(*GeomCollection, AttributeKey, Settings);
						}

						// Convert to dynamic meshes in parallel
						ParallelFor(DynamicMeshes.Num(),
							[&DynamicMeshes, &BoneTransforms, &GeomCollection](int32 TransformIndex)
							{
								if (GeomCollection && GeomCollection->TransformToGeometryIndex.IsValidIndex(TransformIndex))
								{
									const int32 GeometryIndex = GeomCollection->TransformToGeometryIndex[TransformIndex];
									FDynamicMesh3& DynamicMesh = DynamicMeshes[TransformIndex];

									if (GeometryCollectionSurfaceRenderableType::Private::bFastDynamicMeshGeneration)
									{
										GeometryCollectionSurfaceRenderableType::Private::GeometryCollectionToDynamicMesh(*GeomCollection, TransformIndex, DynamicMesh);
									}
									else
									{
										TArrayView<const int32> TransformIndicesToConvert(&TransformIndex, 1);

										FTransform UnusedTransform;
										constexpr bool bWeldEdges = false; // disabling this as this can take 43% of the conversion time for large meshes
										//constexpr bool bAllowInvisible = false;
										//constexpr bool bCenterPivot = false;

										::ConvertGeometryCollectionToDynamicMesh(DynamicMesh, UnusedTransform, false, *GeomCollection, bWeldEdges, BoneTransforms, true, TransformIndicesToConvert);
										//ConvertGeometryCollectionGeometryToDynamicMesh(*GeomCollection, GeometryIndex, bAllowInvisible, bCenterPivot, DynamicMesh, UnusedTransform);
									}
								}
							});
					}
					Instance.CacheValue(DynamicMeshes, MeshArrayCacheName);
					Instance.CacheValue(CurrentSettingsHash, SettingsHashCacheName);
				}

				const FColor SelectionColor = FColor::Orange;
				FSelection Selection;
				if (bShowSelection)
				{
					GetSelection(Instance, Collection, Selection);
				}

				UMaterialInterface* SelectionMaterial = Settings
					? Settings->SelectionMaterial.Get()
					: LoadObject<UMaterialInterface>(nullptr, GeometryCollectionSurfaceRenderableType::Private::DefaultDataflowSelectionMaterialName);

				TArray<FVector> PerTransformOffsets;
				if (ExplodedViewSettings)
				{
					ExplodedViewSettings->ComputePerTransformExplodedVectors(Collection, PerTransformOffsets);
				}

				const GeometryCollection::Facades::FCollectionTransformFacade TransformFacade(Collection);
			
				// generate dynamic mesh components
				static FName BaseParentName = TEXT("Collection");
				UPrimitiveComponent* ParentComponent = OutComponents.AddNewComponent<UStaticMeshComponent>(BaseParentName);
				if (ParentComponent)
				{
					TMap<int32, UPrimitiveComponent*> TransformIndexToComponentMap;

					for (const int32 RootIndex : TransformFacade.GetRootIndices())
					{
						AddComponentsForTransformIndex(Collection, RootIndex, ParentComponent, DynamicMeshes, Selection, SelectionMaterial, OutComponents, /*Out*/TransformIndexToComponentMap);
					}

					// now apply exploded settings if needed
					if (ExplodedViewSettings && ExplodedViewSettings->bExplodedView)
					{
						for (const TPair<int32, UPrimitiveComponent*>& Entry : TransformIndexToComponentMap)
						{
							if (UPrimitiveComponent* Component = Entry.Value)
							{
								if (PerTransformOffsets.IsValidIndex(Entry.Key))
								{
									const FTransform NewTransform(PerTransformOffsets[Entry.Key]);
									Component->SetWorldTransform(NewTransform);
								}
							}
						}
					}
				}

				// Create point component for vertex selection
				if (Selection.bValidVertexSelection)
				{
					const TManagedArray<FVector3f>* VertexAttribute = Collection.FindAttribute<FVector3f>(FGeometryCollection::VertexPositionAttribute, FGeometryCollection::VerticesGroup);
					const TManagedArray<int32>* VertexToTransformIndex = Collection.FindAttribute<int32>(FGeometryCollection::VertexBoneMapAttribute, FGeometryCollection::VerticesGroup);
					const TManagedArray<TSet<int32>>* Children = Collection.FindAttribute<TSet<int32>>(FGeometryCollection::ChildrenAttribute, FGeometryCollection::TransformGroup);
					if (VertexAttribute)
					{
						const TArray<FTransform> CollectionSpaceTransforms = TransformFacade.ComputeCollectionSpaceTransforms();

						const int32 NumVertices = Selection.VertexSelection.Num();
						TArray<FVector> Points;
						Points.Reserve(NumVertices);

						for (int32 Index = 0; Index < NumVertices; ++Index)
						{
							if (Selection.VertexSelection.IsSelected(Index))
							{
								const int32 TransformIndex = VertexToTransformIndex ? (*VertexToTransformIndex)[Index] : INDEX_NONE;
								const bool bIsCluster = (Children && (*Children)[TransformIndex].Num() > 0);
								if (!bIsCluster)
								{
									const FVector Offset = (PerTransformOffsets.IsValidIndex(TransformIndex)) ? PerTransformOffsets[TransformIndex] : FVector::ZeroVector;

									const FTransform Transform = CollectionSpaceTransforms.IsValidIndex(TransformIndex) ? CollectionSpaceTransforms[TransformIndex] : FTransform::Identity;
									const FVector LocalPoint = FVector((*VertexAttribute)[Index]);
									Points.Add(Transform.TransformPosition(LocalPoint) + Offset);
								}
							}
						}

						const FName PointComponentName = Instance.GetComponentName(TEXT("VertexSelection"));

						if (UPointSetComponent* PointComponent = OutComponents.AddNewComponent<UPointSetComponent>(PointComponentName))
						{
							constexpr float PointSize = 8.f;
							PointComponent->ReservePoints(Points.Num());
							PointComponent->AddPoints(Points, SelectionColor, PointSize);
							UMaterialInterface* Material = LoadObject<UMaterialInterface>(nullptr, TEXT("/MeshModelingToolsetExp/Materials/PointSetOverlaidComponentMaterial_Round"));
							PointComponent->SetMaterial(0, Material);
						}
					}
				}

				if (Selection.bValidFaceSelection)
				{
					const TManagedArray<FVector3f>* VertexAttribute = Collection.FindAttribute<FVector3f>(FGeometryCollection::VertexPositionAttribute, FGeometryCollection::VerticesGroup);
					const TManagedArray<FVector3f>* NormalAttribute = Collection.FindAttribute<FVector3f>(FGeometryCollection::VertexNormalAttribute, FGeometryCollection::VerticesGroup);
					const TManagedArray<int32>* VertexToTransformIndex = Collection.FindAttribute<int32>(FGeometryCollection::VertexBoneMapAttribute, FGeometryCollection::VerticesGroup);
					const TManagedArray<FIntVector>* TriangleAttribute = Collection.FindAttribute<FIntVector>(FGeometryCollection::FaceIndicesAttribute, FGeometryCollection::FacesGroup);

					if (VertexAttribute && NormalAttribute && TriangleAttribute)
					{
						const int32 NumSelected = Selection.FaceSelection.NumSelected();
						if (NumSelected > 0 && Selection.FaceSelection.Num() == TriangleAttribute->Num())
						{
							const TArray<FTransform> CollectionSpaceTransforms = TransformFacade.ComputeCollectionSpaceTransforms();

							const FName TriangleComponentName = Instance.GetComponentName(TEXT("FaceSelection"));
							if (UTriangleSetComponent* TriangleComponent = OutComponents.AddNewComponent<UTriangleSetComponent>(TriangleComponentName))
							{
								TriangleComponent->AddTriangles(
									1 /*NumIndices*/, // number of time the lambda will be called
									[&VertexAttribute, &NormalAttribute, &VertexToTransformIndex, &TriangleAttribute, &Selection, &CollectionSpaceTransforms, &SelectionMaterial, &PerTransformOffsets]
									(int32 Iteration, TArray<FRenderableTriangle>& TrianglesOut)
									{
										auto GetPointFromIndex = [&VertexAttribute, &VertexToTransformIndex, &CollectionSpaceTransforms, &PerTransformOffsets](int32 VtxIdx)
											{
												if (VertexAttribute->IsValidIndex(VtxIdx))
												{
													if (VertexToTransformIndex && VertexToTransformIndex->IsValidIndex(VtxIdx) && CollectionSpaceTransforms.IsValidIndex((*VertexToTransformIndex)[VtxIdx]))
													{
														const int32 TransformIndex = (*VertexToTransformIndex)[VtxIdx];
														const FVector Offset = (PerTransformOffsets.IsValidIndex(TransformIndex)) ? PerTransformOffsets[TransformIndex] : FVector::ZeroVector;
														const FTransform Transform = CollectionSpaceTransforms[TransformIndex];
														return Transform.TransformPosition(FVector((*VertexAttribute)[VtxIdx])) + Offset;
													}
													return FVector((*VertexAttribute)[VtxIdx]);
												}
												return FVector::ZeroVector;
											};

										for (int32 TriIdx = 0; TriIdx < TriangleAttribute->Num(); ++TriIdx)
										{
											if (Selection.FaceSelection.IsSelected(TriIdx))
											{
												const FIntVector& Tri = (*TriangleAttribute)[TriIdx];
												const FVector Vtx0 = GetPointFromIndex(Tri[0]);
												const FVector Vtx1 = GetPointFromIndex(Tri[1]);
												const FVector Vtx2 = GetPointFromIndex(Tri[2]);

												FRenderableTriangle TriangleToAdd;
												TriangleToAdd.Material = SelectionMaterial;
												TriangleToAdd.Vertex0.Position = Vtx0;
												TriangleToAdd.Vertex0.Normal = FVector((*NormalAttribute)[Tri[0]]);
												TriangleToAdd.Vertex0.Color = FColor::White;
												TriangleToAdd.Vertex1.Position = Vtx1;
												TriangleToAdd.Vertex1.Color = FColor::White;
												TriangleToAdd.Vertex1.Normal = FVector((*NormalAttribute)[Tri[1]]);
												TriangleToAdd.Vertex2.Position = Vtx2;
												TriangleToAdd.Vertex2.Color = FColor::White;
												TriangleToAdd.Vertex2.Normal = FVector((*NormalAttribute)[Tri[2]]);
												TrianglesOut.Add(TriangleToAdd);
											}
										}
									},
									NumSelected /*TrianglesPerIndexHint*/
								);
								TriangleComponent->SetMaterial(0, SelectionMaterial);
								TriangleComponent->SetOverlayMaterial(SelectionMaterial);

							}
						}
					}
				}

				// Create line component from curve selection 
				if (Selection.bValidCurveSelection)
				{
					const FName LineComponentName = Instance.GetComponentName(TEXT("CurveSelection"));

					if (ULineSetComponent* LineComponent = OutComponents.AddNewComponent<ULineSetComponent>(LineComponentName))
					{
						GeometryCollection::Facades::FCollectionCurveGeometryFacade CurveFacade(Collection);
						const int32 NumCurves = CurveFacade.GetNumCurves();
						TArray<FVector3f> CurvePositions;
						for (int32 CurveIndex = 0; CurveIndex < NumCurves; ++CurveIndex)
						{
							if (Selection.CurveSelection.IsSelected(CurveIndex))
							{
								CurveFacade.GetCurvePointPositions(CurveIndex, CurvePositions);
								for (int32 PointIndex = 1; PointIndex < CurvePositions.Num(); ++PointIndex)
								{
									LineComponent->AddLine(FVector(CurvePositions[PointIndex - 1]), FVector(CurvePositions[PointIndex]), SelectionColor, 3.0f);
								}
							}
						}
						LineComponent->SetLineMaterial(LoadObject<UMaterialInterface>(nullptr, TEXT("/MeshModelingToolsetExp/Materials/LineSetOverlaidComponentMaterial")));
					}
				}
			}
		}

		void AddComponentsForTransformIndex(
			const FManagedArrayCollection& Collection,
			const int32 TransformIndex,
			UPrimitiveComponent* ParentComponent,
			TArray<FDynamicMesh3>& DynamicMeshes,
			const FSelection& Selection,
			UMaterialInterface* SelectionMaterial,
			FRenderableComponents& OutComponents,
			TMap<int32, UPrimitiveComponent*>& OutTransformIndexToComponentMap
			) const
		{
			const TManagedArray<FString>* BoneNameAttributePtr = Collection.FindAttribute<FString>(FTransformCollection::BoneNameAttribute, FTransformCollection::TransformGroup);
			const TManagedArray<TSet<int32>>* ChildrenAttributePtr = Collection.FindAttribute<TSet<int32>>(FTransformCollection::ChildrenAttribute, FTransformCollection::TransformGroup);
			if (BoneNameAttributePtr && ChildrenAttributePtr)
			{
				// Create new component using ParentComponent as parent
				static const FString DefaultClusterString = TEXT("Cluster");
				static const FString DefaultLeafString = TEXT("Leaf");
				const FString BoneTypeString = ((*ChildrenAttributePtr)[TransformIndex].Num() == 0)? DefaultClusterString: DefaultLeafString;
				const FName ComponentName = FName(FString::Printf(TEXT("[%03d]_[%s]_GC_%s"), TransformIndex, *BoneTypeString, *(*BoneNameAttributePtr)[TransformIndex]));

				if (UDynamicMeshComponent* Component = OutComponents.AddNewComponent<UDynamicMeshComponent>(ComponentName, ParentComponent))
				{
					OutTransformIndexToComponentMap.Add(TransformIndex, Component);

					Component->SetCastShadow(false);
					Component->SetMesh(MoveTemp(DynamicMeshes[TransformIndex]));
					if (Selection.bValidTransformSelection && Selection.TransformSelection.IsSelected(TransformIndex))
					{
						Component->SetOverrideRenderMaterial(SelectionMaterial);
					}
					else
					{
						Component->SetOverrideRenderMaterial(FDataflowEditorStyle::Get().VertexMaterial);
					}

					// If no children then exit
					const TSet<int32>& Children = (*ChildrenAttributePtr)[TransformIndex];
					if (Children.Num() == 0)
					{
						return;
					}
					for (const int32& ChildTransformIndex : Children)
					{
						AddComponentsForTransformIndex(Collection, ChildTransformIndex, Component, DynamicMeshes, Selection, SelectionMaterial, OutComponents, OutTransformIndexToComponentMap);
					}
				}
			}
		}

		void GetSelection(const FRenderableTypeInstance& Instance, 
			const FManagedArrayCollection& Collection, 
			FSelection& OutSelection
		) const
		{
			OutSelection = FSelection();

			FDataflowSelection Selection;
			Instance.GetSelectionToVisualize(Selection);
			if (!Selection.IsValidForCollection(Collection))
			{
				FDataflowSelection DefaultValue;
				Selection = Instance.GetOutputValueByType<FDataflowSelectionTypes>(DefaultValue);
			}
			if (!Selection.IsValidForCollection(Collection))
			{
				return;
			}
			if (Selection.AnySelected())
			{
				const FName GroupName = Selection.GetGroupName();
				if (GroupName == FDataflowTransformSelection::TransformGroupName)
				{
					OutSelection.TransformSelection.Initialize(Selection);
					GeometryCollectionSurfaceRenderableType::Private::AddLeavesToSelection(Collection, Selection, OutSelection.TransformSelection);
					OutSelection.bValidTransformSelection = true;
				}
				else if (GroupName == FDataflowVertexSelection::VerticesGroupName)
				{
					OutSelection.VertexSelection.Initialize(Selection);
					OutSelection.bValidVertexSelection = true;
				}
				else if (GroupName == FDataflowFaceSelection::FacesGroupName)
				{
					OutSelection.FaceSelection.Initialize(Selection);
					OutSelection.bValidFaceSelection = true;
				}
				else if (GroupName == FDataflowCurveSelection::CurveGroupName)
				{
					OutSelection.CurveSelection.Initialize(Selection);
					OutSelection.bValidCurveSelection = true;
				}
			}
		}

		template<typename T>
		void TransferTypedAttributeVertexColor(FGeometryCollection& GeomCollection, const FDataflowNode::FAttributeKey& ValueAttributeKey, const UDataflowGeometryCollectionSurfaceRenderSettings* Settings, const T& DefaultValue) const
		{
			TManagedArray<FLinearColor>* ColorAttribute = GeomCollection.FindAttributeTyped<FLinearColor>(FGeometryCollection::ColorAttribute, FGeometryCollection::VerticesGroup);
			const TManagedArray<T>* ValueAttribute = GeomCollection.FindAttributeTyped<T>(ValueAttributeKey.AttributeName, ValueAttributeKey.GroupName);
			if (ColorAttribute && ValueAttribute && ValueAttribute->Num() == ColorAttribute->Num())
			{
				static const FLinearColorRamp EmptyColorRamp;

				const FLinearColorRamp& ColorRamp = Settings ? Settings->ColorRamp : EmptyColorRamp;
				const bool bRemapValue = (Settings && !ColorRamp.IsEmpty()) ? Settings->bUseColorRamp : false;

				const int32 NumVertices = ColorAttribute->Num();

				T MaxValue = DefaultValue;
				T MinValue = DefaultValue;
				T Range = DefaultValue;
				if constexpr (std::is_same_v<T, int32>)
				{
					if (const int32* MaxValuePtr = Algo::MaxElement(*ValueAttribute))
					{
						MaxValue = *MaxValuePtr;
					}
					if (const int32* MinValuePtr = Algo::MinElement(*ValueAttribute))
					{
						MinValue = *MinValuePtr;
					}
					Range = (MaxValue - MinValue);
				}

				for (int32 Index = 0; Index < NumVertices; ++Index)
				{
					const T& Value = (*ValueAttribute)[Index];
					if constexpr (std::is_same_v<T, float> || std::is_same_v<T, double>)
					{
						(*ColorAttribute)[Index] = bRemapValue
							? ColorRamp.GetLinearColorValue(Value)
							: FLinearColor(Value, Value, Value);
					}
					else if constexpr (std::is_same_v<T, bool>)
					{
						const float FloatValue = Value ? 1.0f : 0.0f;
						(*ColorAttribute)[Index] = bRemapValue
							? ColorRamp.GetLinearColorValue(Value)
							: FLinearColor(FloatValue, FloatValue, FloatValue);
					}
					else if constexpr (std::is_same_v<T, int32>)
					{
						const float Multiplier = (Range == 0) ? 1.f : (1.f / Range);
						const float NormalizedValue{ (Value - MinValue) * Multiplier };
						(*ColorAttribute)[Index] = bRemapValue
							? ColorRamp.GetLinearColorValue(NormalizedValue)
							: FLinearColor(NormalizedValue, NormalizedValue, NormalizedValue);
					}
					else if constexpr (std::is_same_v<T, FVector3f> || std::is_same_v<T, FVector3d>)
					{
						FVector3d NormalizedValue{Value};
						NormalizedValue.Normalize();
						(*ColorAttribute)[Index] = FLinearColor(NormalizedValue);
					}
					else if constexpr (std::is_same_v<T, FLinearColor>)
					{
						(*ColorAttribute)[Index] = Value;
					}
				}
			}
		}

		void TransferAttributeToVisualizeToVertexColor(FGeometryCollection& GeomCollection, const FDataflowNode::FAttributeKey& ValueAttributeKey, const UDataflowGeometryCollectionSurfaceRenderSettings* Settings) const
		{
			if (ValueAttributeKey.AttributeName.IsNone() || ValueAttributeKey.GroupName.IsNone())
			{
				return;
			}
			const FManagedArrayCollection::EArrayType AttributeType = GeomCollection.GetAttributeType(ValueAttributeKey.AttributeName, ValueAttributeKey.GroupName);
			switch (AttributeType)
			{
			case FManagedArrayCollection::EArrayType::FFloatType:
				TransferTypedAttributeVertexColor<float>(GeomCollection, ValueAttributeKey, Settings, 0.f);
				break;
			case FManagedArrayCollection::EArrayType::FDoubleType:
				TransferTypedAttributeVertexColor<double>(GeomCollection, ValueAttributeKey, Settings, 0.0);
				break;
			case FManagedArrayCollection::EArrayType::FVectorType:
				TransferTypedAttributeVertexColor<FVector3f>(GeomCollection, ValueAttributeKey, Settings, FVector3f::ZeroVector);
				break;
			case FManagedArrayCollection::EArrayType::FVector3dType:
				TransferTypedAttributeVertexColor<FVector3d>(GeomCollection, ValueAttributeKey, Settings, FVector3d::ZeroVector);
				break;
			case FManagedArrayCollection::EArrayType::FInt32Type:
				TransferTypedAttributeVertexColor<int32>(GeomCollection, ValueAttributeKey, Settings, 0);
				break;
			case FManagedArrayCollection::EArrayType::FBoolType:
				TransferTypedAttributeVertexColor<bool>(GeomCollection, ValueAttributeKey, Settings, false);
				break;
			case FManagedArrayCollection::EArrayType::FLinearColorType:
				TransferTypedAttributeVertexColor<FLinearColor>(GeomCollection, ValueAttributeKey, Settings, FColor::Black);
				break;

			}
		}

		void CullCurvesIfNeeded(FGeometryCollection& GeomCollection, int32 MaxCurveVerticesFromSettings) const
		{
			// Performance control for curve based geometry that can ghave a lot of geometry
			// if the threshold is reached, we only render a certain percentage of the curves
			const int32 NumVertices = GeomCollection.NumElements(FGeometryCollection::VerticesGroup);

			float CullingRatio = 0.0f;
			if (MaxCurveVerticesFromSettings > 0 && NumVertices > MaxCurveVerticesFromSettings)
			{
				CullingRatio = 1.0 - FMath::Clamp((float)MaxCurveVerticesFromSettings / (float)NumVertices, 0, 1);
			}

			GeometryCollection::Facades::FCollectionCurveGeometryFacade CurveFacade(GeomCollection);
			const bool bHasCurves = CurveFacade.IsValid();
			if (bHasCurves)
			{
				TManagedArray<bool>& FaceVisible = GeomCollection.AddAttribute<bool>(FGeometryCollection::FaceVisibleAttribute, FGeometryCollection::FacesGroup);

				FRandomStream Random(0);

				// assumption that vertex index are Pointindex * 2
				const TArray<int32>& CurvePointOffsets = CurveFacade.GetCurvePointOffsets();
				int32 StartFace = 0;
				const int32 NumCurves = CurvePointOffsets.Num();
				for (int32 CurveIndex = 0; CurveIndex < NumCurves; ++CurveIndex)
				{
					const float CurveRand = Random.GetFraction();
					const bool bCurveVisible = (CurveRand >= CullingRatio);

					const int32 CurvPointIndex = CurvePointOffsets[CurveIndex];
					const int32 PrevPointIndex = (CurveIndex > 0) ? CurvePointOffsets[CurveIndex - 1] : 0;
					const int32 CurveLength = FMath::Max(0, CurvPointIndex - PrevPointIndex);
					if (CurveLength > 0)
					{
						const int32 CurveNumFaces = (CurveLength - 1) * 2;
						for (int32 FaceIndex = 0; FaceIndex < CurveNumFaces; ++FaceIndex)
						{
							const int32 GlobalFaceIndex = StartFace + FaceIndex;
							if (FaceVisible.IsValidIndex(GlobalFaceIndex))
							{
								FaceVisible[GlobalFaceIndex] = bCurveVisible;
							}
						}
						StartFace += CurveNumFaces;
					}
				}
			}
		}
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

	class FGeometryCollectionUvsRenderableType : public IRenderableType
	{
		UE_DATAFLOW_IRENDERABLE_PRIMARY_TYPE(FManagedArrayCollection, Collection);
		UE_DATAFLOW_IRENDERABLE_RENDER_GROUP(Uvs);
		UE_DATAFLOW_IRENDERABLE_VIEW_MODE(FDataflowConstructionUVViewMode);
		UE_DATAFLOW_IRENDERABLE_SETTINGS(UDataflowUVsRenderSettings);

		virtual bool CanRender(const FRenderableTypeInstance& Instance) const
		{
			const FManagedArrayCollection& Collection = GetCollection(Instance);
			return Collection.HasGroup(FGeometryCollection::GeometryGroup);
		}

		virtual void GetPrimitiveComponents(const FRenderableTypeInstance& Instance, FRenderableComponents& OutComponents) const override
		{
			const UDataflowUVsRenderSettings* Settings = Instance.GetTypedRenderSettings<UDataflowUVsRenderSettings>();

			const FManagedArrayCollection& Collection = GetCollection(Instance);

			const int32 UVChannel = Settings
				? Settings->GetUVChannel(Rendering::GetUVChannelFromInstance(Instance))
				: 0;
			Rendering::AddUvDynamicMeshComponent(Collection, UVChannel, OutComponents);
		}
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

	class FGeometryCollectionAssetRenderableType : public IRenderableType
	{
		UE_DATAFLOW_IRENDERABLE_PRIMARY_TYPE(TObjectPtr<UGeometryCollection>, GeometryCollectionAsset);
		UE_DATAFLOW_IRENDERABLE_RENDER_GROUP(Asset);
		UE_DATAFLOW_IRENDERABLE_VIEW_MODE(FDataflowConstruction3DViewMode);

		virtual bool CanRender(const FRenderableTypeInstance& Instance) const
		{
			const TObjectPtr<UGeometryCollection> AssetPtr = GetGeometryCollectionAsset(Instance);
			return AssetPtr && !AssetPtr->IsEmpty();
		}

		virtual void GetPrimitiveComponents(const FRenderableTypeInstance& Instance, FRenderableComponents& OutComponents) const override
		{
			if (const TObjectPtr<UGeometryCollection> AssetPtr = GetGeometryCollectionAsset(Instance))
			{
				const FName ComponentName = Instance.GetComponentName(TEXT("Geometry Collection Asset"));

				if (UGeometryCollectionComponent* Component = OutComponents.AddNewComponent<UGeometryCollectionComponent>(ComponentName))
				{
					Component->SetCastShadow(false);
					Component->SetSimulatePhysics(false);
					Component->SetRestCollection(AssetPtr.Get());
				}
			}
		}
	};

	///////////////////////////////////////////////////////////////////////////////////////////////////////////////

	void RegisterGeometryCollectionRenderableTypes()
	{
		UE_DATAFLOW_REGISTER_RENDERABLE_TYPE(FGeometryCollectionSurfaceRenderableType);
		UE_DATAFLOW_REGISTER_RENDERABLE_TYPE(FGeometryCollectionUvsRenderableType);
		UE_DATAFLOW_REGISTER_RENDERABLE_TYPE(FGeometryCollectionAssetRenderableType);

		static const FName SurfaceCategoryName(TEXT("Surface"));

		UDataflowRenderableTypeSettings::RegisterSection(
			UDataflowGeometryCollectionSurfaceRenderSettings::StaticClass(),
			"Collection", LOCTEXT("Collection", "Collection"),
			{ SurfaceCategoryName }
		);
	}
}

#undef LOCTEXT_NAMESPACE 
