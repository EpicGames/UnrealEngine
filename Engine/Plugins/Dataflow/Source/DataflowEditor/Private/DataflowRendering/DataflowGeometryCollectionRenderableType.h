// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataflowRendering/DataflowRenderableTypeSettings.h"
#include "Curves/LinearColorRamp.h"
#include "UObject/ObjectPtr.h"

#include "DataflowGeometryCollectionRenderableType.generated.h"

class UMaterialInterface;
struct FManagedArrayCollection;
namespace GeometryCollection::Facades { class FBoundsFacade; }

UCLASS(MinimalAPI, AutoExpandCategories = "Surface|Selection, Surface|Attribute, Surface|Curves")
class UDataflowGeometryCollectionSurfaceRenderSettings : public UDataflowRenderableTypeSettings
{
	GENERATED_BODY()

	UDataflowGeometryCollectionSurfaceRenderSettings(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

public:
	/** Show selected transforms when relevant to the selected node */
	UPROPERTY(EditAnywhere, Category = "Surface|Selection")
	bool bShowSelection = true;

	/** Material used to display the transform selection */
	UPROPERTY(EditAnywhere, Category = "Surface|Selection", meta = (EditCondition = bShowSelection))
	TObjectPtr<UMaterialInterface> SelectionMaterial = nullptr;

	/** Show vertex attribute if available from the node or the override below */
	UPROPERTY(EditAnywhere, Category = "Surface|Attribute")
	bool bShowVertexAttributeAsVertexColor = true;

	/** control whether the attribute override shoudl be used or not */
	UPROPERTY(EditAnywhere, Category = "Surface|Attribute", meta = (InlineEditConditionToggle, EditCondition = bShowVertexAttributeAsVertexColor))
	bool bUseVertexAttributeOverride = false;

	/** Vertex attribute to display - if not empty this will override any attribute displayed for the node */
	UPROPERTY(EditAnywhere, Category = "Surface|Attribute", meta = (EditCondition = bUseVertexAttributeOverride))
	FString VertexAttributeOverride;

	/** If true use the color ramp to color vertices from the attribute */
	UPROPERTY(EditAnywhere, Category = "Surface|Attribute", meta = (EditCondition = bShowVertexAttributeAsVertexColor))
	bool bUseColorRamp = false;

	/** Color ramp to use to display an single value attribute - ignored for vector based attributes */
	UPROPERTY(EditAnywhere, Category = "Surface|Attribute", meta = (EditCondition = "bShowVertexAttributeAsVertexColor && bUseColorRamp"))
	FLinearColorRamp ColorRamp;

	/** Maxmimum number of vertixes to render for curves (used for groom) , negative number to not cull any curves */
	UPROPERTY(EditAnywhere, Category = "Surface|Curves")
	int32 MaxCurveVertices = 50000;


	const int32 GetHashFromProperties() const ;
};

UCLASS(MinimalAPI, AutoExpandCategories = "ExplodedView")
class UDataflowExplodedViewRenderSettings : public UDataflowRenderableTypeSettings
{
	GENERATED_BODY()

public:
	/** Apply exploded view to pieces */
	UPROPERTY(EditAnywhere, Category = "ExplodedView")
	bool bExplodedView = false;

	/** Amount for the exploded view */
	UPROPERTY(EditAnywhere, Category = "ExplodedView", meta = (EditCondition = bExplodedView))
	float Amount = 1.f;

	/** Offset to the center of the exploded view */
	UPROPERTY(EditAnywhere, Category = "ExplodedView", meta = (EditCondition = bExplodedView))
	FVector CenterOffset = FVector::ZeroVector;

	FVector ComputeExplodedVector(const FVector& Point, const FVector& Center) const;
	void ComputePerTransformExplodedVectors(const FManagedArrayCollection& Collection, TArray<FVector>& OutExplodedVectors) const;
	void ComputePerTransformExplodedVectors(const GeometryCollection::Facades::FBoundsFacade& BoundsFacade, TArray<FVector>& OutExplodedVectors) const;
	
};

namespace UE::Dataflow::Private
{
	void RegisterGeometryCollectionRenderableTypes();
}
