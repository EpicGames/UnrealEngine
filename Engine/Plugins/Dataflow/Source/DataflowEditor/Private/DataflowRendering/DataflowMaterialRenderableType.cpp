// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataflowRendering/DataflowMaterialRenderableType.h"

#include "Components/InstancedStaticMeshComponent.h"

#include "Dataflow/DataflowRenderingViewMode.h"

#include "DataflowRendering/DataflowRenderableComponents.h"
#include "DataflowRendering/DataflowRenderableType.h"
#include "DataflowRendering/DataflowRenderableTypeInstance.h"
#include "DataflowRendering/DataflowRenderableTypeRegistry.h"
#include "DataflowRendering/DataflowRenderableTypeUtils.h"

#include "Components/StaticMeshComponent.h"
#include "Materials/MaterialInterface.h"

#include "Engine/StaticMesh.h"

#include "UObject/ObjectPtr.h"

namespace UE::Dataflow::Private
{
	///////////////////////////////////////////////////////////////////////////////////////////////////////////////

	static UStaticMesh* LoadMaterialPreviewMesh()
	{
		return LoadObject<UStaticMesh>(nullptr, TEXT("/Script/Engine.StaticMesh'/Engine/EngineMeshes/SM_MatPreviewMesh_01.SM_MatPreviewMesh_01'"));
	}

	class FMaterialSurfaceRenderableType : public IRenderableType
	{
		UE_DATAFLOW_IRENDERABLE_PRIMARY_TYPE(TObjectPtr<UMaterialInterface>, Material);
		UE_DATAFLOW_IRENDERABLE_RENDER_GROUP(Material);
		UE_DATAFLOW_IRENDERABLE_VIEW_MODE(FDataflowConstructionMaterialViewMode);

		virtual void GetPrimitiveComponents(const FRenderableTypeInstance& Instance, FRenderableComponents& OutComponents) const override
		{
			const TObjectPtr<UMaterialInterface> Material = GetMaterial(Instance, {});

			const FName MaterialComponentName = Instance.GetComponentName(TEXT("Material"));

			if (UStaticMeshComponent* Component = OutComponents.AddNewComponent<UStaticMeshComponent>(MaterialComponentName))
			{
				//LoadMaterialPreviewMesh
				Component->SetStaticMesh(LoadMaterialPreviewMesh());
				Component->SetMaterial(0, Material);
			}
		}
	};

	///////////////////////////////////////////////////////////////////////////////////////////////////////////////

	class FMaterialArraySurfaceRenderableType : public IRenderableType
	{
		UE_DATAFLOW_IRENDERABLE_PRIMARY_TYPE(TArray<TObjectPtr<UMaterialInterface>>, MaterialArray);
		UE_DATAFLOW_IRENDERABLE_RENDER_GROUP(Material);
		UE_DATAFLOW_IRENDERABLE_VIEW_MODE(FDataflowConstructionMaterialViewMode);

		virtual void GetPrimitiveComponents(const FRenderableTypeInstance& Instance, FRenderableComponents& OutComponents) const override
		{
			const TArray<TObjectPtr<UMaterialInterface>>& MaterialArray = GetMaterialArray(Instance, {});
			const int32 NumMaterials = MaterialArray.Num();
			if (NumMaterials > 0)
			{
				UStaticMesh* MaterialPreviewMesh = LoadMaterialPreviewMesh();
				const FVector PreviewBoundsSize = MaterialPreviewMesh->GetBoundingBox().GetSize();

				const int32 NumMaterialPerDimension = FMath::Max(0, FMath::CeilToInt32(FMath::Sqrt((float)NumMaterials)));
				for (int32 Y = 0; Y < NumMaterialPerDimension; ++Y)
				{
					for (int32 X = 0; X < NumMaterialPerDimension; ++X)
					{
						const int32 MaterialIndex = X + (Y * NumMaterialPerDimension);
						if (MaterialArray.IsValidIndex(MaterialIndex))
						{
							const FName ComponentName = FName(FString::Printf(TEXT("Material [%01d]"), MaterialIndex));
							if (UStaticMeshComponent* Component = OutComponents.AddNewComponent<UStaticMeshComponent>(ComponentName))
							{
								Component->SetStaticMesh(MaterialPreviewMesh);
								Component->SetMaterial(0, MaterialArray[MaterialIndex]);

								const double SpacingFactor = 1.1f;
								const FVector Position
								{
									 PreviewBoundsSize.X * SpacingFactor * (double)X,
									 PreviewBoundsSize.Y * SpacingFactor * (double)Y,
									 0
								};
								Component->SetWorldTransform(FTransform(FQuat::Identity, Position));
							}
						}
					}
				}
			}
		}
	};

	///////////////////////////////////////////////////////////////////////////////////////////////////////////////

	void RegisterMaterialRenderableTypes()
	{
		UE_DATAFLOW_REGISTER_RENDERABLE_TYPE(FMaterialSurfaceRenderableType);
		UE_DATAFLOW_REGISTER_RENDERABLE_TYPE(FMaterialArraySurfaceRenderableType);
	}

	///////////////////////////////////////////////////////////////////////////////////////////////////////////////

}