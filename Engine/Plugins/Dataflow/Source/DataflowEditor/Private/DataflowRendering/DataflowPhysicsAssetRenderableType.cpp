// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataflowRendering/DataflowPhysicsAssetRenderableType.h"

#include "AnimationRuntime.h"
#include "Components/StaticMeshComponent.h"
#include "Dataflow/DataflowEngineUtil.h"
#include "Dataflow/DataflowRenderingViewMode.h"
#include "DataflowRendering/DataflowRenderableComponents.h"
#include "DataflowRendering/DataflowRenderableType.h"
#include "DataflowRendering/DataflowRenderableTypeInstance.h"
#include "DataflowRendering/DataflowRenderableTypeRegistry.h"
#include "DataflowRendering/DataflowRenderableTypeUtils.h"
#include "Engine/SkeletalMesh.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "PhysicsEngine/AggregateGeom.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "PhysicsEngine/SkeletalBodySetup.h"
#include "ReferenceSkeleton.h"
#include "UObject/ObjectPtr.h"

UDataflowPhysicsAssetRenderSettings::UDataflowPhysicsAssetRenderSettings(const FObjectInitializer& ObjectInitializer)
{
	ColorSettings.Color = FLinearColor(0.5f, 1.0f, 1.0f);
	ColorSettings.Transparency = 0.5;
}

namespace UE::Dataflow::Private
{
namespace
{

// Compute component-space reference pose transforms.
// Returns an empty array if PreviewSkeletalMesh is null.
static TArray<FTransform> ComputeReferencePoseCS(const USkeletalMesh* Mesh)
{
	TArray<FTransform> Out;
	if (Mesh)
	{
		const FReferenceSkeleton& RefSkel = Mesh->GetRefSkeleton();
		FAnimationRuntime::FillUpComponentSpaceTransforms(RefSkel, RefSkel.GetRefBonePose(), Out);
	}
	return Out;
}

// create a material instance based on the settings 
static UMaterialInstanceDynamic* CreateMaterialInstance(UObject* Owner, const UDataflowPhysicsAssetRenderSettings* Settings)
{
	UMaterialInstanceDynamic* MaterialInstance = nullptr;

	UMaterialInterface* Material = Settings && Settings->ColorSettings.bWireframe
		? UE::Dataflow::RenderMaterial::GetDataflowColorWireframeMaterial()
		: UE::Dataflow::RenderMaterial::GetDataflowColorMaterial()
		;
	if (Material)
	{
		MaterialInstance = UMaterialInstanceDynamic::Create(Material, Owner);
		if (MaterialInstance)
		{
			static const FName ColorParameterName("Color");
			static const FName TransparencyParameterName("Transparency");
			MaterialInstance->SetVectorParameterValue(ColorParameterName, Settings? Settings->ColorSettings.Color: FLinearColor::White);
			MaterialInstance->SetScalarParameterValue(TransparencyParameterName, Settings ? Settings->ColorSettings.Transparency: 0.f);
		}
	}
	return MaterialInstance;
}

} // anonymous namespace

///////////////////////////////////////////////////////////////////////////////////////////////////////////////

class FPhysicsAssetSurfaceRenderableType : public IRenderableType
{
	UE_DATAFLOW_IRENDERABLE_PRIMARY_TYPE(TObjectPtr<UPhysicsAsset>, PhysicsAsset);
	UE_DATAFLOW_IRENDERABLE_RENDER_GROUP(Surface);
	UE_DATAFLOW_IRENDERABLE_VIEW_MODE(FDataflowConstruction3DViewMode);
	UE_DATAFLOW_IRENDERABLE_SETTINGS(UDataflowPhysicsAssetRenderSettings);

	virtual void GetPrimitiveComponents(const FRenderableTypeInstance& Instance, FRenderableComponents& OutComponents) const override
	{
		const UDataflowPhysicsAssetRenderSettings* Settings = Instance.GetTypedRenderSettings<UDataflowPhysicsAssetRenderSettings>();

		const TObjectPtr<const UPhysicsAsset> PhysicsAsset = GetPhysicsAsset(Instance, nullptr);
		if (!PhysicsAsset)
		{
			return;
		}

		static FName BaseParentName = TEXT("PhysicsAsset");
		UPrimitiveComponent* ParentComponent = OutComponents.AddNewComponent<UStaticMeshComponent>(BaseParentName);
		if (!ParentComponent)
		{
			return;
		}

		UMaterialInstanceDynamic* MaterialInstance = CreateMaterialInstance(ParentComponent, Settings);

		// Compute reference-pose bone transforms. Shapes without a valid preview mesh
		// are placed at the origin (BoneCS = Identity).
		const USkeletalMesh* PreviewMesh = PhysicsAsset->PreviewSkeletalMesh.Get();
		const TArray<FTransform> CSTransforms = ComputeReferencePoseCS(PreviewMesh);

		int32 ShapeCounter = 0;

		for (int32 BodyIdx = 0; BodyIdx < PhysicsAsset->SkeletalBodySetups.Num(); ++BodyIdx)
		{
			const USkeletalBodySetup* BodySetup = PhysicsAsset->SkeletalBodySetups[BodyIdx];
			if (!BodySetup)
			{
				continue;
			}

			// Bone component-space transform (identity if mesh is absent or bone not found)
			const int32 BoneIdx = PreviewMesh
				? PreviewMesh->GetRefSkeleton().FindBoneIndex(BodySetup->BoneName)
				: INDEX_NONE;
			const FTransform BoneCS = (BoneIdx != INDEX_NONE) 
				? CSTransforms[BoneIdx] 
				: FTransform::Identity;

			static const FName UnknownBoneName(TEXT("[Unknown]"));
			const FName BoneName = (BoneIdx != INDEX_NONE)
				? PreviewMesh->GetRefSkeleton().GetBoneName(BoneIdx)
				: UnknownBoneName;

			UE::Dataflow::RenderGeometry::AddAggGeomComponents(
				BodySetup->AggGeom, BoneCS, BoneName,
				OutComponents, ParentComponent, MaterialInstance, ShapeCounter);
		}
	}
};

///////////////////////////////////////////////////////////////////////////////////////////////////////////////

void RegisterPhysicsAssetRenderableTypes()
{
	UE_DATAFLOW_REGISTER_RENDERABLE_TYPE(FPhysicsAssetSurfaceRenderableType);
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Dataflow::Private
