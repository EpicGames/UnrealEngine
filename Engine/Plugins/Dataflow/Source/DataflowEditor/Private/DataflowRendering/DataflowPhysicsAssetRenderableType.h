// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataflowRendering/DataflowRenderableTypeSettings.h"
#include "DataflowRendering/DataflowRenderableTypeUtils.h"

#include "DataflowPhysicsAssetRenderableType.generated.h"

UCLASS(MinimalAPI)
class UDataflowPhysicsAssetRenderSettings : public UDataflowRenderableTypeSettings
{
	GENERATED_BODY()

	UDataflowPhysicsAssetRenderSettings(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

public:
	UPROPERTY(EditAnywhere, Category = "Surface", meta = (ShowOnlyInnerProperties))
	FDataflowSimpleColorCommonRenderSettings ColorSettings;
};

namespace UE::Dataflow::Private
{
	void RegisterPhysicsAssetRenderableTypes();
}
