// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataflowRendering/DataflowPlaneRenderableType.h"
#include "DataflowRendering/DataflowRenderableTypeUtils.h"

#include "Components/BoxComponent.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "Drawing/LineSetComponent.h"
#include "Components/ArrowComponent.h"

#include "Dataflow/DataflowRenderingViewMode.h"
#include "Dataflow/DataflowEngineUtil.h"
#include "Dataflow/DataflowPlane.h"

#include "DataflowRendering/DataflowRenderableComponents.h"
#include "DataflowRendering/DataflowRenderableType.h"
#include "DataflowRendering/DataflowRenderableTypeInstance.h"
#include "DataflowRendering/DataflowRenderableTypeRegistry.h"
#include "DataflowRendering/DataflowRenderableTypeUtils.h"

#include "Engine/StaticMesh.h"

#include "UObject/ObjectPtr.h"

#include "Materials/MaterialInstanceDynamic.h"

namespace UE::Dataflow::Private
{
	///////////////////////////////////////////////////////////////////////////////////////////////////////////////

	class FPlaneSurfaceRenderableType : public IRenderableType
	{
		UE_DATAFLOW_IRENDERABLE_PRIMARY_TYPE(FDataflowPlane, Plane);
		UE_DATAFLOW_IRENDERABLE_RENDER_GROUP(Surface);
		UE_DATAFLOW_IRENDERABLE_VIEW_MODE(FDataflowConstruction3DViewMode);
		UE_DATAFLOW_IRENDERABLE_SETTINGS(UDataflowPlaneRenderSettings);

	private:
		virtual void GetPrimitiveComponents(const FRenderableTypeInstance& Instance, FRenderableComponents& OutComponents) const override
		{
			const FDataflowPlane Plane = GetPlane(Instance, {});

			const FName PlaneComponentName = Instance.GetComponentName(TEXT("Plane"));
			const FName ArrowComponentName = Instance.GetComponentName(TEXT("PlaneNormal"));

			UStaticMesh* PlaneMesh = UE::Dataflow::RenderGeometry::GetDataflowPlane();
			if (PlaneMesh)
			{
				const UDataflowPlaneRenderSettings* Settings = Instance.GetTypedRenderSettings<UDataflowPlaneRenderSettings>();
				if (Settings)
				{
					if (UStaticMeshComponent* Component = OutComponents.AddNewComponent<UStaticMeshComponent>(PlaneComponentName))
					{
						Component->SetStaticMesh(PlaneMesh);

						FTransform PlaneTransform = Plane.AsTransform();
						PlaneTransform.SetScale3D(FVector(Settings->PlaneScale));

						Component->SetWorldTransform(PlaneTransform);
						Component->SetCollisionEnabled(ECollisionEnabled::NoCollision);

						UMaterialInterface* Material = Settings->ColorSettings.bWireframe
							? UE::Dataflow::RenderMaterial::GetDataflowColorWireframeMaterial(true)
							: UE::Dataflow::RenderMaterial::GetDataflowColorMaterial(true);

						UMaterialInstanceDynamic* MaterialInstance = nullptr;

						if (Material)
						{
							MaterialInstance = UMaterialInstanceDynamic::Create(Material, Component);
							if (MaterialInstance)
							{
								MaterialInstance->SetVectorParameterValue(FName("Color"), Settings->ColorSettings.Color);
								MaterialInstance->SetScalarParameterValue(FName("Transparency"), Settings->ColorSettings.Transparency);
							}
						}

						Component->SetMaterial(0, MaterialInstance);

						if (UArrowComponent* ArrowComponent = OutComponents.AddNewComponent<UArrowComponent>(ArrowComponentName))
						{
							ArrowComponent->ArrowColor = Settings->ColorSettings.Color.ToFColor(true);
							ArrowComponent->ArrowSize = Settings->NormalScale;

							FTransform XToZTransform = FTransform(FRotator(90.f, 0.f, 0.f));
							ArrowComponent->SetWorldTransform(XToZTransform * Plane.AsTransform());

							ArrowComponent->SetMaterial(0, MaterialInstance);
						}
					}
				}
			}
		}
	};
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////

namespace UE::Dataflow::Private
{
	void RegisterPlaneRenderableTypes()
	{
		UE_DATAFLOW_REGISTER_RENDERABLE_TYPE(FPlaneSurfaceRenderableType);
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////
