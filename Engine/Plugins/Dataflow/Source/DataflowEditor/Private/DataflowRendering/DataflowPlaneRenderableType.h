// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataflowRendering/DataflowRenderableTypeSettings.h"
#include "DataflowRendering/DataflowRenderableTypeUtils.h"

#include "DataflowPlaneRenderableType.generated.h"

UCLASS(MinimalAPI)
class UDataflowPlaneRenderSettings : public UDataflowRenderableTypeSettings
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = "Surface", meta = (ShowOnlyInnerProperties))
	FDataflowSimpleColorCommonRenderSettings ColorSettings;

	/** Length scalar for the normal */
	UPROPERTY(EditAnywhere, Category = "Plane", meta = (UIMin = "1.0", UIMax = "10.0", ClampMin = "1.0", ClampMax = "10.0"))
	float NormalScale = 1.f;

	/** Plane size scalar */
	UPROPERTY(EditAnywhere, Category = "Plane", meta = (UIMin = "1.0", UIMax = "10.0", ClampMin = "1.0", ClampMax = "10.0"))
	float PlaneScale = 1.f;
};

namespace UE::Dataflow::Private
{
	void RegisterPlaneRenderableTypes();
}



