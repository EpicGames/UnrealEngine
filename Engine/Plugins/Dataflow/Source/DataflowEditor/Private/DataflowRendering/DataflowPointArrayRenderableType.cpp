// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataflowRendering/DataflowPointArrayRenderableType.h"

#include "Drawing/PointSetComponent.h"
#include "Components/InstancedStaticMeshComponent.h"

#include "Dataflow/DataflowRenderingViewMode.h"

#include "DataflowRendering/DataflowRenderableComponents.h"
#include "DataflowRendering/DataflowRenderableType.h"
#include "DataflowRendering/DataflowRenderableTypeInstance.h"
#include "DataflowRendering/DataflowRenderableTypeRegistry.h"
#include "DataflowRendering/DataflowRenderableTypeUtils.h"

#include "Materials/MaterialInterface.h"
#include "Materials/Material.h"

#include "UObject/ObjectPtr.h"
#include "GeometryCollection/Facades/PointsFacade.h"
#include "Dataflow/DataflowPoints.h"

namespace UE::Dataflow::Private
{
	///////////////////////////////////////////////////////////////////////////////////////////////////////////////

	class FPointArraySurfaceRenderableType : public IRenderableType
	{
		UE_DATAFLOW_IRENDERABLE_PRIMARY_TYPE(TArray<FVector>, PointArray);
		UE_DATAFLOW_IRENDERABLE_RENDER_GROUP(Points);
		UE_DATAFLOW_IRENDERABLE_VIEW_MODE(FDataflowConstruction3DViewMode);
		UE_DATAFLOW_IRENDERABLE_SETTINGS(UDataflowPointArrayRenderSettings);

	public:
		FPointArraySurfaceRenderableType()
		{
			constexpr bool bDepthTested = false;
			Material = (bDepthTested)
				? LoadObject<UMaterial>(nullptr, TEXT("/MeshModelingToolsetExp/Materials/PointSetComponentMaterial"))
				: LoadObject<UMaterial>(nullptr, TEXT("/MeshModelingToolsetExp/Materials/PointSetOverlaidComponentMaterial"));
		}

		virtual void GetPrimitiveComponents(const FRenderableTypeInstance& Instance, FRenderableComponents& OutComponents) const override
		{
			const UDataflowPointArrayRenderSettings* Settings = Instance.GetTypedRenderSettings<UDataflowPointArrayRenderSettings>();
			if (Settings)
			{
				const TArray<FVector>& PointArray = GetPointArray(Instance, {});

				const FName PointComponentName = Instance.GetComponentName(TEXT("Points"));

				const int32 NumPoints = PointArray.Num();

				if (UPointSetComponent* PointComponent = OutComponents.AddNewComponent<UPointSetComponent>(PointComponentName))
				{
					PointComponent->ReservePoints(NumPoints);
					PointComponent->AddPoints(PointArray, Settings->Color, Settings->Size);

					PointComponent->SetPointMaterial(Material);
				}
			}
		}

		UMaterialInterface* Material = nullptr;
	};

	///////////////////////////////////////////////////////////////////////////////////////////////////////////////

	void RegisterPointArrayRenderableTypes()
	{
		UE_DATAFLOW_REGISTER_RENDERABLE_TYPE(FPointArraySurfaceRenderableType);
	}

	///////////////////////////////////////////////////////////////////////////////////////////////////////////////

}