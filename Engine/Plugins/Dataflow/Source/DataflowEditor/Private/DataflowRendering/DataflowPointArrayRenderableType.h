// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataflowRendering/DataflowRenderableTypeSettings.h"
#include "DataflowRendering/DataflowRenderableTypeUtils.h"

#include "DataflowPointArrayRenderableType.generated.h"

UCLASS(MinimalAPI, AutoExpandCategories = "")
class UDataflowPointArrayRenderSettings : public UDataflowRenderableTypeSettings
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, Category = "Point")
	float Size = 5.0f;

	UPROPERTY(EditAnywhere, Category = "Point")
	FColor Color = FColor::Blue;
};

namespace UE::Dataflow::Private
{
	void RegisterPointArrayRenderableTypes();
}

