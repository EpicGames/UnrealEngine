// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataflowRendering/DataflowPointRenderableType.h"

#include "Drawing/PointSetComponent.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "Drawing/LineSetComponent.h"

#include "Dataflow/DataflowRenderingViewMode.h"

#include "DataflowRendering/DataflowRenderableComponents.h"
#include "DataflowRendering/DataflowRenderableType.h"
#include "DataflowRendering/DataflowRenderableTypeInstance.h"
#include "DataflowRendering/DataflowRenderableTypeRegistry.h"
#include "DataflowRendering/DataflowRenderableTypeUtils.h"

#include "Materials/MaterialInterface.h"
#include "Materials/Material.h"

#include "GeometryCollection/Facades/PointsFacade.h"

#include "UObject/ObjectPtr.h"
#include "Dataflow/DataflowPoints.h"
#include "DataflowRendering/DataflowDebugMeshRenderableType.h"

UDataflowPointRenderSettings::UDataflowPointRenderSettings(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	constexpr bool bOnlyRGB = true;
	ColorRamp.SetColorAtTime(0.0f, FLinearColor::Blue, bOnlyRGB);
	ColorRamp.SetColorAtTime(1.0f, FLinearColor::Red, bOnlyRGB);

	LengthColorRamp.SetColorAtTime(0.0f, FLinearColor::Green, bOnlyRGB);
	LengthColorRamp.SetColorAtTime(1.0f, FLinearColor::Red, bOnlyRGB);
}

namespace UE::Dataflow::Private
{
	///////////////////////////////////////////////////////////////////////////////////////////////////////////////

	class FPointSurfaceRenderableType : public IRenderableType
	{
		UE_DATAFLOW_IRENDERABLE_PRIMARY_TYPE(FDataflowPoints, PointArray);
		UE_DATAFLOW_IRENDERABLE_RENDER_GROUP(Points);
		UE_DATAFLOW_IRENDERABLE_VIEW_MODE(FDataflowConstruction3DViewMode);
		UE_DATAFLOW_IRENDERABLE_SETTINGS(UDataflowPointRenderSettings);

	public:
		FPointSurfaceRenderableType()
		{
			constexpr bool bDepthTested = false;
			Material = (bDepthTested)
				? LoadObject<UMaterial>(nullptr, TEXT("/MeshModelingToolsetExp/Materials/PointSetComponentMaterial"))
				: LoadObject<UMaterial>(nullptr, TEXT("/MeshModelingToolsetExp/Materials/PointSetOverlaidComponentMaterial"));
		}

		virtual void GetPrimitiveComponents(const FRenderableTypeInstance& Instance, FRenderableComponents& OutComponents) const override
		{
			const UDataflowPointRenderSettings* Settings = Instance.GetTypedRenderSettings<UDataflowPointRenderSettings>();
			if (Settings)
			{
				const FDataflowPoints& Points = GetPointArray(Instance, {});

				const FName PointComponentName = Instance.GetComponentName(TEXT("Points"));

				GeometryCollection::Facades::FPointsFacade PointFacadeOutPoints = Points.GetPointsFacade();

				const TArray<FVector>& PointArray = PointFacadeOutPoints.GetPointsAsArray();

				const int32 NumPoints = PointArray.Num();

				TArray<FRenderablePoint> RenderablePoints;
				RenderablePoints.SetNumUninitialized(NumPoints);

				GeometryCollection::Facades::FPointsFacade PointFacade = Points.GetPointsFacade();

				// Get Attribute name
				FName AttributeName = FName(*Settings->Attribute);

				if (AttributeName == NAME_None)
				{
					FDataflowNode::FAttributeKey AttributeKey = Instance.GetVertexAttributeToVisualize();

					AttributeName = AttributeKey.AttributeName;
				}

				if (Settings->bRenderPoints)
				{
					if (PointFacade.HasAttribute(AttributeName))
					{
						if (PointFacade.GetAttributeType(AttributeName) == EManagedArrayType::FFloatType)
						{
							TArray<float> AttrValues = PointFacade.GetFloatAttributeValues(AttributeName);
							{
								for (int32 Idx = 0; Idx < NumPoints; ++Idx)
								{
									float AttrValue = AttrValues[Idx];

									if (AttrValue < Settings->Min)
									{
										AttrValue = Settings->Min;
									}
									else if (AttrValue > Settings->Max)
									{
										AttrValue = Settings->Max;
									}

									FLinearColor LinearColor = Settings->Color;
									FColor Color = LinearColor.ToFColor(true).WithAlpha(255);

									if (Settings->Max - Settings->Min > UE_SMALL_NUMBER)
									{
										float NormAttrValue = (AttrValue - Settings->Min) / (Settings->Max - Settings->Min);

										LinearColor = Settings->ColorRamp.GetLinearColorValue(NormAttrValue);
										Color = LinearColor.ToFColor(true).WithAlpha(255);
									}

									RenderablePoints[Idx] = FRenderablePoint(PointArray[Idx], Color, Settings->Size);
								}
							}
						}
						else if (PointFacade.GetAttributeType(AttributeName) == EManagedArrayType::FInt32Type)
						{
							TArray<int32> AttrValues = PointFacade.GetIntAttributeValues(AttributeName);
							{
								for (int32 Idx = 0; Idx < NumPoints; ++Idx)
								{
									float AttrValue = float(AttrValues[Idx]);

									if (AttrValue < Settings->Min)
									{
										AttrValue = Settings->Min;
									}
									else if (AttrValue > Settings->Max)
									{
										AttrValue = Settings->Max;
									}

									FLinearColor LinearColor = Settings->Color;
									FColor Color = LinearColor.ToFColor(true).WithAlpha(255);

									if (Settings->Max - Settings->Min > UE_SMALL_NUMBER)
									{
										float NormAttrValue = (AttrValue - Settings->Min) / (Settings->Max - Settings->Min);

										LinearColor = Settings->ColorRamp.GetLinearColorValue(NormAttrValue);
										Color = LinearColor.ToFColor(true).WithAlpha(255);
									}

									RenderablePoints[Idx] = FRenderablePoint(PointArray[Idx], Color, Settings->Size);
								}
							}
						}
						else if (PointFacade.GetAttributeType(AttributeName) == EManagedArrayType::FBoolType)
						{
							TBitArray<> AttrValues = PointFacade.GetBoolAttributeValues(AttributeName);
							{
								for (int32 Idx = 0; Idx < NumPoints; ++Idx)
								{
									float AttrValue = AttrValues[Idx] ? 1.f : 0.f;

									if (AttrValue < Settings->Min)
									{
										AttrValue = Settings->Min;
									}
									else if (AttrValue > Settings->Max)
									{
										AttrValue = Settings->Max;
									}

									FLinearColor LinearColor = Settings->Color;
									FColor Color = LinearColor.ToFColor(true).WithAlpha(255);

									if (Settings->Max - Settings->Min > UE_SMALL_NUMBER)
									{
										float NormAttrValue = (AttrValue - Settings->Min) / (Settings->Max - Settings->Min);

										LinearColor = Settings->ColorRamp.GetLinearColorValue(NormAttrValue);
										Color = LinearColor.ToFColor(true).WithAlpha(255);
									}

									RenderablePoints[Idx] = FRenderablePoint(PointArray[Idx], Color, Settings->Size);
								}
							}
						}
						else if (PointFacade.GetAttributeType(AttributeName) == EManagedArrayType::FVectorType)
						{
							TArray<FVector3f> AttrValues = PointFacade.GetVector3AttributeValues(AttributeName);
							{
								if (Settings->bRenderAsRGB)
								{
									for (int32 Idx = 0; Idx < NumPoints; ++Idx)
									{
										FLinearColor LinearColor = FLinearColor(AttrValues[Idx].X, AttrValues[Idx].Y, AttrValues[Idx].Z);
										FColor Color = LinearColor.ToFColor(true).WithAlpha(255);

										RenderablePoints[Idx] = FRenderablePoint(PointArray[Idx], Color, Settings->Size);
									}
								}
								else
								{
									if (Settings->bRenderPoints)
									{
										for (int32 Idx = 0; Idx < NumPoints; ++Idx)
										{
											float AttrValue = AttrValues[Idx].Length();

											if (AttrValue < Settings->Min)
											{
												AttrValue = Settings->Min;
											}
											else if (AttrValue > Settings->Max)
											{
												AttrValue = Settings->Max;
											}

											FLinearColor LinearColor = Settings->Color;
											FColor Color = LinearColor.ToFColor(true).WithAlpha(255);

											if (Settings->Max - Settings->Min > UE_SMALL_NUMBER)
											{
												float NormAttrValue = (AttrValue - Settings->Min) / (Settings->Max - Settings->Min);

												LinearColor = Settings->ColorRamp.GetLinearColorValue(NormAttrValue);
												Color = LinearColor.ToFColor(true).WithAlpha(255);
											}

											RenderablePoints[Idx] = FRenderablePoint(PointArray[Idx], Color, Settings->Size);
										}
									}

									// Render vectors
									static const FName LineComponentName = TEXT("Vectors");

									if (ULineSetComponent* LineComponent = OutComponents.AddNewComponent<ULineSetComponent>(LineComponentName))
									{
										TArray<FRenderableLine> RenderableLines;
										RenderableLines.SetNumUninitialized(NumPoints);

										for (int32 Idx = 0; Idx < NumPoints; ++Idx)
										{
											float Length = AttrValues[Idx].Length();

											if (Length < Settings->LengthMin)
											{
												Length = Settings->LengthMin;
											}
											else if (Length > Settings->LengthMax)
											{
												Length = Settings->LengthMax;
											}

											FLinearColor LinearColor = Settings->Color;
											FColor Color = LinearColor.ToFColor(true).WithAlpha(255);

											if (Settings->LengthMax - Settings->LengthMin > UE_SMALL_NUMBER)
											{
												float NormAttrValue = (Length - Settings->LengthMin) / (Settings->LengthMax - Settings->LengthMin);

												LinearColor = Settings->LengthColorRamp.GetLinearColorValue(NormAttrValue);
												Color = LinearColor.ToFColor(true).WithAlpha(255);
											}

											RenderableLines[Idx] = FRenderableLine(PointArray[Idx], PointArray[Idx] + Settings->LengthScalar * FVector(AttrValues[Idx]), Color, Settings->LineThickness);
										}

										LineComponent->AddLines(RenderableLines);
										LineComponent->SetLineMaterial(Material);
									}
								}
							}
						}
					}
					else
					{
						for (int32 Idx = 0; Idx < NumPoints; ++Idx)
						{
							RenderablePoints[Idx] = FRenderablePoint(PointArray[Idx], Settings->Color, Settings->Size);
						}
					}
				}
				else
				{
					RenderablePoints.SetNumUninitialized(0);
				}

				// Render points
				if (NumPoints > 0 && RenderablePoints.Num() > 0)
				{
					if (UPointSetComponent* PointComponent = OutComponents.AddNewComponent<UPointSetComponent>(PointComponentName))
					{
						PointComponent->ReservePoints(NumPoints);
						PointComponent->AddPoints(RenderablePoints);

						PointComponent->SetPointMaterial(Material);

						// Display point IDs
						static FName DebugMeshComponentBaseName = TEXT("DebugInfo");
						if (UDataflowDebugMeshComponent* DebugDrawComponent = OutComponents.AddNewComponent<UDataflowDebugMeshComponent>(DebugMeshComponentBaseName, PointComponent))
						{
							if (Settings->bShowPointIDs)
							{
								DebugDrawComponent->IDs.Reserve(NumPoints);
								for (int32 Idx = 0; Idx < NumPoints; ++Idx)
								{
									UDataflowDebugMeshComponent::FText3d Text;
									Text.Color = Settings->IDsColor;
									Text.Location = RenderablePoints[Idx].Position + FVector(4.0, 0.0, -4.0);
									Text.Text = FString::FromInt(Idx);
									DebugDrawComponent->IDs.Add(MoveTemp(Text));
								}
							}

							if (Settings->bShowPositions)
							{
								DebugDrawComponent->IDs.Reserve(DebugDrawComponent->IDs.Num() + NumPoints);
								for (int32 Idx = 0; Idx < NumPoints; ++Idx)
								{
									UDataflowDebugMeshComponent::FText3d Text;
									Text.Color = Settings->PositionsColor;
									Text.Location = RenderablePoints[Idx].Position + FVector(4.0, 0.0, -4.0);
									Text.Text = FString::Printf(TEXT("(%.2f, %.2f, %.2f)"), RenderablePoints[Idx].Position.X, RenderablePoints[Idx].Position.Y, RenderablePoints[Idx].Position.Z);
									DebugDrawComponent->IDs.Add(MoveTemp(Text));
								}
							}
						}
					}
				}
			}
		}

		UMaterialInterface* Material = nullptr;
	};

	///////////////////////////////////////////////////////////////////////////////////////////////////////////////

	void RegisterPointRenderableTypes()
	{
		UE_DATAFLOW_REGISTER_RENDERABLE_TYPE(FPointSurfaceRenderableType);
	}

	///////////////////////////////////////////////////////////////////////////////////////////////////////////////

}