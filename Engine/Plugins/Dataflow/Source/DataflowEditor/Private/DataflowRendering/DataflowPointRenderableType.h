// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataflowRendering/DataflowRenderableTypeSettings.h"
#include "DataflowRendering/DataflowRenderableTypeUtils.h"
#include "Curves/LinearColorRamp.h"
#include "UObject/ObjectPtr.h"

#include "DataflowPointRenderableType.generated.h"

UCLASS(MinimalAPI, AutoExpandCategories = "")
class UDataflowPointRenderSettings : public UDataflowRenderableTypeSettings
{
	GENERATED_BODY()

	UDataflowPointRenderSettings(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

public:
	/** Disable rendring the points */
	UPROPERTY(EditAnywhere, Category = "Point")
	bool bRenderPoints = true;

	/** Size of displayed points */
	UPROPERTY(EditAnywhere, Category = "Point", meta = (UIMin = 0.01f, ClampMin = 0.01f))
	float Size = 10.0f;

	/** Color of displayed points */
	UPROPERTY(EditAnywhere, Category = "Point")
	FColor Color = FLinearColor(0.16f, 0.184f, 0.48f, 1.0f).ToFColor(true);

	/** SHow the points IDs */
	UPROPERTY(EditAnywhere, Category = "Point", meta = (InlineEditConditionToggle))
	bool bShowPointIDs = false;

	/** Color of the IDs text */
	UPROPERTY(EditAnywhere, Category = "Point", meta = (EditCondition = bShowPointIDs))
	FColor IDsColor = FColor::Cyan;

	/** SHow the points positions */
	UPROPERTY(EditAnywhere, Category = "Point", meta = (InlineEditConditionToggle))
	bool bShowPositions = false;

	/** Color of the poaition text */
	UPROPERTY(EditAnywhere, Category = "Point", meta = (EditCondition = bShowPositions))
	FColor PositionsColor = FColor::White;

	/** Attribute on points to color the displayed points (Supported types: float, bool, int32, Vector3f */
	UPROPERTY(EditAnywhere, Category = "Point")
	FString Attribute = FString("Dummy");

	/** Attribute minimum value for coloring (This value represents the minimum color on the color ramp */
	UPROPERTY(EditAnywhere, Category = "Point")
	float Min = 0.f;

	/** Attribute maximum value for coloring (This value represents the maximum color on the color ramp */
	UPROPERTY(EditAnywhere, Category = "Point")
	float Max = 1.f;

	/** Color ramp to use to display a single value attribute - ignored for vector attributes */
	UPROPERTY(EditAnywhere, Category = "Point")
	FLinearColorRamp ColorRamp;

	/** Render vector data as RGB colors */
	UPROPERTY(EditAnywhere, Category = "Vector")
	bool bRenderAsRGB = false;

	UPROPERTY(EditAnywhere, Category = "Vector", meta = (UIMin = 0.01f, ClampMin = 0.01f, EditCondition = "!bRenderAsRGB"))
	float LineThickness = 2.f;

	UPROPERTY(EditAnywhere, Category = "Vector", meta = (UIMin = 0.01f, ClampMin = 0.01f, EditCondition = "!bRenderAsRGB"))
	float LengthScalar = 1.f;

	/** Attribute minimum value for coloring (This value represents the minimum color on the color ramp */
	UPROPERTY(EditAnywhere, Category = "Vector", meta = (UIMin = 0.01f, ClampMin = 0.01f, EditCondition = "!bRenderAsRGB"))
	float LengthMin = 0.f;

	/** Attribute maximum value for coloring (This value represents the maximum color on the color ramp */
	UPROPERTY(EditAnywhere, Category = "Vector", meta = (EditCondition = "!bRenderAsRGB"))
	float LengthMax = 1.f;

	/** Color ramp to use to display a vector value attribute length - ignored for scalar attributes */
	UPROPERTY(EditAnywhere, Category = "Vector", meta = (EditCondition = "!bRenderAsRGB"))
	FLinearColorRamp LengthColorRamp;
};

namespace UE::Dataflow::Private
{
	void RegisterPointRenderableTypes();
}

