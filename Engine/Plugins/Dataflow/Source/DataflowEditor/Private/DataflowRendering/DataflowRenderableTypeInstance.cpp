// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataflowRendering/DataflowRenderableTypeInstance.h"

#include "Dataflow/DataflowCoreNodes.h"
#include "DataflowRendering/DataflowRenderableComponents.h"
#include "DataflowRendering/DataflowRenderableType.h"
#include "DataflowRendering/DataflowRenderableTypeRegistry.h"
#include "DataflowRendering/DataflowRenderableTypeSettings.h"
#include "UObject/UObjectGlobals.h"

namespace UE::Dataflow
{
	namespace Private
	{
		bool IsVisibleRenderGroup(FName RenderGroup, const FDataflowOutput& Output)
		{
			if (const FProperty* Property = Output.GetProperty())
			{
				static const FName RenderGroupsMetaDataName = TEXT("DataflowRenderGroups");
				const FString RenderGroupsMetaData = Property->GetMetaData(RenderGroupsMetaDataName);
				if (!RenderGroupsMetaData.IsEmpty())
				{
					TArray<FString> RenderGroups;
					RenderGroupsMetaData.ParseIntoArray(RenderGroups, TEXT(","));
					if (RenderGroups.Num() > 0)
					{
						for (FString& ParsedRenderGroup : RenderGroups)
						{
							ParsedRenderGroup.TrimStartAndEndInline();
						}
						return RenderGroups.Contains(RenderGroup);
					}
				}
			}
			return true;
		}
	}

	/* static */ void FRenderableTypeInstance::GetRenderableInstancesForNode(FContext& Context, const IDataflowConstructionViewMode& ViewMode, const FDataflowNode& Node, bool bEvaluateOutputs, TArray<FRenderableTypeInstance>& OutRenderableTypeInstances)
	{
		OutRenderableTypeInstances.Reset();

		// we do not want reroute node to be able to render 
		if (Node.AsType<FDataflowReRouteNode>())
		{
			return;
		}

		TArray<FDataflowOutput*> Outputs = Node.GetOutputs();
		for (const FDataflowOutput* Output : Outputs)
		{
			if (Output)
			{
				const FRenderableTypeRegistry::FRenderableTypes& RenderableTypes = FRenderableTypeRegistry::GetInstance().GetRenderableTypes(Output->GetType());
				for (const IRenderableType* RenderableType : RenderableTypes)
				{
					if (RenderableType  && RenderableType->IsViewModeSupported(ViewMode))
					{
						FRenderableTypeInstance RenderableTypeInstance(Context, Node.AsShared(), Output->GetName(), RenderableType, bEvaluateOutputs);
						if (RenderableTypeInstance.CanRender())
						{
							if (Private::IsVisibleRenderGroup(RenderableType->GetRenderGroup(), *Output))
							{
								OutRenderableTypeInstances.Emplace(RenderableTypeInstance);
							}
						}
					}
				}
			}
		}
	}

	bool FRenderableTypeInstance::CanRender() const
	{
		return RenderableType ? RenderableType->CanRender(*this) : false;
	}

	FString FRenderableTypeInstance::GetDisplayName() const
	{
		if (RenderableType)
		{
			return FString::Format(TEXT("{0}.{1}"), { RenderableType->GetOutputType().ToString(), RenderableType->GetRenderGroup().ToString() });
		}
		return { TEXT("-- Unknown --") };
	}

	//
	// Schema: ComponentName = CustomName + OutputName + NodeName
	//
	FName FRenderableTypeInstance::GetComponentName(const FName& InComponentName) const
	{
		if (RenderableType)
		{
			if (Node)
			{
				if (const FDataflowOutput* Output = Node->FindOutput(OutputName))
				{
					const FString NodeNamestr = Node->GetName().ToString();
					const FString OutputNameStr = OutputName.ToString();
					const FString ComponentNameStr = InComponentName.ToString();

					return FName(FString::Format(TEXT("{0}_{1}_{2}"), { ComponentNameStr, OutputName.ToString(), NodeNamestr }));
				}
			}
		}

		return FName(TEXT("-- Unknown --"));
	}

	void FRenderableTypeInstance::GetPrimitiveComponents(FRenderableComponents& OutComponents) const
	{
		if (RenderableType)
		{
			RenderableType->GetPrimitiveComponents(*this, OutComponents);
			if (Node)
			{
				Node->OnRenderOutput(EvaluationContext, OutputName, RenderableType->GetRenderGroup(), OutComponents.GetComponents());
			}
		}
	}

	FContextCacheKey  FRenderableTypeInstance::ComputeCacheKey() const
	{
		FContextCacheKey OutputCacheKey = 0;
		if (Node)
		{
			if (const FDataflowOutput* Output = Node->FindOutput(OutputName))
			{
				OutputCacheKey = Output->CacheKey();
			}
		}
		const uint32 RenderGroupHash = GetTypeHash(RenderableType? RenderableType->GetRenderGroup(): NAME_None);
		return ::HashCombine(OutputCacheKey, RenderGroupHash);
	}

	bool FRenderableTypeInstance::HasMetadata(const FDataflowOutput& Output, FName Metadata)
	{
		if (const FProperty* Property = Output.GetProperty())
		{
			return Property->HasMetaData(Metadata);
		}
		return false;
	}

	TArray<UDataflowRenderableTypeSettings*> FRenderableTypeInstance::GetAllRenderSettings() const
	{
		TArray<UDataflowRenderableTypeSettings*> OutSettings;
		if (RenderableType)
		{
			TArray<const UClass*> SettingsClasses;
			RenderableType->GetSettingsClasses(SettingsClasses);
			for (const UClass* SettingsClass : SettingsClasses)
			{
				if (UDataflowRenderableTypeSettings* Settings = GetRenderSettings(SettingsClass))
				{
					OutSettings.AddUnique(Settings);
				}
			}
		}
		return OutSettings;
	}

	UDataflowRenderableTypeSettings* FRenderableTypeInstance::GetRenderSettings(const UClass* SettingsClass) const
	{
		if (SettingsClass)
		{
			if (SettingsClass->IsChildOf<UDataflowRenderableTypeSettings>())
			{
				const FContextCacheKey RenderSettingsKey = ::GetTypeHash(SettingsClass);
				if (!EvaluationContext.HasData(RenderSettingsKey))
				{
					TObjectPtr<UDataflowRenderableTypeSettings> NewRenderSettings = NewObject<UDataflowRenderableTypeSettings>(GetTransientPackageAsObject(), SettingsClass);
					EvaluationContext.SetData(RenderSettingsKey, nullptr, NewRenderSettings, FGuid(), /*ValueHash*/0, UE::Dataflow::FTimestamp::Current());
				}
				return EvaluationContext.GetData<TObjectPtr<UDataflowRenderableTypeSettings>>(RenderSettingsKey, nullptr, {}).Get();
			}
		}
		return nullptr;
	}

	const FDataflowNode::FAttributeKey FRenderableTypeInstance::GetVertexAttributeToVisualize() const
	{
		if (RenderableType && Node)
		{
			return Node->GetVertexAttributeToVisualize(EvaluationContext, OutputName, RenderableType->GetRenderGroup());
		}
		return {};
	}

	const void FRenderableTypeInstance::GetSelectionToVisualize(FDataflowSelection& OutSelection) const
	{
		if (RenderableType && Node)
		{
			return Node->GetSelectionToVisualize(EvaluationContext, OutputName, RenderableType->GetRenderGroup(), OutSelection);
		}
	}
}
