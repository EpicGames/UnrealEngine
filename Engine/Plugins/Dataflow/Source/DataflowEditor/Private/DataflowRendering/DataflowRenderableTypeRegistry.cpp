// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataflowRendering/DataflowRenderableTypeRegistry.h"

#include "Dataflow/DataflowAnyTypeRegistry.h"
#include "DataflowRendering/DataflowRenderableType.h"

namespace UE::Dataflow
{
	FRenderableTypeRegistry& FRenderableTypeRegistry::GetInstance()
	{
		static FRenderableTypeRegistry Instance;
		return Instance;
	}

	void FRenderableTypeRegistry::Register(const IRenderableType* RenderableType)
	{
		if (RenderableType)
		{
			const FName OutputType = RenderableType->GetOutputType();
			if (FAnyTypesRegistry::IsAnyTypeStatic(OutputType))
			{
				RegisteredAnyTypes.Add(OutputType);
			}
			RenderableTypesByPrimaryType
				.FindOrAdd(OutputType)
				.Add(RenderableType);
		}
	}

	const FRenderableTypeRegistry::FRenderableTypes& FRenderableTypeRegistry::GetRenderableTypes(FName PrimaryType) const
	{
		static const FRenderableTypes EmptyTypes;

		if (const FRenderableTypes* RenderableTypes = RenderableTypesByPrimaryType.Find(PrimaryType))
		{
			return *RenderableTypes;
		}
		// no specific type  regsitered, let's check if there's a anytype compatible
		for (FName RegisteredAnyType : RegisteredAnyTypes)
		{
			if (FAnyTypesRegistry::AreTypesCompatibleStatic(PrimaryType, RegisteredAnyType))
			{
				if (const FRenderableTypes* RenderableTypes = RenderableTypesByPrimaryType.Find(RegisteredAnyType))
				{
					return *RenderableTypes;
				}
			}
		}
		return EmptyTypes;
	}
}
