// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataflowRendering/DataflowRenderableTypeSettings.h"
#include "PropertyEditorModule.h"
#include "Modules/ModuleManager.h"

/*static*/ void UDataflowRenderableTypeSettings::RegisterSection(UClass* Class, FName Name, const FText& DisplayName, TConstArrayView<FName> Categories)
{
	if (Class)
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection(Class->GetFName(), Name, DisplayName);

		// Assign categories to the section
		for (const FName& Category : Categories)
		{
			Section->AddCategory(Category);
		}
	}
}
