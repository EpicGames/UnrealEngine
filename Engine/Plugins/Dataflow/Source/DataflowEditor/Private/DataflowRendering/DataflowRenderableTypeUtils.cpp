// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataflowRendering/DataflowRenderableTypeUtils.h"
#include "DataflowRendering/DataflowRenderableComponents.h"
#include "DataflowRendering/DataflowRenderableTypeInstance.h"
#include "Dataflow/DataflowEngineUtil.h"

#include "Components/DynamicMeshComponent.h"
#include "Components/StaticMeshComponent.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/StaticMesh.h"
#include "Generators/CapsuleGenerator.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "PhysicsEngine/AggregateGeom.h"
#include "PhysicsEngine/BoxElem.h"
#include "PhysicsEngine/ConvexElem.h"
#include "PhysicsEngine/SphereElem.h"
#include "PhysicsEngine/SphylElem.h"
#include "PhysicsEngine/TaperedCapsuleElem.h"

#include "GeometryCollection/Facades/CollectionMeshFacade.h"
#include "GeometryCollection/GeometryCollectionUtility.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "GeometryCollection/ManagedArrayCollection.h"

#include "DynamicMeshToMeshDescription.h"
#include "MeshDescriptionUVsToDynamicMesh.h"
#include "StaticMeshAttributes.h"

#include "Dataflow/DataflowEditorStyle.h"

FDataflowColorCommonRenderSettings::FDataflowColorCommonRenderSettings()
{
	constexpr bool bOnlyRGB = true;
	ColorRamp.SetColorAtTime(0.0f, FLinearColor::Red, bOnlyRGB);
	ColorRamp.SetColorAtTime(0.25f, FLinearColor::Yellow, bOnlyRGB);
	ColorRamp.SetColorAtTime(0.5f, FLinearColor::Green, bOnlyRGB);
	ColorRamp.SetColorAtTime(0.75f, FLinearColor(FColor::Purple), bOnlyRGB);
	ColorRamp.SetColorAtTime(1.0f, FLinearColor::Blue, bOnlyRGB);
}

namespace UE::Dataflow::Rendering
{
	namespace Private
	{
		constexpr float UvScaleFactor = 100.f;

		FVector3d ConvertUvToPosition(const FVector2f& Uv)
		{
			return FVector3d((1 - Uv.Y) * UvScaleFactor, (Uv.X * UvScaleFactor), 0);
		}
	}

	///////////////////////////////////////////////////////////////////////////////////////////////////////////////

	UMaterial* GetVertexMaterial()
	{
		return FDataflowEditorStyle::Get().VertexMaterial;
	}

	///////////////////////////////////////////////////////////////////////////////////////////////////////////////

	TSharedPtr<UE::Geometry::FDynamicMesh3> GenerateUvMesh(const FMeshDescription& MeshDescription, int32 UvChannel)
	{
		if (UvChannel < MeshDescription.GetNumUVElementChannels())
		{
			UE::Geometry::FMeshDescriptionUVsToDynamicMesh Converter;
			Converter.UVLayerIndex = UvChannel;
			Converter.ScaleFactor = Private::UvScaleFactor;
			return Converter.GetUVMesh(&MeshDescription);
		}
		return {};
	}

	///////////////////////////////////////////////////////////////////////////////////////////////////////////////

	TSharedPtr<UE::Geometry::FDynamicMesh3> GenerateUvMesh(const UStaticMesh& StaticMesh, int32 UvChannel)
	{
		if (const FMeshDescription* MeshDescription = StaticMesh.GetMeshDescription(0))
		{
			return GenerateUvMesh(*MeshDescription, UvChannel);
		}
		return {};
	}

	///////////////////////////////////////////////////////////////////////////////////////////////////////////////

	TSharedPtr<UE::Geometry::FDynamicMesh3> GenerateUvMesh(const USkeletalMesh& SkeletalMesh, int32 UvChannel)
	{
		if (const FMeshDescription* MeshDescription = SkeletalMesh.GetMeshDescription(0))
		{
			return GenerateUvMesh(*MeshDescription, UvChannel);
		}
		return {};
	}

	///////////////////////////////////////////////////////////////////////////////////////////////////////////////

	TSharedPtr<UE::Geometry::FDynamicMesh3> GenerateUvMesh(const UE::Geometry::FDynamicMesh3& DynamicMesh, int32 UvChannel)
	{
		// first create  mesh description for this dynamic mesh
		FMeshDescription MeshDescription;
		{
			FStaticMeshAttributes Attributes(MeshDescription);
			Attributes.Register();
			FConversionToMeshDescriptionOptions ConverterOptions;
			FDynamicMeshToMeshDescription Converter(ConverterOptions);
			Converter.Convert(&DynamicMesh, MeshDescription);
		}

		// generate the UV mesh from it 
		return GenerateUvMesh(MeshDescription, UvChannel);
	}

	///////////////////////////////////////////////////////////////////////////////////////////////////////////////

	TSharedPtr<UE::Geometry::FDynamicMesh3> GenerateUvMesh(const FManagedArrayCollection& Collection, int32 UvChannel)
	{
		const GeometryCollection::Facades::FCollectionUVFacade UVFacade(Collection);
		const GeometryCollection::Facades::FCollectionMeshFacade MeshFacade(Collection);
		if (!UVFacade.IsValid() || !MeshFacade.IndicesAttribute.IsValid())
		{
			return {};
		}

		if (!UVFacade.FindUVLayer(UvChannel))
		{
			return {};
		}

		TSharedPtr<FDynamicMesh3> MeshOut = MakeShared<FDynamicMesh3>();

		// create vertices from Uvs
		MeshOut->BeginUnsafeVerticesInsert();
		const TManagedArray<FVector2f>& UVs = UVFacade.GetUVLayer(UvChannel);
		for (int32 UvIndex = 0; UvIndex < UVs.Num(); ++UvIndex)
		{
			MeshOut->InsertVertex(UvIndex, Private::ConvertUvToPosition(UVs[UvIndex]), true);
		}
		MeshOut->EndUnsafeVerticesInsert();

		// connect them using the regular triangles
		MeshOut->BeginUnsafeTrianglesInsert();
		for (int32 TriIndex = 0; TriIndex < MeshFacade.IndicesAttribute.Num(); ++TriIndex)
		{
			const bool bIsVisible = (MeshFacade.VisibleAttribute.IsValid()) ? MeshFacade.VisibleAttribute.Get()[TriIndex] : true;
			if (bIsVisible)
			{
				const FIntVector& Triangle = MeshFacade.IndicesAttribute[TriIndex];
				MeshOut->InsertTriangle(TriIndex, { Triangle.X, Triangle.Y, Triangle.Z }, 0, true);
			}
		}
		MeshOut->EndUnsafeTrianglesInsert();

		return MeshOut;
	}

	///////////////////////////////////////////////////////////////////////////////////////////////////////////////

	void SetUvMaterial(UDynamicMeshComponent& Component, int32 MaterialIndex)
	{
		UMaterialInterface* UvMaterial = LoadObject<UMaterialInterface>(nullptr, TEXT("/UVEditor/Materials/UVEditor_UnwrapMaterial"));
		Component.SetMaterial(MaterialIndex, UvMaterial);
		Component.SetTwoSided(true); // need to be two sided for the Uv display
	}

	///////////////////////////////////////////////////////////////////////////////////////////////////////////////
	
	UDynamicMeshComponent* AddDynamicMeshComponent(UE::Geometry::FDynamicMesh3&& DynamicMesh, FRenderableComponents& OutComponents)
	{
		if (UDynamicMeshComponent* Component = OutComponents.AddNewComponent<UDynamicMeshComponent>())
		{
			Component->SetMesh(MoveTemp(DynamicMesh));
			return Component;
		}
		return nullptr;
	}

	///////////////////////////////////////////////////////////////////////////////////////////////////////////////
	
	int32 GetUVChannelFromInstance(const FRenderableTypeInstance& Instance, int32 DefaultValue)
	{
		const static FName DataflowUVChannelName(TEXT("DataflowUVChannel"));
		return Instance.GetOutputValueByType<int32>(DefaultValue, DataflowUVChannelName);
	}

	///////////////////////////////////////////////////////////////////////////////////////////////////////////////

	FLinearColor GetColor(const int32 InIdx, const FBox& InComponentBoundingBox, const FVector& InCenter, float InSize,
		const FDataflowColorCommonRenderSettings& InSettings)
	{
		FLinearColor Color = FLinearColor::White;

		if (InSettings.ColorMethod == EDataflowColorMethodType::Constant)
		{
			Color = InSettings.Color;
		}
		else if (InSettings.ColorMethod == EDataflowColorMethodType::Random)
		{
			Color = UE::Dataflow::Color::GetRandomColor(InSettings.RandomSeed, InIdx);
		}
		else if (InSettings.ColorMethod == EDataflowColorMethodType::BySize)
		{
			if (InSize < InSettings.SizeMin)
			{
				InSize = InSettings.SizeMin;
			}
			else if (InSize > InSettings.SizeMax)
			{
				InSize = InSettings.SizeMax;
			}

			if (InSettings.SizeMax - InSettings.SizeMin > UE_SMALL_NUMBER)
			{
				float NormSize = (InSize - InSettings.SizeMin) / (InSettings.SizeMax - InSettings.SizeMin);

				Color = InSettings.ColorRamp.GetLinearColorValue(NormSize);
			}
		}
		else if (InSettings.ColorMethod == EDataflowColorMethodType::BoundingBox)
		{
			if (InComponentBoundingBox.GetExtent().X > UE_SMALL_NUMBER &&
				InComponentBoundingBox.GetExtent().Y > UE_SMALL_NUMBER &&
				InComponentBoundingBox.GetExtent().Z > UE_SMALL_NUMBER)
			{
				float DeltaX = (InCenter.X - InComponentBoundingBox.Min.X) / (InComponentBoundingBox.Max.X - InComponentBoundingBox.Min.X);
				float DeltaY = (InCenter.Y - InComponentBoundingBox.Min.Y) / (InComponentBoundingBox.Max.Y - InComponentBoundingBox.Min.Y);
				float DeltaZ = (InCenter.Z - InComponentBoundingBox.Min.Z) / (InComponentBoundingBox.Max.Z - InComponentBoundingBox.Min.Z);

				switch (InSettings.PermutationIndex)
				{
				case 0:
					Color = FLinearColor(DeltaX, DeltaY, DeltaZ);
					break;
				case 1:
					Color = FLinearColor(DeltaX, 1.f - DeltaY, DeltaZ);
					break;
				case 2:
					Color = FLinearColor(DeltaX, DeltaY, 1.f - DeltaZ);
					break;
				case 3:
					Color = FLinearColor(DeltaX, 1.f - DeltaY, 1.f - DeltaZ);
					break;
				case 4:
					Color = FLinearColor(DeltaX, DeltaZ, DeltaY);
					break;
				case 5:
					Color = FLinearColor(DeltaX, 1.f - DeltaZ, DeltaY);
					break;
				case 6:
					Color = FLinearColor(DeltaX, DeltaZ, 1.f - DeltaY);
					break;
				case 7:
					Color = FLinearColor(DeltaX, 1.f - DeltaZ, 1.f - DeltaY);
					break;
				case 8:
					Color = FLinearColor(DeltaY, DeltaX, DeltaZ);
					break;
				case 9:
					Color = FLinearColor(DeltaY, 1.f - DeltaX, DeltaZ);
					break;
				case 10:
					Color = FLinearColor(DeltaY, DeltaX, 1.f - DeltaZ);
					break;
				case 11:
					Color = FLinearColor(DeltaY, 1.f - DeltaX, 1.f - DeltaZ);
					break;
				case 12:
					Color = FLinearColor(DeltaY, DeltaZ, DeltaX);
					break;
				case 13:
					Color = FLinearColor(DeltaY, 1.f - DeltaZ, DeltaX);
					break;
				case 14:
					Color = FLinearColor(DeltaY, DeltaZ, 1.f - DeltaX);
					break;
				case 15:
					Color = FLinearColor(DeltaY, 1.f - DeltaZ, 1.f - DeltaX);
					break;
				case 16:
					Color = FLinearColor(DeltaZ, DeltaX, DeltaY);
					break;
				case 17:
					Color = FLinearColor(DeltaZ, 1.f - DeltaX, DeltaY);
					break;
				case 18:
					Color = FLinearColor(DeltaZ, DeltaX, 1.f - DeltaY);
					break;
				case 19:
					Color = FLinearColor(DeltaZ, 1.f - DeltaX, 1.f - DeltaY);
					break;
				case 20:
					Color = FLinearColor(DeltaZ, DeltaY, DeltaX);
					break;
				case 21:
					Color = FLinearColor(DeltaZ, 1.f - DeltaY, DeltaX);
					break;
				case 22:
					Color = FLinearColor(DeltaZ, DeltaY, 1.f - DeltaX);
					break;
				case 23:
					Color = FLinearColor(DeltaZ, 1.f - DeltaY, 1.f - DeltaX);
					break;
				default:
					Color = FLinearColor(DeltaX, DeltaY, DeltaZ);
					break;
				}
			}
		}

		return Color;
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////

namespace UE::Dataflow::RenderGeometry
{

UE::Geometry::FDynamicMesh3 BuildCapsuleMesh(float Radius, float Length)
{
	UE::Geometry::FCapsuleGenerator Generator;
	Generator.Radius            = Radius;
	Generator.SegmentLength     = Length;
	Generator.NumHemisphereArcSteps = 6;
	Generator.NumCircleSteps        = 12;
	Generator.Generate();

	UE::Geometry::FDynamicMesh3 Mesh(&Generator);

	// FCapsuleGenerator places the cylinder from Z=0; shift down to center the mesh at Z=0.
	const double OffsetZ = -(double)Length * 0.5;
	for (int32 VID : Mesh.VertexIndicesItr())
	{
		FVector3d V = Mesh.GetVertex(VID);
		V.Z += OffsetZ;
		Mesh.SetVertex(VID, V);
	}
	return Mesh;
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////

UE::Geometry::FDynamicMesh3 BuildTaperedCapsuleMesh(float Radius0, float Radius1, float Length, int32 ArcSteps, int32 CircleSteps)
{
	UE::Geometry::FDynamicMesh3 Mesh;
	const double HalfLength = (double)Length * 0.5;
	const double DTheta      = 2.0 * UE_DOUBLE_PI / CircleSteps;

	// Profile: (radius, z) pairs from bottom pole to top pole.
	TArray<TTuple<double, double>> Profile;

	// Bottom pole
	Profile.Add(MakeTuple(0.0, -HalfLength - (double)Radius0));
	// Bottom hemisphere (pole excluded, equator included)
	for (int32 i = 1; i <= ArcSteps; ++i)
	{
		const double Phi = (UE_DOUBLE_PI * 0.5) * (double)i / (double)ArcSteps;
		Profile.Add(MakeTuple((double)Radius0 * FMath::Sin(Phi), -HalfLength - (double)Radius0 * FMath::Cos(Phi)));
	}
	// Top equator (separate ring to allow the taper)
	Profile.Add(MakeTuple((double)Radius1, HalfLength));
	// Top hemisphere (equator excluded, pole included)
	for (int32 i = ArcSteps - 1; i >= 0; --i)
	{
		const double Phi = (UE_DOUBLE_PI * 0.5) * (double)i / (double)ArcSteps;
		Profile.Add(MakeTuple((double)Radius1 * FMath::Sin(Phi), HalfLength + (double)Radius1 * FMath::Cos(Phi)));
	}

	// Revolve profile to produce vertex rings.
	const int32 N = Profile.Num();
	TArray<TArray<int32>> Rings;
	Rings.SetNum(N);

	for (int32 PIdx = 0; PIdx < N; ++PIdx)
	{
		const double R = Profile[PIdx].Get<0>();
		const double Z = Profile[PIdx].Get<1>();
		if (FMath::IsNearlyZero(R))
		{
			Rings[PIdx].Add(Mesh.AppendVertex(FVector3d(0.0, 0.0, Z)));
		}
		else
		{
			for (int32 CIdx = 0; CIdx < CircleSteps; ++CIdx)
			{
				const double Theta = CIdx * DTheta;
				Rings[PIdx].Add(Mesh.AppendVertex(FVector3d(R * FMath::Cos(Theta), R * FMath::Sin(Theta), Z)));
			}
		}
	}

	// Stitch adjacent rings with triangles.
	for (int32 PIdx = 0; PIdx + 1 < N; ++PIdx)
	{
		const TArray<int32>& RingA = Rings[PIdx];
		const TArray<int32>& RingB = Rings[PIdx + 1];

		if (RingA.Num() == 1)
		{
			const int32 Pole = RingA[0];
			for (int32 CIdx = 0; CIdx < CircleSteps; ++CIdx)
			{
				Mesh.AppendTriangle(Pole, RingB[CIdx], RingB[(CIdx + 1) % CircleSteps]);
			}
		}
		else if (RingB.Num() == 1)
		{
			const int32 Pole = RingB[0];
			for (int32 CIdx = 0; CIdx < CircleSteps; ++CIdx)
			{
				Mesh.AppendTriangle(RingA[CIdx], Pole, RingA[(CIdx + 1) % CircleSteps]);
			}
		}
		else
		{
			for (int32 CIdx = 0; CIdx < CircleSteps; ++CIdx)
			{
				const int32 Next = (CIdx + 1) % CircleSteps;
				Mesh.AppendTriangle(RingA[CIdx], RingB[CIdx],  RingB[Next]);
				Mesh.AppendTriangle(RingA[CIdx], RingB[Next],  RingA[Next]);
			}
		}
	}

	return Mesh;
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////

UE::Geometry::FDynamicMesh3 BuildConvexMesh(const FKConvexElem& Elem)
{
	UE::Geometry::FDynamicMesh3 Mesh;

	if (Elem.VertexData.IsEmpty() || Elem.IndexData.Num() < 3)
	{
		return Mesh;
	}

	for (const FVector& V : Elem.VertexData)
	{
		Mesh.AppendVertex(FVector3d(V));
	}

	for (int32 Tri = 0; Tri + 2 < Elem.IndexData.Num(); Tri += 3)
	{
		Mesh.AppendTriangle(Elem.IndexData[Tri], Elem.IndexData[Tri + 1], Elem.IndexData[Tri + 2]);
	}

	return Mesh;
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////

void AddAggGeomComponents(
	const FKAggregateGeom& AggGeom,
	const FTransform& BoneTransform,
	FName BoneName,
	FRenderableComponents& OutComponents,
	UObject* ComponentParent,
	UMaterialInstanceDynamic* MaterialInstance,
	int32& InOutShapeCounter)
{
	auto MakeName = [&](const TCHAR* ShapeType) -> FName
	{
		return BoneName.IsNone()
			? FName(FString::Printf(TEXT("[%d]_(%s)"), InOutShapeCounter, ShapeType))
			: FName(FString::Printf(TEXT("[%d]_%s_(%s)"), InOutShapeCounter, *BoneName.ToString(), ShapeType));
	};

	// --- Spheres ---
	for (const FKSphereElem& Elem : AggGeom.SphereElems)
	{
		if (UStaticMesh* SphereMesh = GetDataflowSphere())
		{
			if (UStaticMeshComponent* Comp = OutComponents.AddNewComponent<UStaticMeshComponent>(MakeName(TEXT("Sphere")), ComponentParent))
			{
				Comp->SetStaticMesh(SphereMesh);
				Comp->SetCollisionEnabled(ECollisionEnabled::NoCollision);
				FTransform T(FQuat::Identity, BoneTransform.TransformPosition(FVector(Elem.Center)));
				T.SetScale3D(FVector(Elem.Radius / 50.f));
				Comp->SetWorldTransform(T);
				Comp->SetMaterial(0, MaterialInstance);
			}
		}
		++InOutShapeCounter;
	}

	// --- Boxes ---
	for (const FKBoxElem& Elem : AggGeom.BoxElems)
	{
		if (UStaticMesh* BoxMesh = GetDataflowBox())
		{
			if (UStaticMeshComponent* Comp = OutComponents.AddNewComponent<UStaticMeshComponent>(MakeName(TEXT("Box")), ComponentParent))
			{
				Comp->SetStaticMesh(BoxMesh);
				Comp->SetCollisionEnabled(ECollisionEnabled::NoCollision);
				FTransform T = Elem.GetTransform() * BoneTransform;
				T.SetScale3D(FVector(Elem.X, Elem.Y, Elem.Z) / 100.f);
				Comp->SetWorldTransform(T);
				Comp->SetMaterial(0, MaterialInstance);
			}
		}
		++InOutShapeCounter;
	}

	// --- Capsules (Sphyl) ---
	for (const FKSphylElem& Elem : AggGeom.SphylElems)
	{
		if (UDynamicMeshComponent* Comp = OutComponents.AddNewComponent<UDynamicMeshComponent>(MakeName(TEXT("Capsule")), ComponentParent))
		{
			Comp->SetMesh(BuildCapsuleMesh(Elem.Radius, Elem.Length));
			Comp->SetCollisionEnabled(ECollisionEnabled::NoCollision);
			Comp->SetWorldTransform(Elem.GetTransform() * BoneTransform);
			Comp->SetOverrideRenderMaterial(MaterialInstance);
		}
		++InOutShapeCounter;
	}

	// --- Tapered Capsules ---
	for (const FKTaperedCapsuleElem& Elem : AggGeom.TaperedCapsuleElems)
	{
		if (UDynamicMeshComponent* Comp = OutComponents.AddNewComponent<UDynamicMeshComponent>(MakeName(TEXT("TaperedCapsule")), ComponentParent))
		{
			Comp->SetMesh(BuildTaperedCapsuleMesh(Elem.Radius0, Elem.Radius1, Elem.Length));
			Comp->SetCollisionEnabled(ECollisionEnabled::NoCollision);
			Comp->SetWorldTransform(Elem.GetTransform() * BoneTransform);
			Comp->SetOverrideRenderMaterial(MaterialInstance);
		}
		++InOutShapeCounter;
	}

	// --- Convex Hulls ---
	for (const FKConvexElem& Elem : AggGeom.ConvexElems)
	{
		UE::Geometry::FDynamicMesh3 ConvexMesh = BuildConvexMesh(Elem);
		if (ConvexMesh.TriangleCount() > 0)
		{
			if (UDynamicMeshComponent* Comp = OutComponents.AddNewComponent<UDynamicMeshComponent>(MakeName(TEXT("Convex")), ComponentParent))
			{
				Comp->SetMesh(MoveTemp(ConvexMesh));
				Comp->SetCollisionEnabled(ECollisionEnabled::NoCollision);
				Comp->SetWorldTransform(Elem.GetTransform() * BoneTransform);
				Comp->SetOverrideRenderMaterial(MaterialInstance);
			}
		}
		++InOutShapeCounter;
	}
}

} // namespace UE::Dataflow::RenderGeometry
