// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataflowRendering/DataflowSamplerRenderableType.h"

#include "Drawing/PointSetComponent.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "Drawing/LineSetComponent.h"
#include "Components/BoxComponent.h"

#include "Dataflow/DataflowRenderingViewMode.h"

#include "DataflowRendering/DataflowRenderableComponents.h"
#include "DataflowRendering/DataflowRenderableType.h"
#include "DataflowRendering/DataflowRenderableTypeInstance.h"
#include "DataflowRendering/DataflowRenderableTypeRegistry.h"
#include "DataflowRendering/DataflowRenderableTypeUtils.h"

#include "Materials/MaterialInterface.h"
#include "Materials/Material.h"

#include "GeometryCollection/Facades/PointsFacade.h"

#include "UObject/ObjectPtr.h"
#include "Dataflow/SamplerNodes/DataflowSamplerTypes.h"

UDataflowFloatSamplerRenderSettings::UDataflowFloatSamplerRenderSettings(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	constexpr bool bOnlyRGB = true;
	ColorRamp.SetColorAtTime(0.0f, FLinearColor::Green, bOnlyRGB);
	ColorRamp.SetColorAtTime(1.0f, FLinearColor::Blue, bOnlyRGB);
}

namespace UE::Dataflow::Private
{
	static TArray<FVector> GetPointsOnSlice(const FVector& InCenter, const FVector& InExtent, const EDataflowSlicePlane& InPlane, float InPointSeparation, const float InOffset)
	{
		TArray<FVector> OutPoints;

		FVector NewExtent;

		if (InPointSeparation < SMALL_NUMBER)
		{
			InPointSeparation = 2.f;
		}

		NewExtent.X = (FMath::CeilToDouble(InExtent.X / InPointSeparation) + 1.f) * InPointSeparation;
		NewExtent.Y = (FMath::CeilToDouble(InExtent.Y / InPointSeparation) + 1.f) * InPointSeparation;
		NewExtent.Z = (FMath::CeilToDouble(InExtent.Z / InPointSeparation) + 1.f) * InPointSeparation;

		FVector Min = InCenter - 0.5 * NewExtent;
		FVector Max = InCenter + 0.5 * NewExtent;

		const int32 NumX = FMath::Min(1000, int32(NewExtent.X / InPointSeparation)) + 1;
		const int32 NumY = FMath::Min(1000, int32(NewExtent.Y / InPointSeparation)) + 1;
		const int32 NumZ = FMath::Min(1000, int32(NewExtent.Z / InPointSeparation)) + 1;

		float X, Y, Z;
		if (InPlane == EDataflowSlicePlane::XYPlane)
		{
			for (int32 IdxX = 0; IdxX < NumX; ++IdxX)
			{
				for (int32 IdxY = 0; IdxY < NumY; ++IdxY)
				{
					X = Min.X + float(IdxX) * InPointSeparation;
					Y = Min.Y + float(IdxY) * InPointSeparation;
					Z = InCenter.Z + InOffset * 0.5 * InExtent.Z;

					OutPoints.Add(FVector(X, Y, Z));
				}
			}
		}
		else if (InPlane == EDataflowSlicePlane::YZPlane)
		{
			for (int32 IdxY = 0; IdxY < NumY; ++IdxY)
			{
				for (int32 IdxZ = 0; IdxZ < NumZ; ++IdxZ)
				{
					X = InCenter.X + InOffset * 0.5 * InExtent.X;
					Y = Min.Y + float(IdxY) * InPointSeparation;
					Z = Min.Z + float(IdxZ) * InPointSeparation;

					OutPoints.Add(FVector(X, Y, Z));
				}
			}
		}
		else if (InPlane == EDataflowSlicePlane::ZXPlane)
		{
			for (int32 IdxZ = 0; IdxZ < NumZ; ++IdxZ)
			{
				for (int32 IdxX = 0; IdxX < NumX; ++IdxX)
				{
					X = Min.X + float(IdxX) * InPointSeparation;
					Y = InCenter.Y + InOffset * 0.5 * InExtent.Y;
					Z = Min.Z + float(IdxZ) * InPointSeparation;

					OutPoints.Add(FVector(X, Y, Z));
				}
			}
		}

		return OutPoints;
	}

	///////////////////////////////////////////////////////////////////////////////////////////////////////////////

	class FFloatSamplerSurfaceRenderableType : public IRenderableType
	{
		UE_DATAFLOW_IRENDERABLE_PRIMARY_TYPE(FDataflowFloatSampler, Sampler);
		UE_DATAFLOW_IRENDERABLE_RENDER_GROUP(Sampler);
		UE_DATAFLOW_IRENDERABLE_VIEW_MODE(FDataflowConstruction3DViewMode);
		UE_DATAFLOW_IRENDERABLE_SETTINGS(UDataflowFloatSamplerRenderSettings);

	public:
		FFloatSamplerSurfaceRenderableType()
		{
			constexpr bool bDepthTested = false;
			Material = (bDepthTested)
				? LoadObject<UMaterial>(nullptr, TEXT("/MeshModelingToolsetExp/Materials/PointSetComponentMaterial"))
				: LoadObject<UMaterial>(nullptr, TEXT("/MeshModelingToolsetExp/Materials/PointSetOverlaidComponentMaterial"));
		}

		virtual void GetPrimitiveComponents(const FRenderableTypeInstance& Instance, FRenderableComponents& OutComponents) const override
		{
			const UDataflowFloatSamplerRenderSettings* Settings = Instance.GetTypedRenderSettings<UDataflowFloatSamplerRenderSettings>();
			if (Settings)
			{
				const FDataflowFloatSampler& InSampler = GetSampler(Instance);

				FBox RenderBounds = InSampler.GetRenderBounds();
				if (Settings->bOverrideBounds)
				{
					RenderBounds.Min = Settings->Center - Settings->Extent;
					RenderBounds.Max = Settings->Center + Settings->Extent;
				}

				TArray<FVector> Points = GetPointsOnSlice(RenderBounds.GetCenter(), 2.f * RenderBounds.GetExtent(), Settings->Plane, Settings->PointSeparation, Settings->Offset);

				const int32 NumPoints = Points.Num();

				TArray<float> Values; Values.SetNumUninitialized(NumPoints);
				InSampler.Sample(TArray<FVector3f>(Points), Values);

				const FName PointComponentName = Instance.GetComponentName(TEXT("FloatSampler"));

				TArray<FRenderablePoint> RenderablePoints;
				RenderablePoints.SetNumUninitialized(NumPoints);

				for (int32 Idx = 0; Idx < NumPoints; ++Idx)
				{
					const float AttrValue = FMath::Clamp(Values[Idx], Settings->Min, Settings->Max);

					FLinearColor LinearColor = FLinearColor::Green;
					FColor Color = LinearColor.ToFColor(true).WithAlpha(255);

					if (Settings->Max - Settings->Min > UE_SMALL_NUMBER)
					{
						float NormAttrValue = (AttrValue - Settings->Min) / (Settings->Max - Settings->Min);

						LinearColor = Settings->ColorRamp.GetLinearColorValue(NormAttrValue);
						Color = LinearColor.ToFColor(true).WithAlpha(255);
					}

					RenderablePoints[Idx] = FRenderablePoint(Points[Idx], Color, Settings->Size);
				}

				// Render box
				static const FName BoxComponentName = TEXT("RenderBounds");

				if (UBoxComponent* BoxComponent = OutComponents.AddNewComponent<UBoxComponent>(BoxComponentName))
				{
					BoxComponent->SetBoxExtent(RenderBounds.GetExtent());

					FTransform Transform = FTransform::Identity;
					Transform.SetTranslation(RenderBounds.GetCenter());

					BoxComponent->SetWorldTransform(Transform);
				}

				// Render points
				if (UPointSetComponent* PointComponent = OutComponents.AddNewComponent<UPointSetComponent>(PointComponentName))
				{
					PointComponent->ReservePoints(NumPoints);
					PointComponent->AddPoints(RenderablePoints);

					PointComponent->SetPointMaterial(Material);
				}
			}
		}

		UMaterialInterface* Material = nullptr;
	};
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////

UDataflowVectorSamplerRenderSettings::UDataflowVectorSamplerRenderSettings(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	constexpr bool bOnlyRGB = true;
	ColorRamp.SetColorAtTime(0.0f, FLinearColor::Green, bOnlyRGB);
	ColorRamp.SetColorAtTime(1.0f, FLinearColor::Blue, bOnlyRGB);
}

namespace UE::Dataflow::Private
{
	class FVectorSamplerSurfaceRenderableType : public IRenderableType
	{
		UE_DATAFLOW_IRENDERABLE_PRIMARY_TYPE(FDataflowVectorSampler, Sampler);
		UE_DATAFLOW_IRENDERABLE_RENDER_GROUP(Sampler);
		UE_DATAFLOW_IRENDERABLE_VIEW_MODE(FDataflowConstruction3DViewMode);
		UE_DATAFLOW_IRENDERABLE_SETTINGS(UDataflowVectorSamplerRenderSettings);

	public:
		FVectorSamplerSurfaceRenderableType()
		{
			constexpr bool bDepthTested = false;
			Material = (bDepthTested)
				? LoadObject<UMaterial>(nullptr, TEXT("/MeshModelingToolsetExp/Materials/PointSetComponentMaterial"))
				: LoadObject<UMaterial>(nullptr, TEXT("/MeshModelingToolsetExp/Materials/PointSetOverlaidComponentMaterial"));
		}

		virtual void GetPrimitiveComponents(const FRenderableTypeInstance& Instance, FRenderableComponents& OutComponents) const override
		{
			const UDataflowVectorSamplerRenderSettings* Settings = Instance.GetTypedRenderSettings<UDataflowVectorSamplerRenderSettings>();
			if (Settings)
			{
				const FDataflowVectorSampler& InSampler = GetSampler(Instance);

				FBox RenderBounds = InSampler.GetRenderBounds();
				if (Settings->bOverrideBounds)
				{
					RenderBounds.Min = Settings->Center - Settings->Extent;
					RenderBounds.Max = Settings->Center + Settings->Extent;
				}

				TArray<FVector> Points = GetPointsOnSlice(RenderBounds.GetCenter(), 2.f * RenderBounds.GetExtent(), Settings->Plane, Settings->Delta, Settings->Offset);

				const int32 NumPoints = Points.Num();

				TArray<FVector3f> Values; Values.SetNumUninitialized(NumPoints);
				InSampler.Sample(TArray<FVector3f>(Points), Values);

				const FName PointComponentName = Instance.GetComponentName(TEXT("VectorSamplerPoints"));
				const FName LineComponentName = Instance.GetComponentName(TEXT("VectorSampler"));

				TArray<FRenderablePoint> RenderablePoints;
				RenderablePoints.SetNumUninitialized(NumPoints);

				TArray<FRenderableLine> RenderableLines;
				RenderableLines.SetNumUninitialized(NumPoints);

				for (int32 Idx = 0; Idx < NumPoints; ++Idx)
				{
					const float AttrValue = FMath::Clamp(Values[Idx].Length(), Settings->Min, Settings->Max);

					FLinearColor LinearColor = FLinearColor::Green;
					FColor Color = LinearColor.ToFColor(true).WithAlpha(255);

					if (Settings->Max - Settings->Min > UE_SMALL_NUMBER)
					{
						float NormAttrValue = (AttrValue - Settings->Min) / (Settings->Max - Settings->Min);

						LinearColor = Settings->ColorRamp.GetLinearColorValue(NormAttrValue);
						Color = LinearColor.ToFColor(true).WithAlpha(255);
					}

					RenderablePoints[Idx] = FRenderablePoint(Points[Idx], Color, Settings->Size);
					RenderableLines[Idx] = FRenderableLine(Points[Idx], Points[Idx] + FVector(Values[Idx]) * Settings->LengthScalar, Color, Settings->LineThickness);
				}
				
				if (ULineSetComponent* LineComponent = OutComponents.AddNewComponent<ULineSetComponent>(LineComponentName))				
				{
					LineComponent->AddLines(RenderableLines);
					LineComponent->SetLineMaterial(Material);
				}

				// Render box
				static const FName BoxComponentName = TEXT("RenderBounds");

				if (UBoxComponent* BoxComponent = OutComponents.AddNewComponent<UBoxComponent>(BoxComponentName))
				{
					BoxComponent->SetBoxExtent(RenderBounds.GetExtent());

					FTransform Transform = FTransform::Identity;
					Transform.SetTranslation(RenderBounds.GetCenter());

					BoxComponent->SetWorldTransform(Transform);
				}

				// Render points
				if (UPointSetComponent* PointComponent = OutComponents.AddNewComponent<UPointSetComponent>(PointComponentName))
				{
					PointComponent->ReservePoints(NumPoints);
					PointComponent->AddPoints(RenderablePoints);

					PointComponent->SetPointMaterial(Material);
				}
			}
		}

		UMaterialInterface* Material = nullptr;
	};

	///////////////////////////////////////////////////////////////////////////////////////////////////////////////

	void RegisterSamplerRenderableTypes()
	{
		UE_DATAFLOW_REGISTER_RENDERABLE_TYPE(FFloatSamplerSurfaceRenderableType);
		UE_DATAFLOW_REGISTER_RENDERABLE_TYPE(FVectorSamplerSurfaceRenderableType);
	}

	///////////////////////////////////////////////////////////////////////////////////////////////////////////////

}