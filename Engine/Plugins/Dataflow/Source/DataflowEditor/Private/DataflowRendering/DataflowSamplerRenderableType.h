// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataflowRendering/DataflowRenderableTypeSettings.h"
#include "DataflowRendering/DataflowRenderableTypeUtils.h"
#include "Curves/LinearColorRamp.h"
#include "UObject/ObjectPtr.h"

#include "DataflowSamplerRenderableType.generated.h"

UENUM(BlueprintType)
enum class EDataflowSlicePlane : uint8
{
	XYPlane UMETA(DisplayName = "XY Plane"),
	YZPlane UMETA(DisplayName = "YZ Plane"),
	ZXPlane UMETA(DisplayName = "ZX Plane"),
};

UCLASS(MinimalAPI, AutoExpandCategories = "")
class UDataflowFloatSamplerRenderSettings : public UDataflowRenderableTypeSettings
{
	GENERATED_BODY()

	UDataflowFloatSamplerRenderSettings(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

public:
	/** Override the sampler's specified render bounds and use the Center end Extent instead */
	UPROPERTY(EditAnywhere, Category = "Render Bounds")
	bool bOverrideBounds = false;

	/** Center of the render bounds */
	UPROPERTY(EditAnywhere, Category = "Render Bounds", meta = (EditCondition = bOverrideBounds))
	FVector Center = FVector(0.f);

	/** Extent of the render bounds */
	UPROPERTY(EditAnywhere, Category = "Render Bounds", meta = (UIMin = 0.01f, ClampMin = 0.01f, EditCondition = bOverrideBounds))
	FVector Extent = FVector(50.0f);

	/** Plane orientation for the visualization */
	UPROPERTY(EditAnywhere, Category = "Slice Settings")
	EDataflowSlicePlane Plane = EDataflowSlicePlane::XYPlane;

	/** Distance between the displayed points on the plane */
	UPROPERTY(EditAnywhere, Category = "Slice Settings", meta = (UIMin = 0.01f, ClampMin = 0.01f))
	float PointSeparation = 2.f;

	/** Relative offset of the plane inside the render bounds. The offset is beetween -1 and 1 */
	UPROPERTY(EditAnywhere, Category = "Slice Settings", meta = (ClampMin = "-1", ClampMax = "1"))
	float Offset = 0.f;

	/** Size of the displayed points */
	UPROPERTY(EditAnywhere, Category = "Points", meta = (UIMin = 0.01f, ClampMin = 0.01f))
	float Size = 5.0f;

	/** Attribute minimum value for coloring. This value represents the minimum color on the color ramp */
	UPROPERTY(EditAnywhere, Category = "Color")
	float Min = 0.f;

	/** Attribute maximum value for coloring. This value represents the maximum color on the color ramp */
	UPROPERTY(EditAnywhere, Category = "Color")
	float Max = 1.f;

	/** Color ramp specifiyng the color for displayed points using a single value attribute */
	UPROPERTY(EditAnywhere, Category = "Color")
	FLinearColorRamp ColorRamp;
};


UCLASS(MinimalAPI, AutoExpandCategories = "")
class UDataflowVectorSamplerRenderSettings : public UDataflowRenderableTypeSettings
{
	GENERATED_BODY()

	UDataflowVectorSamplerRenderSettings(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

public:
	/** Override the sampler's specified render bounds and use the Center end Extent instead */
	UPROPERTY(EditAnywhere, Category = "Render Bounds")
	bool bOverrideBounds = false;

	/** Center of the render bounds */
	UPROPERTY(EditAnywhere, Category = "Render Bounds", meta = (EditCondition = bOverrideBounds))
	FVector Center = FVector(0.f);

	/** Extent of the render bounds */
	UPROPERTY(EditAnywhere, Category = "Render Bounds", meta = (UIMin = 0.01f, ClampMin = 0.01f, EditCondition = bOverrideBounds))
	FVector Extent = FVector(50.0f);

	/** Plane orientation for the visualization */
	UPROPERTY(EditAnywhere, Category = "Slice Settings")
	EDataflowSlicePlane Plane = EDataflowSlicePlane::XYPlane;

	/** Distance between the displayed points on the plane */
	UPROPERTY(EditAnywhere, Category = "Slice Settings", meta = (UIMin = 0.01f, ClampMin = 0.01f))
	float Delta = 5.f;

	/** Relative offset of the plane inside the render bounds. The offset is beetween -1 and 1 */
	UPROPERTY(EditAnywhere, Category = "Slice Settings", meta = (ClampMin = "-1", ClampMax = "1"))
	float Offset = 0.f;

	/** Size of displayed points */
	UPROPERTY(EditAnywhere, Category = "Points", meta = (UIMin = 0.01f, ClampMin = 0.01f))
	float Size = 5.0f;

	/** Scalar applied to the vectors */
	UPROPERTY(EditAnywhere, Category = "Vectors")
	float LengthScalar = 1.f;

	/** Line thickness of the displayed vectors */
	UPROPERTY(EditAnywhere, Category = "Vectors")
	float LineThickness = 1.f;

	/** Attribute minimum value for coloring. This value represents the minimum color on the color ramp */
	UPROPERTY(EditAnywhere, Category = "Color")
	float Min = 0.f;

	/** Attribute maximum value for coloring. This value represents the maximum color on the color ramp */
	UPROPERTY(EditAnywhere, Category = "Color")
	float Max = 1.f;

	/** Color ramp specifiyng the color for displayed points and vectors using a the length of the vector attribute */
	UPROPERTY(EditAnywhere, Category = "Color")
	FLinearColorRamp ColorRamp;
};



namespace UE::Dataflow::Private
{
	void RegisterSamplerRenderableTypes();
}

