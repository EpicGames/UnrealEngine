// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataflowRendering/DataflowSkeletalMeshRenderableType.h"

#include "Components/SkeletalMeshComponent.h"

#include "Dataflow/DataflowRenderingViewMode.h"

#include "DataflowRendering/DataflowRenderableComponents.h"
#include "DataflowRendering/DataflowRenderableType.h"
#include "DataflowRendering/DataflowRenderableTypeInstance.h"
#include "DataflowRendering/DataflowRenderableTypeRegistry.h"
#include "DataflowRendering/DataflowRenderableTypeSettings.h"
#include "DataflowRendering/DataflowRenderableTypeUtils.h"

#include "Engine/SkeletalMesh.h"

#include "UObject/ObjectPtr.h"

namespace UE::Dataflow::Private
{
	///////////////////////////////////////////////////////////////////////////////////////////////////////////////

	class FSkeletalMeshSurfaceRenderableType : public IRenderableType
	{
		UE_DATAFLOW_IRENDERABLE_PRIMARY_TYPE(TObjectPtr<USkeletalMesh>, SkeletalMesh);
		UE_DATAFLOW_IRENDERABLE_RENDER_GROUP(Surface);
		UE_DATAFLOW_IRENDERABLE_VIEW_MODE(FDataflowConstruction3DViewMode);

		virtual void GetPrimitiveComponents(const FRenderableTypeInstance& Instance, FRenderableComponents& OutComponents) const override
		{
			if (const TObjectPtr<USkeletalMesh> SkeletalMesh = GetSkeletalMesh(Instance, nullptr))
			{
				const FName ComponentName = Instance.GetComponentName(SkeletalMesh->GetFName());

				if (USkeletalMeshComponent* Component = OutComponents.AddNewComponent<USkeletalMeshComponent>(ComponentName))
				{
					Component->SetSkeletalMesh(SkeletalMesh);
				}
			}
		}
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

	class FSkeletalMeshUvsRenderableType : public IRenderableType
	{
		UE_DATAFLOW_IRENDERABLE_PRIMARY_TYPE(TObjectPtr<USkeletalMesh>, SkeletalMesh);
		UE_DATAFLOW_IRENDERABLE_RENDER_GROUP(Uvs);
		UE_DATAFLOW_IRENDERABLE_VIEW_MODE(FDataflowConstructionUVViewMode);
		UE_DATAFLOW_IRENDERABLE_SETTINGS(UDataflowUVsRenderSettings);

		virtual void GetPrimitiveComponents(const FRenderableTypeInstance& Instance, FRenderableComponents& OutComponents) const override
		{
			const UDataflowUVsRenderSettings* Settings = Instance.GetTypedRenderSettings<UDataflowUVsRenderSettings>();

			if (const TObjectPtr<USkeletalMesh> SkeletalMesh = GetSkeletalMesh(Instance, nullptr))
			{
				const int32 UVChannel = Settings
					? Settings->GetUVChannel(Rendering::GetUVChannelFromInstance(Instance))
					: 0;
				Rendering::AddUvDynamicMeshComponent(*SkeletalMesh, UVChannel, OutComponents);
			}
		}
	};

	///////////////////////////////////////////////////////////////////////////////////////////////////////////////

	void RegisterSkeletalMeshRenderableTypes()
	{
		UE_DATAFLOW_REGISTER_RENDERABLE_TYPE(FSkeletalMeshSurfaceRenderableType);
		UE_DATAFLOW_REGISTER_RENDERABLE_TYPE(FSkeletalMeshUvsRenderableType);
	}
}