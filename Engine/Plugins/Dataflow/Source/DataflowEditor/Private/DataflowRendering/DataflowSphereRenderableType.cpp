// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataflowRendering/DataflowSphereRenderableType.h"

#include "Components/StaticMeshComponent.h"
#include "Components/InstancedStaticMeshComponent.h"

#include "Dataflow/DataflowRenderingViewMode.h"
#include "Dataflow/DataflowEngineUtil.h"

#include "DataflowRendering/DataflowRenderableComponents.h"
#include "DataflowRendering/DataflowRenderableType.h"
#include "DataflowRendering/DataflowRenderableTypeInstance.h"
#include "DataflowRendering/DataflowRenderableTypeRegistry.h"
#include "DataflowRendering/DataflowRenderableTypeUtils.h"

#include "Engine/StaticMesh.h"

#include "UObject/ObjectPtr.h"

#include "Materials/MaterialInstanceDynamic.h"

namespace UE::Dataflow::Private
{
	///////////////////////////////////////////////////////////////////////////////////////////////////////////////

	class FSphereSurfaceRenderableType : public IRenderableType
	{
		UE_DATAFLOW_IRENDERABLE_PRIMARY_TYPE(FSphere, Sphere);
		UE_DATAFLOW_IRENDERABLE_RENDER_GROUP(Surface);
		UE_DATAFLOW_IRENDERABLE_VIEW_MODE(FDataflowConstruction3DViewMode);
		UE_DATAFLOW_IRENDERABLE_SETTINGS(UDataflowSphereRenderSettings);

		virtual void GetPrimitiveComponents(const FRenderableTypeInstance& Instance, FRenderableComponents& OutComponents) const override
		{
			const FSphere Sphere = GetSphere(Instance, {});

			const FName SphereComponentName = Instance.GetComponentName(TEXT("Sphere"));

			UStaticMesh* SphereMesh = UE::Dataflow::RenderGeometry::GetDataflowSphere();
			if (SphereMesh)
			{
				if (UStaticMeshComponent* Component = OutComponents.AddNewComponent<UStaticMeshComponent>(SphereComponentName))
				{
					Component->SetStaticMesh(SphereMesh);

					FTransform SphereTransform;
					SphereTransform.SetTranslation(Sphere.Center);
					SphereTransform.SetScale3D(FVector(Sphere.W) / 50.f);

					Component->SetWorldTransform(SphereTransform);
					Component->SetCollisionEnabled(ECollisionEnabled::NoCollision);

					const UDataflowSphereRenderSettings* Settings = Instance.GetTypedRenderSettings<UDataflowSphereRenderSettings>();
					if (Settings)
					{
						UMaterialInterface* Material = Settings->ColorSettings.bWireframe
							? UE::Dataflow::RenderMaterial::GetDataflowColorWireframeMaterial()
							: UE::Dataflow::RenderMaterial::GetDataflowColorMaterial();
						if (Material)
						{
							UMaterialInstanceDynamic* MaterialInstance = UMaterialInstanceDynamic::Create(Material, Component);
							if (MaterialInstance)
							{
								Component->SetMaterial(0, MaterialInstance);

								MaterialInstance->SetVectorParameterValue(FName("Color"), Settings->ColorSettings.Color);
								MaterialInstance->SetScalarParameterValue(FName("Transparency"), Settings->ColorSettings.Transparency);
							}
						}
					}
				}
			}
		}
	};

	///////////////////////////////////////////////////////////////////////////////////////////////////////////////

	class FSphereArraySurfaceRenderableType : public IRenderableType
	{
		UE_DATAFLOW_IRENDERABLE_PRIMARY_TYPE(TArray<FSphere>, SphereArray);
		UE_DATAFLOW_IRENDERABLE_RENDER_GROUP(Surface);
		UE_DATAFLOW_IRENDERABLE_VIEW_MODE(FDataflowConstruction3DViewMode);
		UE_DATAFLOW_IRENDERABLE_SETTINGS(UDataflowSphereArrayRenderSettings);

		virtual void GetPrimitiveComponents(const FRenderableTypeInstance& Instance, FRenderableComponents& OutComponents) const override
		{
			UStaticMesh* SphereMesh = UE::Dataflow::RenderGeometry::GetDataflowSphere();
			if (SphereMesh)
			{
				const TArray<FSphere>& SphereArray = GetSphereArray(Instance, {});

				if (SphereArray.Num())
				{
					const FName SphereArrayComponentName = Instance.GetComponentName(TEXT("SphereArray"));

					const int32 NumSpheres = SphereArray.Num();

					UInstancedStaticMeshComponent* ISMComponent = OutComponents.AddNewComponent<UInstancedStaticMeshComponent>(SphereArrayComponentName);
					if (ISMComponent)
					{
						ISMComponent->PreAllocateInstancesMemory(NumSpheres);
						ISMComponent->SetStaticMesh(SphereMesh);

						ISMComponent->SetCollisionEnabled(ECollisionEnabled::NoCollision);

						FBox ComponentBoundingBox;
						for (const FSphere& Sphere : SphereArray)
						{
							ComponentBoundingBox += FBox(Sphere.Center - FVector(Sphere.W), Sphere.Center + FVector(Sphere.W));

							FTransform SphereTransform;
							SphereTransform.SetTranslation(Sphere.Center);
							SphereTransform.SetScale3D(FVector(Sphere.W) / 50.f);

							ISMComponent->AddInstance(SphereTransform);
						}

						ISMComponent->SetNumCustomDataFloats(3); // RGB

						const UDataflowSphereArrayRenderSettings* Settings = Instance.GetTypedRenderSettings<UDataflowSphereArrayRenderSettings>();
						if (Settings)
						{
							UMaterialInterface* Material = Settings->ColorSettings.bWireframe
								? UE::Dataflow::RenderMaterial::GetDataflowColorWireframeMaterial()
								: UE::Dataflow::RenderMaterial::GetDataflowColorMaterial();
							if (Material)
							{
								UMaterialInstanceDynamic* MaterialInstance = UMaterialInstanceDynamic::Create(Material, ISMComponent);
								if (MaterialInstance)
								{
									ISMComponent->SetMaterial(0, MaterialInstance);

									MaterialInstance->SetScalarParameterValue(FName("Transparency"), Settings->ColorSettings.Transparency);

									TArray<float> Colors; // NumInstances * 3
									Colors.Init(1.0, SphereArray.Num() * 3);

									int32 Idx = 0;
									for (const FSphere& Sphere : SphereArray)
									{
										FLinearColor Color = UE::Dataflow::Rendering::GetColor(Idx,
											ComponentBoundingBox,
											Sphere.Center,
											2.0 * Sphere.W,
											Settings->ColorSettings);

										Colors[3 * Idx + 0] = Color.R;
										Colors[3 * Idx + 1] = Color.G;
										Colors[3 * Idx + 2] = Color.B;

										Idx++;
									}

									ISMComponent->SetCustomData(0, SphereArray.Num() - 1, Colors);
								}
							}
						}
					}
				}
			}
		}
	};
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////

namespace UE::Dataflow::Private
{
	void RegisterSphereRenderableTypes()
	{
		UE_DATAFLOW_REGISTER_RENDERABLE_TYPE(FSphereSurfaceRenderableType);
		UE_DATAFLOW_REGISTER_RENDERABLE_TYPE(FSphereArraySurfaceRenderableType);
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////

