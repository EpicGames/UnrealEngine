// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataflowRendering/DataflowStaticMeshRenderableType.h"

#include "Components/StaticMeshComponent.h"

#include "Dataflow/DataflowRenderingViewMode.h"

#include "DataflowRendering/DataflowRenderableComponents.h"
#include "DataflowRendering/DataflowRenderableType.h"
#include "DataflowRendering/DataflowRenderableTypeInstance.h"
#include "DataflowRendering/DataflowRenderableTypeRegistry.h"
#include "DataflowRendering/DataflowRenderableTypeUtils.h"

#include "Engine/StaticMesh.h"

#include "UObject/ObjectPtr.h"

namespace UE::Dataflow::Private
{
	///////////////////////////////////////////////////////////////////////////////////////////////////////////////

	class FStaticMeshSurfaceRenderableType : public IRenderableType
	{
		UE_DATAFLOW_IRENDERABLE_PRIMARY_TYPE(TObjectPtr<UStaticMesh>, StaticMesh);
		UE_DATAFLOW_IRENDERABLE_RENDER_GROUP(Surface);
		UE_DATAFLOW_IRENDERABLE_VIEW_MODE(FDataflowConstruction3DViewMode);

		virtual void GetPrimitiveComponents(const FRenderableTypeInstance& Instance, FRenderableComponents& OutComponents) const override
		{
			if (const TObjectPtr<UStaticMesh> StaticMesh = GetStaticMesh(Instance, nullptr))
			{
				const FName ComponentName = Instance.GetComponentName(StaticMesh->GetFName());

				if (UStaticMeshComponent* Component = OutComponents.AddNewComponent<UStaticMeshComponent>(ComponentName))
				{
					Component->SetStaticMesh(StaticMesh);
				}
			}
		}
	};

	///////////////////////////////////////////////////////////////////////////////////////////////////////////////

	class FStaticMeshUvsRenderableType : public IRenderableType
	{
		UE_DATAFLOW_IRENDERABLE_PRIMARY_TYPE(TObjectPtr<UStaticMesh>, StaticMesh)
		UE_DATAFLOW_IRENDERABLE_RENDER_GROUP(Uvs)
		UE_DATAFLOW_IRENDERABLE_VIEW_MODE(FDataflowConstructionUVViewMode);

		virtual void GetPrimitiveComponents(const FRenderableTypeInstance& Instance, FRenderableComponents& OutComponents) const override
		{
			if (const TObjectPtr<UStaticMesh> StaticMesh = GetStaticMesh(Instance, nullptr))
			{
				Rendering::AddUvDynamicMeshComponent(*StaticMesh, 0, OutComponents);
			}
		}
	};

	///////////////////////////////////////////////////////////////////////////////////////////////////////////////

	void RegisterStaticMeshRenderableTypes()
	{
		UE_DATAFLOW_REGISTER_RENDERABLE_TYPE(FStaticMeshSurfaceRenderableType);
		UE_DATAFLOW_REGISTER_RENDERABLE_TYPE(FStaticMeshUvsRenderableType);
	}
}