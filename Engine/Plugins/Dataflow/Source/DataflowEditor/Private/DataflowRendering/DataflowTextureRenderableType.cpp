// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataflowRendering/DataflowTextureRenderableType.h"

#include "Dataflow/DataflowImage.h"
#include "Dataflow/DataflowRenderingViewMode.h"

#include "DataflowRendering/DataflowRenderableComponents.h"
#include "DataflowRendering/DataflowRenderableType.h"
#include "DataflowRendering/DataflowRenderableTypeInstance.h"
#include "DataflowRendering/DataflowRenderableTypeRegistry.h"
#include "DataflowRendering/DataflowRenderableTypeUtils.h"

#include "Components/StaticMeshComponent.h"
#include "Materials/MaterialInterface.h"
#include "Materials/MaterialInstanceDynamic.h"

#include "Engine/StaticMesh.h"
#include "Engine/Texture2D.h"

#include "UObject/ObjectPtr.h"

namespace UE::Dataflow::Private
{
	///////////////////////////////////////////////////////////////////////////////////////////////////////////////

	static UStaticMesh* LoadTexturePreviewMesh()
	{
		return LoadObject<UStaticMesh>(nullptr, TEXT("/Engine/BasicShapes/Plane.Plane"));
	}

	static UMaterialInterface* LoadTexturePreviewMaterial()
	{
		return LoadObject<UMaterialInterface>(nullptr, TEXT("/Engine/EditorMaterials/Dataflow/M_Dataflow_TexturePreview.M_Dataflow_TexturePreview"));
	}

	static UMaterialInterface* MakeMaterialPreviewInstance(UObject* Owner, UTexture2D* InTexture, bool bRed, bool bGreen, bool bBlue, bool bAlpha, bool bGreyscale)
	{
		UMaterialInstanceDynamic* PreviewMaterialInstance = nullptr;
		if (UMaterialInterface* PreviewMaterial = LoadTexturePreviewMaterial())
		{
			PreviewMaterialInstance = UMaterialInstanceDynamic::Create(PreviewMaterial, Owner);
			if (PreviewMaterialInstance)
			{
				static FName TextureParameterName = TEXT("Texture");
				PreviewMaterialInstance->SetTextureParameterValue(TextureParameterName, InTexture);

				static FName ColorMaskParameterName = TEXT("ColorMask");

				const FVector4 ColorMask(
					bRed ? 1 : 0,
					bGreen ? 1 : 0,
					bBlue ? 1 : 0,
					bAlpha ? 1 : 0
				);
				PreviewMaterialInstance->SetVectorParameterValue(ColorMaskParameterName, ColorMask);

				static FName GreyScaleFactorParameterName = TEXT("GreyScaleFactor");
				PreviewMaterialInstance->SetScalarParameterValue(GreyScaleFactorParameterName, bGreyscale? 1: 0);
			}
		}
		return PreviewMaterialInstance;
	}

	static UTexture2D* CreateTexture2DFromImage(const FImage& InImage, FRenderableComponents& Components)
	{
		UTexture2D* NewTexture = Components.MakeNewObject<UTexture2D>();
		if (NewTexture)
		{
			// convert to the BGRA8
			FImage ConvertedImage(InImage);
			if (ConvertedImage.GetNumPixels() > 0)
			{
//				ConvertedImage.ChangeFormat(ERawImageFormat::Type::BGRA8, EGammaSpace::Linear);

#if WITH_EDITOR
				NewTexture->PreEditChange(nullptr);
#endif

#if WITH_EDITORONLY_DATA
				NewTexture->Source.Init(ConvertedImage);
#endif
				NewTexture->UpdateResource();

#if WITH_EDITOR
				NewTexture->PostEditChange();
#endif
			}
		}
		return NewTexture;
	}

	class FTexture2DRenderableType : public IRenderableType
	{
		UE_DATAFLOW_IRENDERABLE_PRIMARY_TYPE(TObjectPtr<UTexture2D>, Texture);
		UE_DATAFLOW_IRENDERABLE_RENDER_GROUP(Texture);
		UE_DATAFLOW_IRENDERABLE_VIEW_MODE(FDataflowConstruction2DViewMode);

		virtual void GetPrimitiveComponents(const FRenderableTypeInstance& Instance, FRenderableComponents& OutComponents) const override
		{
			const TObjectPtr<UTexture2D> Texture = GetTexture(Instance, {});

			// we use the number of component to determine the offset of component in the XY plane 
			// this allow us to display multiple images at the same time without having them overlapping 
			const int32 NumComponents = OutComponents.GetComponents().Num();

			const FName ComponentName = Instance.GetComponentName(TEXT("Texture"));

			if (UStaticMeshComponent* Component = OutComponents.AddNewComponent<UStaticMeshComponent>(ComponentName))
			{
				if (Texture)
				{
					UMaterialInterface* PreviewMaterialInstance = MakeMaterialPreviewInstance(Component, Texture, /*bRed*/true, /*bGreen*/true, /*bBlue*/true, /*bAlpha*/true, /*bGreyscale*/false);
					Component->SetMaterial(0, PreviewMaterialInstance);
				}
				UStaticMesh* PreviewMesh = LoadTexturePreviewMesh();
				if (PreviewMesh)
				{
					Component->SetStaticMesh(PreviewMesh);

					const FVector PreviewMeshExtent = PreviewMesh->GetBoundingBox().GetExtent();
					const FVector Translation(NumComponents * PreviewMeshExtent.X * 2.2, 0, 0);

					// texture is upside down by default , let's flip it around X
					Component->SetWorldTransform(FTransform(FRotator(0, 0, 180), Translation));
				}
			}
		}
	};

	class FImageRenderableType : public IRenderableType
	{
		UE_DATAFLOW_IRENDERABLE_PRIMARY_TYPE(FDataflowImage, Image);
		UE_DATAFLOW_IRENDERABLE_RENDER_GROUP(Texture);
		UE_DATAFLOW_IRENDERABLE_VIEW_MODE(FDataflowConstruction2DViewMode);

		virtual void GetPrimitiveComponents(const FRenderableTypeInstance& Instance, FRenderableComponents& OutComponents) const override
		{
			const FDataflowImage& Image = GetImage(Instance, {});

			TObjectPtr<UTexture2D> TextureObject = nullptr;
			if (Instance.HasUptoDateCachedValue())
			{
				TextureObject = Instance.GetCachedValue<TObjectPtr<UTexture2D>>();
			}
			else
			{
				TextureObject = CreateTexture2DFromImage(Image.GetImage(), OutComponents);
			}
			Instance.CacheValue(TextureObject);

			// we use the number of component to determine the offset of component in the XY plane 
			// this allow us to display multiple images at the same time without having them overlapping 
			const int32 NumComponents = OutComponents.GetComponents().Num();

			static const FName ComponentName = TEXT("Image");
			if (UStaticMeshComponent* Component = OutComponents.AddNewComponent<UStaticMeshComponent>(ComponentName))
			{
				if (TextureObject)
				{
					const bool bGreyscale = Image.GetImage().Format == ERawImageFormat::R32F;
					UMaterialInterface* PreviewMaterialInstance = MakeMaterialPreviewInstance(Component, TextureObject, /*bRed*/true, /*bGreen*/true, /*bBlue*/true, /*bAlpha*/true, bGreyscale);
					Component->SetMaterial(0, PreviewMaterialInstance);
				}

				UStaticMesh* PreviewMesh = LoadTexturePreviewMesh();
				if (PreviewMesh)
				{
					Component->SetStaticMesh(PreviewMesh);

					const FVector PreviewMeshExtent = PreviewMesh->GetBoundingBox().GetExtent();
					const FVector Translation(NumComponents * PreviewMeshExtent.X * 2.2, 0, 0);

					// texture is upside down by default , let's flip it around X
					Component->SetWorldTransform(FTransform(FRotator(0, 0, 180), Translation));
				}
			}
		}
	};

	///////////////////////////////////////////////////////////////////////////////////////////////////////////////

	void RegisterTextureRenderableTypes()
	{
		UE_DATAFLOW_REGISTER_RENDERABLE_TYPE(FTexture2DRenderableType);
		UE_DATAFLOW_REGISTER_RENDERABLE_TYPE(FImageRenderableType);
	}

	///////////////////////////////////////////////////////////////////////////////////////////////////////////////

}