// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataflowSimulationBinding.h"
#include "CommonFrameRates.h"
#include "Chaos/CacheCollection.h"
#include "Chaos/ChaosCache.h"
#include "Dataflow/DataflowSimulationScene.h"
#include "Fonts/FontMeasure.h"

FDataflowSimulationBinding::FDataflowSimulationBinding(const TWeakPtr<FDataflowSimulationScene>&  InSimulationScene)
	: SimulationScene(InSimulationScene)
{
	if (const TSharedPtr<FDataflowSimulationScene> PreviewScene = SimulationScene.Pin())
	{
		OnSceneTimeRangeUpdated(*PreviewScene);
		PreviewScene->OnTimeRangeUpdated.AddRaw(this, &FDataflowSimulationBinding::OnSceneTimeRangeUpdated);
	}
}

FDataflowSimulationBinding::~FDataflowSimulationBinding()
{
	if (const TSharedPtr<FDataflowSimulationScene> PreviewScene = SimulationScene.Pin())
	{
		PreviewScene->OnTimeRangeUpdated.RemoveAll(this);
	}
}

void FDataflowSimulationBinding::OnSceneTimeRangeUpdated(const FDataflowSimulationScene& Scene)
{
	const FAnimatedRange NewRange(Scene.GetTimeRange()[0], Scene.GetTimeRange()[1]);
	ViewRange = NewRange;
	WorkingRange = NewRange;
}

FFrameRate FDataflowSimulationBinding::GetFrameRate() const
{
	if (const TSharedPtr<const FDataflowSimulationScene> PreviewScene = SimulationScene.Pin())
	{
		return FFrameRate(PreviewScene->GetFrameRate(), 1);
	}
	return FFrameRate();
}

int32 FDataflowSimulationBinding::GetTickResolution() const
{
	return int32(1000);
}

FAnimatedRange FDataflowSimulationBinding::GetViewRange() const
{
	return ViewRange;
}

void FDataflowSimulationBinding::SetViewRange(TRange<double> InRange)
{
	ViewRange = InRange;

	if (WorkingRange.HasLowerBound() && WorkingRange.HasUpperBound())
	{
		WorkingRange = TRange<double>::Hull(WorkingRange, ViewRange);
	}
	else
	{
		WorkingRange = ViewRange;
	}
}

FAnimatedRange FDataflowSimulationBinding::GetWorkingRange() const
{
	return WorkingRange;
}

TRange<FFrameNumber> FDataflowSimulationBinding::GetPlaybackRange() const
{
	if (const TSharedPtr<const FDataflowSimulationScene> PreviewScene = SimulationScene.Pin())
	{
		const float TickResolution = static_cast<float>(GetTickResolution());
		const float StartRange = PreviewScene->GetTimeRange()[0];
		const float EndRange = PreviewScene->GetTimeRange()[1];
		return TRange<FFrameNumber>(
			FFrameNumber(FMath::RoundToInt32(StartRange * TickResolution)),
			FFrameNumber(FMath::RoundToInt32(EndRange * TickResolution))
		);
	}
	return TRange<FFrameNumber>(FFrameNumber(0), FFrameNumber(150)); // default 
}


void FDataflowSimulationBinding::SetPlaybackRange(const TRange<FFrameNumber>& NewRange)
{
	if (const TSharedPtr<FDataflowSimulationScene> PreviewScene = SimulationScene.Pin())
	{
		const float TickResolution = static_cast<double>(GetTickResolution());
		if (TickResolution > 0)
		{
			FVector2f TimeRange = {
				((float)NewRange.GetLowerBoundValue().Value / TickResolution),
				((float)NewRange.GetUpperBoundValue().Value / TickResolution),
			};
			PreviewScene->SetTimeRange(TimeRange);
		}
	}
}


FFrameNumber FDataflowSimulationBinding::GetScrubPosition() const
{
	if (const TSharedPtr<const FDataflowSimulationScene> PreviewScene = SimulationScene.Pin())
	{
		return FFrameNumber(FMath::RoundToInt32(GetScrubTime() * (double)GetTickResolution()));
	}

	return FFrameNumber(0);
}

float FDataflowSimulationBinding::GetScrubTime() const
{
	if (const TSharedPtr<FDataflowSimulationScene> PreviewScene = SimulationScene.Pin())
	{
		return PreviewScene->SimulationTime-PreviewScene->GetTimeRange()[0];
	}

	return 0.0f;
}

void FDataflowSimulationBinding::SetScrubPosition(FFrameTime ScrubPosition) const
{
	if (const TSharedPtr<FDataflowSimulationScene> PreviewScene = SimulationScene.Pin())
	{
		const float TickResolution = static_cast<float>(GetTickResolution());
		const float ScrubTime = ScrubPosition.AsDecimal() / TickResolution;
		PreviewScene->SimulationTime = ScrubTime + PreviewScene->GetTimeRange()[0];
	}
}

void FDataflowSimulationBinding::SetScrubTime(const float ScrubTime)
{
	if (const TSharedPtr<FDataflowSimulationScene> PreviewScene = SimulationScene.Pin())
	{
		PreviewScene->SimulationTime = ScrubTime + PreviewScene->GetTimeRange()[0];
	}
}

float FDataflowSimulationBinding::GetDeltaTime() const
{
	if (const TSharedPtr<FDataflowSimulationScene> PreviewScene = SimulationScene.Pin())
	{
		return PreviewScene->GetDeltaTime();
	}
	return 0.0f;
}

float FDataflowSimulationBinding::GetSequenceLength() const
{
	if (const TSharedPtr<FDataflowSimulationScene> PreviewScene = SimulationScene.Pin())
	{
		return PreviewScene->GetTimeRange()[1] - PreviewScene->GetTimeRange()[0];
	}
	return 0.0f;
}

void FDataflowSimulationBinding::RecordSimulationCache()
{
	if (const TSharedPtr<FDataflowSimulationScene> PreviewScene = SimulationScene.Pin())
	{
		return PreviewScene->RecordSimulationCache();
	}
}

void FDataflowSimulationBinding::ResetSimulationScene()
{
	if (const TSharedPtr<FDataflowSimulationScene> PreviewScene = SimulationScene.Pin())
	{
		const bool bSimulationWasEnabled = PreviewScene->IsSimulationEnabled();
		PreviewScene->RebuildSimulationScene();
		PreviewScene->SetSimulationEnabled(bSimulationWasEnabled);
	}
}

bool FDataflowSimulationBinding::IsSimulationLocked() const
{
	if (const TSharedPtr<FDataflowSimulationScene> PreviewScene = SimulationScene.Pin())
	{
		return PreviewScene->IsSimulationLocked();
	}
	return false;
}

void FDataflowSimulationBinding::SetSimulationLocked(const bool bIsSimulaitonLocked)
{
	if (const TSharedPtr<FDataflowSimulationScene> PreviewScene = SimulationScene.Pin())
	{
		PreviewScene->SetSimulationLocked(bIsSimulaitonLocked, false);
	}
}

void FDataflowSimulationBinding::FillCacheNames(TArray<FString>& CacheNames) const
{
	CacheNames.Reset();
	if (const TSharedPtr<FDataflowSimulationScene> PreviewScene = SimulationScene.Pin())
	{
		if(UDataflowSimulationSceneDescription* SceneDescription = PreviewScene->GetPreviewSceneDescription())
		{
			if(SceneDescription->CacheAsset)
			{
				for(UChaosCache* Cache : SceneDescription->CacheAsset->Caches)
				{
					CacheNames.Add(Cache->GetName());
				}
			}
		}
	}
}

void FDataflowSimulationBinding::HandleViewRangeChanged(TRange<double> InRange, EViewRangeInterpolation InInterpolation)
{
	SetViewRange(InRange);
}

void FDataflowSimulationBinding::HandleWorkingRangeChanged(TRange<double> InRange)
{
	WorkingRange = InRange;
}