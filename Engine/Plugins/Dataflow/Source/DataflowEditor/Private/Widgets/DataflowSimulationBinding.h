// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "ITimeSlider.h"
#include "TimeSliderArgs.h"
#include "Dataflow/DataflowSimulationScene.h"
#include "Editor/SequencerWidgets/Public/AnimatedRange.h"

/** Dataflow binding to centralize the bridge in between the timeline widget and the dataflow simulation scene */
class FDataflowSimulationBinding : public TSharedFromThis<FDataflowSimulationBinding>
{
public:
	FDataflowSimulationBinding(const TWeakPtr<FDataflowSimulationScene>&  InSimulationScene);
	~FDataflowSimulationBinding();

	/** Get the simulation scene from the binding */
	TWeakPtr<FDataflowSimulationScene> GetSimulationScene() const { return SimulationScene; }

	/** Get the framerate specified by the anim sequence */
	FFrameRate GetFrameRate() const;

	/** Get the tick resolution we are displaying at */
	int32 GetTickResolution() const;

	/** Get the current view range */
	FAnimatedRange GetViewRange() const;

	/** Set the current view range */
	void SetViewRange(TRange<double> InRange);

	/** Get the working range of the model's data */
	FAnimatedRange GetWorkingRange() const;

	/** Get the playback range of the model's data */
	TRange<FFrameNumber> GetPlaybackRange() const;

	/** Set the playback range  */
	void SetPlaybackRange(const TRange<FFrameNumber>& NewRange);

	/** Get the current scrub position */
	FFrameNumber GetScrubPosition() const;

	/** Get the current scrub time */
	float GetScrubTime() const;

	/** Set the current scrub time */
	void SetScrubTime(const float NewScrubTime);
	
	/** Get the simulation delta time*/
	float GetDeltaTime() const;

	/** Get the simulation sequence length */
	float GetSequenceLength() const;

	/** Set the current scrub position */
	void SetScrubPosition(FFrameTime NewScrubPostion) const;

	/** Handle the view range being changed */
	void HandleViewRangeChanged(TRange<double> InRange, EViewRangeInterpolation InInterpolation);

	/** Handle the working range being changed */
	void HandleWorkingRangeChanged(TRange<double> InRange);

	/** Handle the recording of the simulation cache */
	void RecordSimulationCache();

	/** Handle the reset of the simulation scene */
	void ResetSimulationScene();

	/** Get the locked simulation flag */
	bool IsSimulationLocked() const;

	/** Set the locked simulation flag */
	void SetSimulationLocked(const bool bIsSimulaitonLocked);

	/** Retrieve all the cache names */
	void FillCacheNames(TArray<FString>& CacheNames) const;

private:

	void OnSceneTimeRangeUpdated(const FDataflowSimulationScene& Scene);

	/** Simulation scene to be used for the widget */
	TWeakPtr<FDataflowSimulationScene> SimulationScene;

	/** View simulation range */
	FAnimatedRange ViewRange;

	/** Working simulation range */
	FAnimatedRange WorkingRange;
};
