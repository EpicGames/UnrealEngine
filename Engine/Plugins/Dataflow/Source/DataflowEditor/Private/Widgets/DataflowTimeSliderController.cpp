// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataflowTimeSliderController.h"

#include "CommonFrameRates.h"
#include "Fonts/FontMeasure.h"
#include "Framework/Application/SlateApplication.h"
#include "MovieSceneTimeHelpers.h"
#include "Styling/AppStyle.h"
#include "Widgets/SDataflowSimulationTimeline.h"

namespace ScrubConstants
{
	/**The fraction of the current view range to scroll per unit delta  */
	constexpr float ScrollPanFraction = 0.1f;

}

struct FDataflowTimeSliderController::FScrubRangeToScreen
{
	double ViewStart;
	float  PixelsPerInput;

	FScrubRangeToScreen(const TRange<double>& InViewInput, const FVector2D& InWidgetSize)
	{
		const double ViewInputRange = InViewInput.Size<double>();

		ViewStart = InViewInput.GetLowerBoundValue();
		PixelsPerInput = ViewInputRange > 0 ? static_cast<float>(InWidgetSize.X / ViewInputRange) : 0;
	}

	/** Local Widget Space -> Curve Input domain. */
	double LocalXToInput(float ScreenX) const
	{
		return PixelsPerInput > 0 ? (ScreenX / PixelsPerInput) + ViewStart : ViewStart;
	}

	/** Curve Input domain -> local Widget Space */
	float InputToLocalX(double Input) const
	{
		return static_cast<float>((Input - ViewStart) * PixelsPerInput);
	}
};

struct FDataflowTimeSliderController::FDrawTickArgs
{
	/** Geometry of the area */
	FGeometry AllottedGeometry;
	
	/** Culling rect of the area */
	FSlateRect CullingRect;
	
	/** Color of each tick */
	FLinearColor TickColor;
	
	/** Offset in Y where to start the tick */
	float TickOffset;
	
	/** Height in of major ticks */
	float MajorTickHeight;
	
	/** Start layer for elements */
	int32 StartLayer;
	
	/** Draw effects to apply */
	ESlateDrawEffect DrawEffects;
	
	/** Whether or not to only draw major ticks */
	bool bOnlyDrawMajorTicks;
	
	/** Whether or not to mirror labels */
	bool bMirrorLabels;

};

FDataflowTimeSliderController::FDataflowTimeSliderController(const FTimeSliderArgs& InArgs, TWeakPtr<SDataflowSimulationTimeline> InSimulationTimeline)
	: TimeSliderArgs(InArgs)
	, SimulationTimeline(InSimulationTimeline)
{
	ScrubFillBrush = FAppStyle::GetBrush(TEXT("Sequencer.Timeline.ScrubFill"));
	ScrubHandleUpBrush = FAppStyle::GetBrush(TEXT("Sequencer.Timeline.VanillaScrubHandleUp"));
	ScrubHandleDownBrush = FAppStyle::GetBrush(TEXT("Sequencer.Timeline.VanillaScrubHandleDown"));
	EditableTimeBrush = FAppStyle::GetBrush(TEXT("AnimTimeline.SectionMarker"));
}

FDataflowTimeSliderController::~FDataflowTimeSliderController()
{
}

int32 FDataflowTimeSliderController::OnPaintTimeSlider(bool bMirrorLabels, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	const bool bEnabled = bParentEnabled;
	const ESlateDrawEffect DrawEffects = bEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;

	const TRange<double> LocalViewRange = TimeSliderArgs.ViewRange.Get();
	const float    LocalViewRangeMin = static_cast<float>(LocalViewRange.GetLowerBoundValue());
	const float    LocalViewRangeMax = static_cast<float>(LocalViewRange.GetUpperBoundValue());
	const float    LocalSequenceLength = LocalViewRangeMax - LocalViewRangeMin;
	const TRange<FFrameNumber> LocalPlaybackRange = TimeSliderArgs.PlaybackRange.Get();

	if (LocalSequenceLength > 0)
	{
		FScrubRangeToScreen RangeToScreen(LocalViewRange, AllottedGeometry.Size);

		// draw tick marks
		constexpr float MajorTickHeight = 9.0f;

		FDrawTickArgs Args;
		{
			Args.AllottedGeometry = AllottedGeometry;
			Args.bMirrorLabels = bMirrorLabels;
			Args.bOnlyDrawMajorTicks = false;
			Args.TickColor = FLinearColor::White;
			Args.CullingRect = MyCullingRect;
			Args.DrawEffects = DrawEffects;
			Args.StartLayer = LayerId;
			Args.TickOffset = bMirrorLabels ? 0.0f : FMath::Abs(static_cast<float>(AllottedGeometry.Size.Y) - MajorTickHeight);
			Args.MajorTickHeight = MajorTickHeight;
		}

		DrawTicks(OutDrawElements, LocalViewRange, RangeToScreen, Args);

		// draw playback & selection range
		FPaintPlaybackRangeArgs PlaybackRangeArgs(
			bMirrorLabels ? FAppStyle::GetBrush("Sequencer.Timeline.PlayRange_Bottom_L") : FAppStyle::GetBrush("Sequencer.Timeline.PlayRange_Top_L"),
			bMirrorLabels ? FAppStyle::GetBrush("Sequencer.Timeline.PlayRange_Bottom_R") : FAppStyle::GetBrush("Sequencer.Timeline.PlayRange_Top_R"),
			6.f
		);

		LayerId = DrawPlaybackRange(AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, RangeToScreen, PlaybackRangeArgs);

		PlaybackRangeArgs.SolidFillOpacity = 0.05f;
		LayerId = DrawSelectionRange(AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, RangeToScreen, PlaybackRangeArgs);

		// Draw the scrub handle
		const float      HandleStart = RangeToScreen.InputToLocalX(TimeSliderArgs.ScrubPosition.Get().AsDecimal() / GetTickResolution().AsDecimal()) - 7.0f;
		const float      HandleEnd = HandleStart + 13.0f;

		const int32 ArrowLayer = LayerId + 2;
		FPaintGeometry MyGeometry = AllottedGeometry.ToPaintGeometry(FVector2f(HandleEnd - HandleStart, AllottedGeometry.Size.Y), FSlateLayoutTransform(FVector2f(HandleStart, 0.f)));
		FLinearColor ScrubColor = InWidgetStyle.GetColorAndOpacityTint();
		{
			ScrubColor.A = ScrubColor.A * 0.75f;
			ScrubColor.B *= 0.1f;
			ScrubColor.G *= 0.2f;
		}

		const FSlateBrush* Brush = (bMirrorLabels ? ScrubHandleUpBrush : ScrubHandleDownBrush);

		FSlateDrawElement::MakeBox(
			OutDrawElements,
			ArrowLayer,
			MyGeometry,
			Brush,
			DrawEffects,
			ScrubColor
		);

		{
			// Draw the current time next to the scrub handle
			FString FrameString = TimeSliderArgs.NumericTypeInterface->ToString(TimeSliderArgs.ScrubPosition.Get().GetFrame().Value);
			FSlateFontInfo SmallLayoutFont = FCoreStyle::GetDefaultFontStyle("Regular", 10);

			const TSharedRef< FSlateFontMeasure > FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
			const FVector2D TextSize = FontMeasureService->Measure(FrameString, SmallLayoutFont);

			// Flip the text position if getting near the end of the view range
			constexpr float TextOffsetPx = 2.f;
			const bool  bDrawLeft = (static_cast<float>(AllottedGeometry.Size.X) - HandleEnd) < (TextSize.X + 14.f) - TextOffsetPx;
			const float TextPosition = bDrawLeft ? HandleStart - static_cast<float>(TextSize.X) - TextOffsetPx : HandleEnd + TextOffsetPx;

			FVector2D TextOffset(TextPosition, Args.bMirrorLabels ? TextSize.Y - 6.f : Args.AllottedGeometry.Size.Y - (Args.MajorTickHeight + TextSize.Y));

			FSlateDrawElement::MakeText(
				OutDrawElements,
				Args.StartLayer + 1,
				Args.AllottedGeometry.ToPaintGeometry(TextSize, FSlateLayoutTransform(TextOffset)),
				FrameString,
				SmallLayoutFont,
				Args.DrawEffects,
				Args.TickColor
			);
		}

		return ArrowLayer;
	}

	return LayerId;
}

int32 FDataflowTimeSliderController::OnPaintViewArea(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, bool bEnabled, const FPaintViewAreaArgs& Args) const
{
	const ESlateDrawEffect DrawEffects = bEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;

	TRange<double> LocalViewRange = TimeSliderArgs.ViewRange.Get();
	FScrubRangeToScreen RangeToScreen(LocalViewRange, AllottedGeometry.Size);

	if (Args.PlaybackRangeArgs.IsSet())
	{
		FPaintPlaybackRangeArgs PaintArgs = Args.PlaybackRangeArgs.GetValue();
		LayerId = DrawPlaybackRange(AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, RangeToScreen, PaintArgs);
		PaintArgs.SolidFillOpacity = 0.2f;
		LayerId = DrawSelectionRange(AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, RangeToScreen, PaintArgs);
	}

	if (Args.bDisplayTickLines)
	{
		static FLinearColor TickColor(0.1f, 0.1f, 0.1f, 0.3f);

		// Draw major tick lines in the section area
		FDrawTickArgs DrawTickArgs;
		{
			DrawTickArgs.AllottedGeometry = AllottedGeometry;
			DrawTickArgs.bMirrorLabels = false;
			DrawTickArgs.bOnlyDrawMajorTicks = true;
			DrawTickArgs.TickColor = TickColor;
			DrawTickArgs.CullingRect = MyCullingRect;
			DrawTickArgs.DrawEffects = DrawEffects;
			// Draw major ticks under sections
			DrawTickArgs.StartLayer = LayerId - 1;
			// Draw the tick the entire height of the section area
			DrawTickArgs.TickOffset = 0.0f;
			DrawTickArgs.MajorTickHeight = static_cast<float>(AllottedGeometry.Size.Y);
		}

		DrawTicks(OutDrawElements, LocalViewRange, RangeToScreen, DrawTickArgs);
	}

	if (Args.bDisplayScrubPosition)
	{
		// Draw a line for the scrub position
		const float LinePos = RangeToScreen.InputToLocalX(TimeSliderArgs.ScrubPosition.Get().AsDecimal() / GetTickResolution().AsDecimal());

		TArray<FVector2D> LinePoints;
		{
			LinePoints.AddUninitialized(2);
			LinePoints[0] = FVector2D(0.0f, 0.0f);
			LinePoints[1] = FVector2D(0.0f, FMath::FloorToFloat(AllottedGeometry.Size.Y));
		}

		FSlateDrawElement::MakeLines(
			OutDrawElements,
			LayerId + 1,
			AllottedGeometry.ToPaintGeometry(FVector2f(1.0f, 1.0f), FSlateLayoutTransform(FVector2f(LinePos, 0.0f))),
			LinePoints,
			DrawEffects,
			FLinearColor(1.f, 1.f, 1.f, .5f),
			false
		);
	}

	return LayerId;
}

FReply FDataflowTimeSliderController::OnMouseButtonDown(SWidget& WidgetOwner, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	DistanceDragged = 0;
	MouseDownPosition[0] = MouseDownPosition[1] = MouseEvent.GetScreenSpacePosition();
	MouseDownGeometry = MyGeometry;	
	return FReply::Unhandled();
}

FReply FDataflowTimeSliderController::OnMouseButtonUp(SWidget& WidgetOwner, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	bool bHandleLeftMouseButton = MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton && WidgetOwner.HasMouseCapture();
	bool bHandleRightMouseButton = MouseEvent.GetEffectingButton() == EKeys::RightMouseButton && WidgetOwner.HasMouseCapture() && TimeSliderArgs.AllowZoom;

	FScrubRangeToScreen RangeToScreen = FScrubRangeToScreen(TimeSliderArgs.ViewRange.Get(), MyGeometry.Size);
	FFrameTime          MouseTime = ComputeFrameTimeFromMouse(MyGeometry, MouseEvent.GetScreenSpacePosition(), RangeToScreen);

	if (bHandleRightMouseButton)
	{
		if (!bPanning && DistanceDragged == 0.0f)
		{
			return SimulationTimeline.Pin()->OnMouseButtonUp(MyGeometry, MouseEvent).ReleaseMouseCapture();
		}

		bPanning = false;
		DistanceDragged = 0.f;

		return FReply::Handled().ReleaseMouseCapture();
	}
	else if (bHandleLeftMouseButton)
	{
		if (MouseDragType == DRAG_PLAYBACK_START)
		{
			TimeSliderArgs.OnPlaybackRangeEndDrag.ExecuteIfBound();
		}
		else if (MouseDragType == DRAG_PLAYBACK_END)
		{
			TimeSliderArgs.OnPlaybackRangeEndDrag.ExecuteIfBound();
		}
		else if (MouseDragType == DRAG_SELECTION_START)
		{
			TimeSliderArgs.OnSelectionRangeEndDrag.ExecuteIfBound();
		}
		else if (MouseDragType == DRAG_SELECTION_END)
		{
			TimeSliderArgs.OnSelectionRangeEndDrag.ExecuteIfBound();
		}
		else if (MouseDragType == DRAG_SETTING_RANGE)
		{
			// Zooming
			FFrameTime MouseDownStart = ComputeFrameTimeFromMouse(MyGeometry, MouseDownPosition[0], RangeToScreen);

			const bool bCanZoomIn = MouseTime > MouseDownStart;
			const bool bCanZoomOut = ViewRangeStack.Num() > 0;
			if (bCanZoomIn || bCanZoomOut)
			{
				TRange<double> ViewRange = TimeSliderArgs.ViewRange.Get();
				if (!bCanZoomIn)
				{
					ViewRange = ViewRangeStack.Pop();
				}

				if (bCanZoomIn)
				{
					// push the current value onto the stack
					ViewRangeStack.Add(ViewRange);

					ViewRange = TRange<double>(MouseDownStart.FrameNumber / GetTickResolution(), MouseTime.FrameNumber / GetTickResolution());
				}

				TimeSliderArgs.OnViewRangeChanged.ExecuteIfBound(ViewRange, EViewRangeInterpolation::Immediate);
				if (!TimeSliderArgs.ViewRange.IsBound())
				{
					// The output is not bound to a delegate so we'll manage the value ourselves
					TimeSliderArgs.ViewRange.Set(ViewRange);
				}
			}
		}
		else
		{
			TimeSliderArgs.OnEndScrubberMovement.ExecuteIfBound();

			CommitScrubPosition(MouseTime, /*bIsScrubbing=*/false);
		}

		MouseDragType = DRAG_NONE;
		DistanceDragged = 0.f;

		return FReply::Handled().ReleaseMouseCapture();
	}

	return FReply::Unhandled();
}

FReply FDataflowTimeSliderController::OnMouseButtonDoubleClick(SWidget& OwnerWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	return FReply::Unhandled();
}

FReply FDataflowTimeSliderController::OnMouseMove(SWidget& WidgetOwner, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	const bool bHandleLeftMouseButton = MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton);
	const bool bHandleRightMouseButton = MouseEvent.IsMouseButtonDown(EKeys::RightMouseButton) && TimeSliderArgs.AllowZoom;

	if (bHandleRightMouseButton)
	{
		if (!bPanning)
		{
			DistanceDragged += static_cast<float>(FMath::Abs(MouseEvent.GetCursorDelta().X));
			if (DistanceDragged > FSlateApplication::Get().GetDragTriggerDistance())
			{
				bPanning = true;
			}
		}
		else
		{
			const TRange<double> LocalViewRange = TimeSliderArgs.ViewRange.Get();
			const double LocalViewRangeMin = LocalViewRange.GetLowerBoundValue();
			const double LocalViewRangeMax = LocalViewRange.GetUpperBoundValue();

			const FScrubRangeToScreen ScaleInfo(LocalViewRange, MyGeometry.Size);
			const FVector2D ScreenDelta = MouseEvent.GetCursorDelta();
			FVector2D InputDelta;
			InputDelta.X = ScreenDelta.X / ScaleInfo.PixelsPerInput;

			double NewViewOutputMin = LocalViewRangeMin - InputDelta.X;
			double NewViewOutputMax = LocalViewRangeMax - InputDelta.X;

			ClampViewRange(NewViewOutputMin, NewViewOutputMax);
			SetViewRange(NewViewOutputMin, NewViewOutputMax, EViewRangeInterpolation::Immediate);
		}
	}
	else if (bHandleLeftMouseButton)
	{
		const TRange<double> LocalViewRange = TimeSliderArgs.ViewRange.Get();
		const FScrubRangeToScreen RangeToScreen(LocalViewRange, MyGeometry.Size);
		DistanceDragged += static_cast<float>(FMath::Abs(MouseEvent.GetCursorDelta().X));

		if (MouseDragType == DRAG_NONE)
		{
			if (DistanceDragged > FSlateApplication::Get().GetDragTriggerDistance())
			{
				const FFrameTime MouseDownFree = ComputeFrameTimeFromMouse(MyGeometry, MouseDownPosition[0], RangeToScreen, false);

				const FFrameRate FrameResolution = GetTickResolution();
				const bool       bLockedPlayRange = TimeSliderArgs.IsPlaybackRangeLocked.Get();
				const float      MouseDownPixel = RangeToScreen.InputToLocalX(MouseDownFree / FrameResolution);
				const bool       bHitScrubber = GetHitTestScrubberPixelRange(TimeSliderArgs.ScrubPosition.Get(), RangeToScreen).HandleRange.Contains(MouseDownPixel);
				
				const TRange<double>   SelectionRange = TimeSliderArgs.SelectionRange.Get() / FrameResolution;
				const TRange<double>   PlaybackRange = TimeSliderArgs.PlaybackRange.Get() / FrameResolution;

				// Disable selection range test if it's empty so that the playback range scrubbing gets priority
				if (!SelectionRange.IsEmpty() && !bHitScrubber && HitTestRangeEnd(RangeToScreen, SelectionRange, MouseDownPixel))
				{
					// selection range end scrubber
					MouseDragType = DRAG_SELECTION_END;
					TimeSliderArgs.OnSelectionRangeBeginDrag.ExecuteIfBound();
				}
				else if (!SelectionRange.IsEmpty() && !bHitScrubber && HitTestRangeStart(RangeToScreen, SelectionRange, MouseDownPixel))
				{
					// selection range start scrubber
					MouseDragType = DRAG_SELECTION_START;
					TimeSliderArgs.OnSelectionRangeBeginDrag.ExecuteIfBound();
				}
				else if (!bLockedPlayRange && !bHitScrubber && HitTestRangeEnd(RangeToScreen, PlaybackRange, MouseDownPixel))
				{
					// playback range end scrubber
					MouseDragType = DRAG_PLAYBACK_END;
					TimeSliderArgs.OnPlaybackRangeBeginDrag.ExecuteIfBound();
				}
				else if (!bLockedPlayRange && !bHitScrubber && HitTestRangeStart(RangeToScreen, PlaybackRange, MouseDownPixel))
				{
					// playback range start scrubber
					MouseDragType = DRAG_PLAYBACK_START;
					TimeSliderArgs.OnPlaybackRangeBeginDrag.ExecuteIfBound();
				}
				else if (FSlateApplication::Get().GetModifierKeys().AreModifersDown(EModifierKey::Control))
				{
					MouseDragType = DRAG_SETTING_RANGE;
				}
				else
				{
					MouseDragType = DRAG_SCRUBBING_TIME;
					TimeSliderArgs.OnBeginScrubberMovement.ExecuteIfBound();
				}
			}
		}
		else
		{
			FFrameTime MouseTime = ComputeFrameTimeFromMouse(MyGeometry, MouseEvent.GetScreenSpacePosition(), RangeToScreen);

			// Set the start range time?
			if (MouseDragType == DRAG_PLAYBACK_START)
			{
				SetPlaybackRangeStart(MouseTime.FrameNumber);
			}
			// Set the end range time?
			else if (MouseDragType == DRAG_PLAYBACK_END)
			{
				SetPlaybackRangeEnd(MouseTime.FrameNumber - 1);
			}
			else if (MouseDragType == DRAG_SELECTION_START)
			{
				SetSelectionRangeStart(MouseTime.FrameNumber);
			}
			// Set the end range time?
			else if (MouseDragType == DRAG_SELECTION_END)
			{
				SetSelectionRangeEnd(MouseTime.FrameNumber);
			}
			else if (MouseDragType == DRAG_SCRUBBING_TIME)
			{
				// Delegate responsibility for clamping to the current viewrange to the client
				CommitScrubPosition(MouseTime, /*bIsScrubbing=*/true);
			}
			else if (MouseDragType == DRAG_SETTING_RANGE)
			{
				MouseDownPosition[1] = MouseEvent.GetScreenSpacePosition();
			}
			else if (MouseDragType == DRAG_TIME)
			{
				return FReply::Unhandled();
			}
		}
	}

	if (DistanceDragged != 0.f && (bHandleLeftMouseButton || bHandleRightMouseButton))
	{
		return FReply::Handled().CaptureMouse(WidgetOwner.AsShared());
	}

	return FReply::Handled();

}

FReply FDataflowTimeSliderController::OnMouseWheel(SWidget& WidgetOwner, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	TOptional<TRange<float>> NewTargetRange;

	if (TimeSliderArgs.AllowZoom && MouseEvent.IsControlDown())
	{
		const float MouseFractionX = static_cast<float>(MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition()).X / MyGeometry.GetLocalSize().X);

		const float ZoomDelta = -0.2f * MouseEvent.GetWheelDelta();
		if (ZoomByDelta(ZoomDelta, MouseFractionX))
		{
			return FReply::Handled();
		}
	}
	else if (MouseEvent.IsShiftDown())
	{
		PanByDelta(-MouseEvent.GetWheelDelta());
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FCursorReply FDataflowTimeSliderController::OnCursorQuery(const SWidget& WidgetOwner, const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const
{
	const FScrubRangeToScreen RangeToScreen(TimeSliderArgs.ViewRange.Get(), MyGeometry.Size);

	const FFrameRate FrameResolution = GetTickResolution();
	const bool       bLockedPlayRange = TimeSliderArgs.IsPlaybackRangeLocked.Get();
	const float      HitTestPixel = static_cast<float>(MyGeometry.AbsoluteToLocal(CursorEvent.GetScreenSpacePosition()).X);
	const bool       bHitScrubber = GetHitTestScrubberPixelRange(TimeSliderArgs.ScrubPosition.Get(), RangeToScreen).HandleRange.Contains(HitTestPixel);
	
	const TRange<double>   SelectionRange = TimeSliderArgs.SelectionRange.Get() / FrameResolution;
	const TRange<double>   PlaybackRange = TimeSliderArgs.PlaybackRange.Get() / FrameResolution;

	if (MouseDragType == DRAG_SCRUBBING_TIME)
	{
		return FCursorReply::Unhandled();
	}

	// Use L/R resize cursor if we're dragging or hovering a playback range bound
	if ((MouseDragType == DRAG_PLAYBACK_END) ||
		(MouseDragType == DRAG_PLAYBACK_START) ||
		(MouseDragType == DRAG_SELECTION_START) ||
		(MouseDragType == DRAG_SELECTION_END) ||
		(!bLockedPlayRange && !bHitScrubber && HitTestRangeStart(RangeToScreen, PlaybackRange, HitTestPixel)) ||
		(!bLockedPlayRange && !bHitScrubber && HitTestRangeEnd(RangeToScreen, PlaybackRange, HitTestPixel)) ||
		(!SelectionRange.IsEmpty() && !bHitScrubber && HitTestRangeStart(RangeToScreen, SelectionRange, HitTestPixel)) ||
		(!SelectionRange.IsEmpty() && !bHitScrubber && HitTestRangeEnd(RangeToScreen, SelectionRange, HitTestPixel)))
	{
		return FCursorReply::Cursor(EMouseCursor::ResizeLeftRight);
	}

	return FCursorReply::Unhandled();
}

void FDataflowTimeSliderController::SetViewRange(double NewRangeMin, double NewRangeMax, EViewRangeInterpolation Interpolation)
{
	// Clamp to a minimum size to avoid zero-sized or negative visible ranges
	double MinVisibleTimeRange = FFrameNumber(1) / GetTickResolution();
	TRange<double> ExistingViewRange = TimeSliderArgs.ViewRange.Get();

	if (NewRangeMax == ExistingViewRange.GetUpperBoundValue())
	{
		if (NewRangeMin > NewRangeMax - MinVisibleTimeRange)
		{
			NewRangeMin = NewRangeMax - MinVisibleTimeRange;
		}
	}
	else if (NewRangeMax < NewRangeMin + MinVisibleTimeRange)
	{
		NewRangeMax = NewRangeMin + MinVisibleTimeRange;
	}

	// Clamp to the clamp range
	const TRange<double> NewRange = TRange<double>(NewRangeMin, NewRangeMax);
	TimeSliderArgs.OnViewRangeChanged.ExecuteIfBound(NewRange, Interpolation);

	if (!TimeSliderArgs.ViewRange.IsBound())
	{
		// The  output is not bound to a delegate so we'll manage the value ourselves (no animation)
		TimeSliderArgs.ViewRange.Set(NewRange);
	}
}

void FDataflowTimeSliderController::SetClampRange(double NewRangeMin, double NewRangeMax)
{
	const TRange<double> NewRange(NewRangeMin, NewRangeMax);

	TimeSliderArgs.OnClampRangeChanged.ExecuteIfBound(NewRange);

	if (!TimeSliderArgs.ClampRange.IsBound())
	{
		// The  output is not bound to a delegate so we'll manage the value ourselves (no animation)
		TimeSliderArgs.ClampRange.Set(NewRange);
	}
}

void FDataflowTimeSliderController::SetPlayRange(FFrameNumber RangeStart, int32 RangeDuration)
{
	check(RangeDuration >= 0);

	const TRange<FFrameNumber> NewRange(RangeStart, RangeStart + RangeDuration);

	TimeSliderArgs.OnPlaybackRangeChanged.ExecuteIfBound(NewRange);

	if (!TimeSliderArgs.PlaybackRange.IsBound())
	{
		// The  output is not bound to a delegate so we'll manage the value ourselves (no animation)
		TimeSliderArgs.PlaybackRange.Set(NewRange);
	}
}

void FDataflowTimeSliderController::ClampViewRange(double& NewRangeMin, double& NewRangeMax)
{
	bool bNeedsClampSet = false;
	double NewClampRangeMin = TimeSliderArgs.ClampRange.Get().GetLowerBoundValue();
	if (NewRangeMin < TimeSliderArgs.ClampRange.Get().GetLowerBoundValue())
	{
		NewClampRangeMin = NewRangeMin;
		bNeedsClampSet = true;
	}

	double NewClampRangeMax = TimeSliderArgs.ClampRange.Get().GetUpperBoundValue();
	if (NewRangeMax > TimeSliderArgs.ClampRange.Get().GetUpperBoundValue())
	{
		NewClampRangeMax = NewRangeMax;
		bNeedsClampSet = true;
	}

	if (bNeedsClampSet)
	{
		SetClampRange(NewClampRangeMin, NewClampRangeMax);
	}
}

bool FDataflowTimeSliderController::ZoomByDelta(float InDelta, float MousePositionFraction)
{
	TRange<double> LocalViewRange = TimeSliderArgs.ViewRange.Get().GetAnimationTarget();
	double LocalViewRangeMax = LocalViewRange.GetUpperBoundValue();
	double LocalViewRangeMin = LocalViewRange.GetLowerBoundValue();
	const double OutputViewSize = LocalViewRangeMax - LocalViewRangeMin;
	const double OutputChange = OutputViewSize * InDelta;

	double NewViewOutputMin = LocalViewRangeMin - (OutputChange * MousePositionFraction);
	double NewViewOutputMax = LocalViewRangeMax + (OutputChange * (1.f - MousePositionFraction));

	if (NewViewOutputMin < NewViewOutputMax)
	{
		ClampViewRange(NewViewOutputMin, NewViewOutputMax);
		SetViewRange(NewViewOutputMin, NewViewOutputMax, EViewRangeInterpolation::Animated);
		return true;
	}

	return false;
}

void FDataflowTimeSliderController::PanByDelta(float InDelta)
{
	const TRange<double> LocalViewRange = TimeSliderArgs.ViewRange.Get().GetAnimationTarget();

	const double CurrentMin = LocalViewRange.GetLowerBoundValue();
	const double CurrentMax = LocalViewRange.GetUpperBoundValue();

	// Adjust the delta to be a percentage of the current range
	InDelta *= ScrubConstants::ScrollPanFraction * static_cast<float>(CurrentMax - CurrentMin);

	double NewViewOutputMin = CurrentMin + InDelta;
	double NewViewOutputMax = CurrentMax + InDelta;

	ClampViewRange(NewViewOutputMin, NewViewOutputMax);
	SetViewRange(NewViewOutputMin, NewViewOutputMax, EViewRangeInterpolation::Animated);
}

FFrameTime FDataflowTimeSliderController::GetFrameTimeFromMouse(const FGeometry& Geometry, FVector2D ScreenSpacePosition) const
{
	FScrubRangeToScreen ScrubRangeToScreen(TimeSliderArgs.ViewRange.Get(), Geometry.Size);
	return ComputeFrameTimeFromMouse(Geometry, ScreenSpacePosition, ScrubRangeToScreen);
}

void FDataflowTimeSliderController::CommitScrubPosition(FFrameTime NewValue, bool bIsScrubbing)
{
	// Clamp the scrub position to the visible view range
	const TRange<double> ViewRange = TimeSliderArgs.ViewRange.Get();
	if (ViewRange.HasLowerBound() && ViewRange.HasUpperBound())
	{
		const FFrameRate TickResolution = GetTickResolution();
		const FFrameTime LowerBound = ViewRange.GetLowerBoundValue() * TickResolution;
		const FFrameTime UpperBound = ViewRange.GetUpperBoundValue() * TickResolution;
		NewValue = FMath::Clamp(NewValue, LowerBound, UpperBound);
	}

	// Manage the scrub position ourselves if its not bound to a delegate
	if (!TimeSliderArgs.ScrubPosition.IsBound())
	{
		TimeSliderArgs.ScrubPosition.Set(NewValue);
	}

	TimeSliderArgs.OnScrubPositionChanged.ExecuteIfBound(NewValue, bIsScrubbing, /*bEvaluate*/ true); 
}

void FDataflowTimeSliderController::DrawTicks(FSlateWindowElementList& OutDrawElements, const TRange<double>& ViewRange, const FScrubRangeToScreen& RangeToScreen, FDrawTickArgs& InArgs) const
{
	TSharedPtr<SDataflowSimulationTimeline> Timeline = SimulationTimeline.Pin();
	if (!Timeline.IsValid())
	{
		return;
	}

	if (!FMath::IsFinite(ViewRange.GetLowerBoundValue()) || !FMath::IsFinite(ViewRange.GetUpperBoundValue()))
	{
		return;
	}

	FFrameRate     FrameResolution = GetTickResolution();
	FPaintGeometry PaintGeometry = InArgs.AllottedGeometry.ToPaintGeometry();
	FSlateFontInfo SmallLayoutFont = FCoreStyle::GetDefaultFontStyle("Regular", 8);

	double MajorGridStep = 0.0;
	int32  MinorDivisions = 0;
	if (!Timeline->GetGridMetrics(static_cast<float>(InArgs.AllottedGeometry.Size.X), MajorGridStep, MinorDivisions))
	{
		return;
	}

	if (InArgs.bOnlyDrawMajorTicks)
	{
		MinorDivisions = 0;
	}

	TArray<FVector2D> LinePoints;
	LinePoints.SetNumUninitialized(2);

	const bool bAntiAliasLines = false;

	const double FirstMajorLine = FMath::FloorToDouble(ViewRange.GetLowerBoundValue() / MajorGridStep) * MajorGridStep;
	const double LastMajorLine = FMath::CeilToDouble(ViewRange.GetUpperBoundValue() / MajorGridStep) * MajorGridStep;

	for (double CurrentMajorLine = FirstMajorLine; CurrentMajorLine < LastMajorLine; CurrentMajorLine += MajorGridStep)
	{
		float MajorLinePx = RangeToScreen.InputToLocalX(CurrentMajorLine);

		LinePoints[0] = FVector2D(MajorLinePx, InArgs.TickOffset);
		LinePoints[1] = FVector2D(MajorLinePx, InArgs.TickOffset + InArgs.MajorTickHeight);

		// Draw each tick mark
		FSlateDrawElement::MakeLines(
			OutDrawElements,
			InArgs.StartLayer,
			PaintGeometry,
			LinePoints,
			InArgs.DrawEffects,
			InArgs.TickColor,
			bAntiAliasLines
		);

		if (!InArgs.bOnlyDrawMajorTicks)
		{
			FString FrameString = TimeSliderArgs.NumericTypeInterface->ToString((CurrentMajorLine * FrameResolution).RoundToFrame().Value);

			// Space the text between the tick mark but slightly above
			FVector2D TextOffset(MajorLinePx + 5.f, InArgs.bMirrorLabels ? 3.f : FMath::Abs(InArgs.AllottedGeometry.Size.Y - (InArgs.MajorTickHeight + 3.f)));
			FSlateDrawElement::MakeText(
				OutDrawElements,
				InArgs.StartLayer + 1,
				InArgs.AllottedGeometry.ToPaintGeometry(InArgs.AllottedGeometry.Size, FSlateLayoutTransform(TextOffset)),
				FrameString,
				SmallLayoutFont,
				InArgs.DrawEffects,
				InArgs.TickColor * 0.65f
			);
		}

		for (int32 Step = 1; Step < MinorDivisions; ++Step)
		{
			// Compute the size of each tick mark.  If we are half way between to visible values display a slightly larger tick mark
			const float MinorTickHeight = ((MinorDivisions % 2 == 0) && (Step % (MinorDivisions / 2)) == 0) ? 6.0f : 2.0f;
			const float MinorLinePx = RangeToScreen.InputToLocalX(CurrentMajorLine + Step * MajorGridStep / MinorDivisions);

			LinePoints[0] = FVector2D(MinorLinePx, InArgs.bMirrorLabels ? 0.0f : FMath::Abs(InArgs.AllottedGeometry.Size.Y - MinorTickHeight));
			LinePoints[1] = FVector2D(MinorLinePx, LinePoints[0].Y + MinorTickHeight);

			// Draw each sub mark
			FSlateDrawElement::MakeLines(
				OutDrawElements,
				InArgs.StartLayer,
				PaintGeometry,
				LinePoints,
				InArgs.DrawEffects,
				InArgs.TickColor,
				bAntiAliasLines
			);
		}
	}
}

int32 FDataflowTimeSliderController::DrawSelectionRange(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FScrubRangeToScreen& RangeToScreen, const FPaintPlaybackRangeArgs& Args) const
{
	TRange<double> SelectionRange = TimeSliderArgs.SelectionRange.Get() / GetTickResolution();

	if (!SelectionRange.IsEmpty() && SelectionRange.HasLowerBound() && SelectionRange.HasUpperBound())
	{
		const float SelectionRangeL = RangeToScreen.InputToLocalX(SelectionRange.GetLowerBoundValue()) - 1;
		const float SelectionRangeR = RangeToScreen.InputToLocalX(SelectionRange.GetUpperBoundValue()) + 1;
		const FLinearColor DrawColor = FAppStyle::GetSlateColor("SelectionColor").GetColor(FWidgetStyle());

		if (Args.SolidFillOpacity > 0.f)
		{
			FSlateDrawElement::MakeBox(
				OutDrawElements,
				LayerId + 1,
				AllottedGeometry.ToPaintGeometry(FVector2f(SelectionRangeR - SelectionRangeL, AllottedGeometry.Size.Y), FSlateLayoutTransform(FVector2D(SelectionRangeL, 0.f))),
				FAppStyle::GetBrush("WhiteBrush"),
				ESlateDrawEffect::None,
				DrawColor.CopyWithNewOpacity(Args.SolidFillOpacity)
			);
		}

		FSlateDrawElement::MakeBox(
			OutDrawElements,
			LayerId + 1,
			AllottedGeometry.ToPaintGeometry(FVector2f(Args.BrushWidth, AllottedGeometry.Size.Y), FSlateLayoutTransform(FVector2D(SelectionRangeL, 0.f))),
			Args.StartBrush,
			ESlateDrawEffect::None,
			DrawColor
		);

		FSlateDrawElement::MakeBox(
			OutDrawElements,
			LayerId + 1,
			AllottedGeometry.ToPaintGeometry(FVector2f(Args.BrushWidth, AllottedGeometry.Size.Y), FSlateLayoutTransform(FVector2D(SelectionRangeR - Args.BrushWidth, 0.f))),
			Args.EndBrush,
			ESlateDrawEffect::None,
			DrawColor
		);
	}

	return LayerId + 1;
}

int32 FDataflowTimeSliderController::DrawPlaybackRange(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FScrubRangeToScreen& RangeToScreen, const FPaintPlaybackRangeArgs& Args) const
{
	if (!TimeSliderArgs.PlaybackRange.IsSet())
	{
		return LayerId;
	}

	const uint8 OpacityBlend = TimeSliderArgs.SubSequenceRange.Get().IsSet() ? 128 : 255;

	TRange<FFrameNumber> PlaybackRange = TimeSliderArgs.PlaybackRange.Get();
	FFrameRate TickResolution = GetTickResolution();
	const float PlaybackRangeL = RangeToScreen.InputToLocalX(PlaybackRange.GetLowerBoundValue() / TickResolution);
	const float PlaybackRangeR = RangeToScreen.InputToLocalX(PlaybackRange.GetUpperBoundValue() / TickResolution) - 1;

	FSlateDrawElement::MakeBox(
		OutDrawElements,
		LayerId + 1,
		AllottedGeometry.ToPaintGeometry(FVector2f(Args.BrushWidth, AllottedGeometry.Size.Y), FSlateLayoutTransform(FVector2D(PlaybackRangeL, 0.f))),
		Args.StartBrush,
		ESlateDrawEffect::None,
		FColor(32, 128, 32, OpacityBlend)	// 120, 75, 50 (HSV)
	);

	FSlateDrawElement::MakeBox(
		OutDrawElements,
		LayerId + 1,
		AllottedGeometry.ToPaintGeometry(FVector2f(Args.BrushWidth, AllottedGeometry.Size.Y), FSlateLayoutTransform(FVector2D(PlaybackRangeR - Args.BrushWidth, 0.f))),
		Args.EndBrush,
		ESlateDrawEffect::None,
		FColor(128, 32, 32, OpacityBlend)	// 0, 75, 50 (HSV)
	);

	// Black tint for excluded regions
	FSlateDrawElement::MakeBox(
		OutDrawElements,
		LayerId + 1,
		AllottedGeometry.ToPaintGeometry(FVector2f(PlaybackRangeL, AllottedGeometry.Size.Y), FSlateLayoutTransform(FVector2D(0.f, 0.f))),
		FAppStyle::GetBrush("WhiteBrush"),
		ESlateDrawEffect::None,
		FLinearColor::Black.CopyWithNewOpacity(0.3f * OpacityBlend / 255.f)
	);

	FSlateDrawElement::MakeBox(
		OutDrawElements,
		LayerId + 1,
		AllottedGeometry.ToPaintGeometry(FVector2f(AllottedGeometry.Size.X - PlaybackRangeR, AllottedGeometry.Size.Y), FSlateLayoutTransform(FVector2D(PlaybackRangeR, 0.f))),
		FAppStyle::GetBrush("WhiteBrush"),
		ESlateDrawEffect::None,
		FLinearColor::Black.CopyWithNewOpacity(0.3f * OpacityBlend / 255.f)
	);

	return LayerId + 1;
}

bool FDataflowTimeSliderController::HitTestRangeStart(const FScrubRangeToScreen& RangeToScreen, const TRange<double>& Range, float HitPixel) const
{
	if (Range.HasLowerBound())
	{
		static float BrushSizeInStateUnits = 6.f, DragToleranceSlateUnits = 2.f, MouseTolerance = 2.f;
		const float  RangeStartPixel = RangeToScreen.InputToLocalX(Range.GetLowerBoundValue());

		// Hit test against the brush region to the right of the playback start position, +/- DragToleranceSlateUnits
		return HitPixel >= RangeStartPixel - MouseTolerance - DragToleranceSlateUnits &&
			HitPixel <= RangeStartPixel + MouseTolerance + BrushSizeInStateUnits + DragToleranceSlateUnits;
	}

	return false;
}

bool FDataflowTimeSliderController::HitTestRangeEnd(const FScrubRangeToScreen& RangeToScreen, const TRange<double>& Range, float HitPixel) const
{
	if (Range.HasUpperBound())
	{
		static float BrushSizeInStateUnits = 6.f, DragToleranceSlateUnits = 2.f, MouseTolerance = 2.f;
		const float  RangeEndPixel = RangeToScreen.InputToLocalX(Range.GetUpperBoundValue());

		// Hit test against the brush region to the left of the playback end position, +/- DragToleranceSlateUnits
		return HitPixel >= RangeEndPixel - MouseTolerance - BrushSizeInStateUnits - DragToleranceSlateUnits &&
			HitPixel <= RangeEndPixel + MouseTolerance + DragToleranceSlateUnits;
	}

	return false;
}

void FDataflowTimeSliderController::SetPlaybackRangeStart(FFrameNumber NewStart)
{
	TRange<FFrameNumber> PlaybackRange = TimeSliderArgs.PlaybackRange.Get();

	if (NewStart <= UE::MovieScene::DiscreteExclusiveUpper(PlaybackRange))
	{
		TimeSliderArgs.OnPlaybackRangeChanged.ExecuteIfBound(TRange<FFrameNumber>(NewStart, PlaybackRange.GetUpperBound()));
	}
}

void FDataflowTimeSliderController::SetPlaybackRangeEnd(FFrameNumber NewEnd)
{
	TRange<FFrameNumber> PlaybackRange = TimeSliderArgs.PlaybackRange.Get();

	if (NewEnd >= UE::MovieScene::DiscreteInclusiveLower(PlaybackRange))
	{
		TimeSliderArgs.OnPlaybackRangeChanged.ExecuteIfBound(TRange<FFrameNumber>(PlaybackRange.GetLowerBound(), NewEnd));
	}
}

void FDataflowTimeSliderController::SetSelectionRangeStart(FFrameNumber NewStart)
{
	TRange<FFrameNumber> SelectionRange = TimeSliderArgs.SelectionRange.Get();

	if (SelectionRange.IsEmpty())
	{
		TimeSliderArgs.OnSelectionRangeChanged.ExecuteIfBound(TRange<FFrameNumber>(NewStart, NewStart + 1));
	}
	else if (NewStart <= UE::MovieScene::DiscreteExclusiveUpper(SelectionRange))
	{
		TimeSliderArgs.OnSelectionRangeChanged.ExecuteIfBound(TRange<FFrameNumber>(NewStart, SelectionRange.GetUpperBound()));
	}
}

void FDataflowTimeSliderController::SetSelectionRangeEnd(FFrameNumber NewEnd)
{
	TRange<FFrameNumber> SelectionRange = TimeSliderArgs.SelectionRange.Get();

	if (SelectionRange.IsEmpty())
	{
		TimeSliderArgs.OnSelectionRangeChanged.ExecuteIfBound(TRange<FFrameNumber>(NewEnd - 1, NewEnd));
	}
	else if (NewEnd >= UE::MovieScene::DiscreteInclusiveLower(SelectionRange))
	{
		TimeSliderArgs.OnSelectionRangeChanged.ExecuteIfBound(TRange<FFrameNumber>(SelectionRange.GetLowerBound(), NewEnd));
	}
}

FFrameTime FDataflowTimeSliderController::ComputeFrameTimeFromMouse(const FGeometry& Geometry, FVector2D ScreenSpacePosition, FScrubRangeToScreen RangeToScreen, bool CheckSnapping) const
{
	const FVector2D CursorPos = Geometry.AbsoluteToLocal(ScreenSpacePosition);
	const double    MouseValue = RangeToScreen.LocalXToInput(static_cast<float>(CursorPos.X));

	return MouseValue * GetTickResolution();
}

FDataflowTimeSliderController::FScrubPixelRange FDataflowTimeSliderController::GetHitTestScrubberPixelRange(FFrameTime ScrubTime, const FScrubRangeToScreen& RangeToScreen) const
{
	constexpr float DragToleranceSlateUnits = 2.f, MouseTolerance = 2.f;
	return GetScrubberPixelRange(ScrubTime, GetTickResolution(), GetDisplayRate(), RangeToScreen, DragToleranceSlateUnits + MouseTolerance);
}

FDataflowTimeSliderController::FScrubPixelRange FDataflowTimeSliderController::GetScrubberPixelRange(FFrameTime ScrubTime, const FScrubRangeToScreen& RangeToScreen) const
{
	return GetScrubberPixelRange(ScrubTime, GetTickResolution(), GetDisplayRate(), RangeToScreen);
}

FDataflowTimeSliderController::FScrubPixelRange FDataflowTimeSliderController::GetScrubberPixelRange(FFrameTime ScrubTime, FFrameRate Resolution, FFrameRate PlayRate, const FScrubRangeToScreen& RangeToScreen, float DilationPixels) const
{
	const FFrameNumber Frame = ScrubTime.FloorToFrame();// ConvertFrameTime(ScrubTime, Resolution, PlayRate).FloorToFrame();

	float StartPixel = RangeToScreen.InputToLocalX(Frame / Resolution);
	float EndPixel = RangeToScreen.InputToLocalX((Frame + 1) / Resolution);

	{
		const float RoundedStartPixel = static_cast<float>(FMath::RoundToInt(StartPixel));
		EndPixel -= (StartPixel - RoundedStartPixel);

		StartPixel = RoundedStartPixel;
		EndPixel = FMath::Max(EndPixel, StartPixel + 1);
	}

	FScrubPixelRange Range;

	float MinScrubSize = 14.f;
	Range.bClamped = EndPixel - StartPixel < MinScrubSize;
	Range.Range = TRange<float>(StartPixel, EndPixel);
	if (Range.bClamped)
	{
		Range.HandleRange = TRange<float>(
			(StartPixel + EndPixel - MinScrubSize) * .5f,
			(StartPixel + EndPixel + MinScrubSize) * .5f);
	}
	else
	{
		Range.HandleRange = Range.Range;
	}

	return Range;
}