// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "ITimeSlider.h"
#include "TimeSliderArgs.h"
#include "Input/CursorReply.h"
#include "Input/Reply.h"

struct FSlateBrush;
class SDataflowSimulationTimeline;

class FDataflowTimeSliderController : public ITimeSliderController, public TSharedFromThis<FDataflowTimeSliderController>
{
	struct FDrawTickArgs;
	struct FScrubRangeToScreen;

public:
	FDataflowTimeSliderController(const FTimeSliderArgs& InArgs, TWeakPtr<SDataflowSimulationTimeline> DataflowSimulationTimeline);
	~FDataflowTimeSliderController();

	//~ Begin ITimeSliderController Interface 
	virtual int32 OnPaintTimeSlider(bool bMirrorLabels, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	virtual int32 OnPaintViewArea(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, bool bEnabled, const FPaintViewAreaArgs& Args) const override;
	virtual FReply OnMouseButtonDown(SWidget& WidgetOwner, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonUp(SWidget& WidgetOwner, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonDoubleClick(SWidget& OwnerWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseMove(SWidget& WidgetOwner, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseWheel(SWidget& WidgetOwner, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FCursorReply OnCursorQuery(const SWidget& WidgetOwner, const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const override;
	virtual void SetViewRange(double NewRangeMin, double NewRangeMax, EViewRangeInterpolation Interpolation) override;
	virtual void SetClampRange(double NewRangeMin, double NewRangeMax) override;
	virtual void SetPlayRange(FFrameNumber RangeStart, int32 RangeDuration) override;
	virtual FFrameRate GetDisplayRate() const override { return TimeSliderArgs.DisplayRate.Get(); }
	virtual FFrameRate GetTickResolution() const override { return TimeSliderArgs.TickResolution.Get(); }
	virtual FAnimatedRange GetViewRange() const override { return TimeSliderArgs.ViewRange.Get(); }
	virtual FAnimatedRange GetClampRange() const override { return TimeSliderArgs.ClampRange.Get(); }
	virtual TRange<FFrameNumber> GetPlayRange() const override { return TimeSliderArgs.PlaybackRange.Get(TRange<FFrameNumber>()); }
	virtual FFrameTime GetScrubPosition() const { return TimeSliderArgs.ScrubPosition.Get(); }
	virtual void SetScrubPosition(FFrameTime InTime, bool bEvaluate) { TimeSliderArgs.OnScrubPositionChanged.ExecuteIfBound(InTime, false, bEvaluate); }
	//~ End ITimeSliderController Interface 

	/**
	 * Clamp the given range to the clamp range
	 *
	 * @param NewRangeMin		The new lower bound of the range
	 * @param NewRangeMax		The new upper bound of the range
	 */
	void ClampViewRange(double& NewRangeMin, double& NewRangeMax);

	/**
	 * Zoom the range by a given delta.
	 *
	 * @param InDelta					The total amount to zoom by (+ve = zoom out, -ve = zoom in)
	 * @param MousePositionFraction		Bias to apply to lower/upper extents of the range. (0 = lower, 0.5 = equal, 1 = upper)
	 */
	bool ZoomByDelta(float InDelta, float MousePositionFraction = 0.5f);

	/**
	 * Pan the range by a given delta
	 *
	 * @param InDelta		The total amount to pan by (+ve = pan forwards in time, -ve = pan backwards in time)
	 */
	void PanByDelta(float InDelta);

	/** Determine frame time from a mouse position */
	FFrameTime GetFrameTimeFromMouse(const FGeometry& Geometry, FVector2D ScreenSpacePosition) const;

private:
	/**
	 * Call this method when the user's interaction has changed the scrub position
	 *
	 * @param NewValue				Value resulting from the user's interaction
	 * @param bIsScrubbing			True if done via scrubbing, false if just releasing scrubbing
	 */
	void CommitScrubPosition(FFrameTime NewValue, bool bIsScrubbing);

	/**
	 * Draw time tick marks
	 *
	 * @param OutDrawElements	List to add draw elements to
	 * @param ViewRange			The currently visible time range in seconds
	 * @param RangeToScreen		Time range to screen space converter
	 * @param InArgs			Parameters for drawing the tick lines
	 */
	void DrawTicks(FSlateWindowElementList& OutDrawElements, const TRange<double>& ViewRange, const FScrubRangeToScreen& RangeToScreen, FDrawTickArgs& InArgs) const;

	/**
	 * Draw the selection range.
	 *
	 * @return The new layer ID.
	 */
	int32 DrawSelectionRange(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FScrubRangeToScreen& RangeToScreen, const FPaintPlaybackRangeArgs& Args) const;

	/**
	 * Draw the playback range.
	 *
	 * @return the new layer ID
	 */
	int32 DrawPlaybackRange(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FScrubRangeToScreen& RangeToScreen, const FPaintPlaybackRangeArgs& Args) const;

	/** Hit test the lower bound of a range */
	bool HitTestRangeStart(const FScrubRangeToScreen& RangeToScreen, const TRange<double>& Range, float HitPixel) const;

	/** Hit test the upper bound of a range */
	bool HitTestRangeEnd(const FScrubRangeToScreen& RangeToScreen, const TRange<double>& Range, float HitPixel) const;

	/** Set the playback range start */
	void SetPlaybackRangeStart(FFrameNumber NewStart);

	/** Set the playback range end */
	void SetPlaybackRangeEnd(FFrameNumber NewEnd);

	/** Set the selection range start */
	void SetSelectionRangeStart(FFrameNumber NewStart);

	/** Set the selection range end */
	void SetSelectionRangeEnd(FFrameNumber NewEnd);

	FFrameTime ComputeFrameTimeFromMouse(const FGeometry& Geometry, FVector2D ScreenSpacePosition, FScrubRangeToScreen RangeToScreen, bool CheckSnapping = true) const;

private:
	struct FScrubPixelRange
	{
		TRange<float> Range;
		TRange<float> HandleRange;
		bool bClamped;
	};

	FScrubPixelRange GetHitTestScrubberPixelRange(FFrameTime ScrubTime, const FScrubRangeToScreen& RangeToScreen) const;
	FScrubPixelRange GetScrubberPixelRange(FFrameTime ScrubTime, const FScrubRangeToScreen& RangeToScreen) const;
	FScrubPixelRange GetScrubberPixelRange(FFrameTime ScrubTime, FFrameRate Resolution, FFrameRate PlayRate, const FScrubRangeToScreen& RangeToScreen, float DilationPixels = 0.f) const;

	/** Time Slider args */
	FTimeSliderArgs TimeSliderArgs;

	/** Pointer back to the timeline */
	TWeakPtr<SDataflowSimulationTimeline> SimulationTimeline;

	/** Brush for drawing the fill area on the scrubber */
	const FSlateBrush* ScrubFillBrush;

	/** Brush for drawing an upwards facing scrub handles */
	const FSlateBrush* ScrubHandleUpBrush;

	/** Brush for drawing a downwards facing scrub handle */
	const FSlateBrush* ScrubHandleDownBrush;

	/** Brush for drawing an editable time */
	const FSlateBrush* EditableTimeBrush;

	// Mouse Controls
	/** Total mouse delta during dragging **/
	float DistanceDragged;

	/** If we are dragging a scrubber or dragging to set the time range */
	enum DragType
	{
		DRAG_SCRUBBING_TIME,
		DRAG_SETTING_RANGE,
		DRAG_PLAYBACK_START,
		DRAG_PLAYBACK_END,
		DRAG_SELECTION_START,
		DRAG_SELECTION_END,
		DRAG_TIME,
		DRAG_NONE
	};

	DragType MouseDragType;

	/** If we are currently panning the panel */
	bool bPanning;

	/** Index of the current dragged time */
	int32 DraggedTimeIndex;

	/** Mouse down position range */
	FVector2D MouseDownPosition[2];

	/** Geometry on mouse down */
	FGeometry MouseDownGeometry;

	/** Range stack */
	TArray<TRange<double>> ViewRangeStack;

};