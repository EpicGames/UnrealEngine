// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ITimeSlider.h"
#include "AnimatedRange.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Input/NumericTypeInterface.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"

class FDataflowTimeSliderController;
class FDataflowSimulationBinding;

/** Main dataflow simulation timeline widget to be used in the dataflow toolkit */
class SDataflowSimulationTimeline : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SDataflowSimulationTimeline) { }
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<FDataflowSimulationBinding>& InSimulationBinbding);

	/** Get numeric Type interface for converting between frame numbers and display formats. */
	TSharedRef<INumericTypeInterface<double>> GetNumericTypeInterface() const { return NumericTypeInterface.ToSharedRef(); }

	/** Compute a major grid interval and number of minor divisions to display */
	bool GetGridMetrics(float PhysicalWidth, double& OutMajorInterval, int32& OutMinorDivisions) const;

private:
	/** Handle any changes to the scrub position */
	void HandleScrubPositionChanged(FFrameTime NewScrubPosition, bool bIsScrubbing, bool bEvaluate) const;

	void HandlePlaybackRangeChanged(TRange<FFrameNumber> NewRange);

	/** Init track names from the simulation binding */
	void InitTrackNames();

	/** Get the column coefficient */
	float GetColumnFillCoefficient(int32 ColumnIndex) const
	{
		return ColumnFillCoefficients[ColumnIndex];
	}
	/** callback when the column coefficient is changing */
	void OnColumnFillCoefficientChanged(float FillCoefficient, int32 ColumnIndex);

	/** Add a track row in the list viiew */
	TSharedRef<ITableRow> HandleTimelineListViewGenerateRow(TSharedPtr<FString> Text, const TSharedRef<STableViewBase>& OwnerTable);

	/** The fill coefficients of each column in the grid. */
	float ColumnFillCoefficients[2];

	/** Simulation binding to get the timing information from dataflow */
	TWeakPtr<FDataflowSimulationBinding> SimulationBinding;

	/** Interface to display the frame number */
	TSharedPtr<INumericTypeInterface<double>> NumericTypeInterface;

	/** Time slider controller used in the timeline */
	TSharedPtr<FDataflowTimeSliderController> TimeSliderController;

	/** Time Slider used in the time line */
	TSharedPtr<ITimeSlider> TimeSlider;

	/** List of track names holding the caches */
	TArray<TSharedPtr<FString>> TrackNames;

	/** List of filtered tracks holding the caches */
	TArray<TSharedPtr<FString>> FilteredTracks;

	/** Track list view in betwwen the timeline and the transport controls */
	TSharedPtr<SListView<TSharedPtr<FString>>> TracksListView;

	/** Animation view range*/
	TAttribute<FAnimatedRange> ViewRange;
};