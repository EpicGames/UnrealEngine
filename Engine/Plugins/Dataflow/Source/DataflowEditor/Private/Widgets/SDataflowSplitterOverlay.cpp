// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDataflowSplitterOverlay.h"

#include "Delegates/Delegate.h"
#include "HAL/PlatformCrt.h"
#include "Layout/ArrangedChildren.h"
#include "Layout/ArrangedWidget.h"
#include "Layout/ChildrenBase.h"
#include "Layout/Geometry.h"
#include "Layout/Visibility.h"
#include "Math/Vector2D.h"
#include "Misc/Attribute.h"
#include "SlotBase.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Layout/SBox.h"

struct FCaptureLostEvent;
struct FPointerEvent;

void SDataflowSplitterOverlay::Construct( const FArguments& InArgs )
{
	SetVisibility(EVisibility::SelfHitTestInvisible);

	SplitterWidget = SArgumentNew(InArgs, SSplitter);
	SplitterWidget->SetVisibility(EVisibility::HitTestInvisible);
	AddSlot()
	[
		SplitterWidget.ToSharedRef()
	];

	for (int32 Index = 0; Index < SplitterWidget->GetChildren()->Num() - 1; ++Index)
	{
		AddSlot()
		.Padding(TAttribute<FMargin>::Create(TAttribute<FMargin>::FGetter::CreateSP(this, &SDataflowSplitterOverlay::GetSplitterHandlePadding, Index)))
		[
			SNew(SBox)
			.Visibility(EVisibility::Visible)
		];
	}
}

void SDataflowSplitterOverlay::OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const
{
	FArrangedChildren SplitterChildren(ArrangedChildren.GetFilter());
	SplitterWidget->ArrangeChildren(AllottedGeometry, SplitterChildren);

	SlotPadding.Reset();

	for (int32 Index = 0; Index < SplitterChildren.Num() - 1; ++Index)
	{
		const FGeometry& ThisGeometry = SplitterChildren[Index].Geometry;
		const FGeometry& NextGeometry = SplitterChildren[Index + 1].Geometry;

		if (SplitterWidget->GetOrientation() == EOrientation::Orient_Horizontal)
		{
			SlotPadding.Add(FMargin(
				ThisGeometry.Position.X + static_cast<float>(ThisGeometry.GetLocalSize().X),
				0,
				static_cast<float>(AllottedGeometry.Size.X) - NextGeometry.Position.X,
				0)
			);
		}
		else
		{
			SlotPadding.Add(FMargin(
				0,
				ThisGeometry.Position.Y + static_cast<float>(ThisGeometry.GetLocalSize().Y),
				0,
				static_cast<float>(AllottedGeometry.Size.Y) - NextGeometry.Position.Y)
			);
		}
	}

	SOverlay::OnArrangeChildren(AllottedGeometry, ArrangedChildren);
}

FMargin SDataflowSplitterOverlay::GetSplitterHandlePadding(int32 Index) const
{
	if (SlotPadding.IsValidIndex(Index))
	{
		return SlotPadding[Index];
	}

	return 0.f;
}
	
FCursorReply SDataflowSplitterOverlay::OnCursorQuery( const FGeometry& MyGeometry, const FPointerEvent& CursorEvent ) const
{
	return SplitterWidget->OnCursorQuery(MyGeometry, CursorEvent);
}

FReply SDataflowSplitterOverlay::OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	FReply Reply = SplitterWidget->OnMouseButtonDown(MyGeometry, MouseEvent);
	if (Reply.GetMouseCaptor().IsValid())
	{
		// Set us to be the mouse captor so we can forward events properly
		Reply.CaptureMouse( SharedThis(this) );
		SetVisibility(EVisibility::Visible);
	}
	return Reply;
}
	
void SDataflowSplitterOverlay::OnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent)
{
	SetVisibility(EVisibility::SelfHitTestInvisible);
	SOverlay::OnMouseCaptureLost(CaptureLostEvent);
}

FReply SDataflowSplitterOverlay::OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	FReply Reply = SplitterWidget->OnMouseButtonUp(MyGeometry, MouseEvent);
	if (Reply.ShouldReleaseMouse())
	{
		SetVisibility(EVisibility::SelfHitTestInvisible);
	}
	return Reply;
}

FReply SDataflowSplitterOverlay::OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	return SplitterWidget->OnMouseMove(MyGeometry, MouseEvent);
}

void SDataflowSplitterOverlay::OnMouseLeave( const FPointerEvent& MouseEvent )
{
	return SplitterWidget->OnMouseLeave(MouseEvent);
}

