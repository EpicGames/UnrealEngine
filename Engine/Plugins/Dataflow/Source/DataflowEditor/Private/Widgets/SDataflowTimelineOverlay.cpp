// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDataflowTimelineOverlay.h"
#include "DataflowTimeSliderController.h"

int32 SDataflowTimelineOverlay::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry,
	const FSlateRect& CullingRect, FSlateWindowElementList& DrawElements, int32 LayerId, const FWidgetStyle& WidgetStyle, bool bParentEnabled) const
{
	FPaintViewAreaArgs PaintArgs;
	PaintArgs.bDisplayTickLines = bDisplayTickLines.Get();
	PaintArgs.bDisplayScrubPosition = bDisplayScrubPosition.Get();

	if (PaintPlaybackRangeArgs.IsSet())
	{
		PaintArgs.PlaybackRangeArgs = PaintPlaybackRangeArgs.Get();
	}

	if(TimeSliderController)
	{
		TimeSliderController->OnPaintViewArea(AllottedGeometry, CullingRect, DrawElements, LayerId, ShouldBeEnabled(bParentEnabled), PaintArgs);
	}

	return SCompoundWidget::OnPaint(Args, AllottedGeometry, CullingRect, DrawElements, LayerId, WidgetStyle, bParentEnabled);
}

