// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "ITimeSlider.h"
#include "Misc/Attribute.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FDataflowTimeSliderController;
class FPaintArgs;
class FSlateRect;
class FSlateWindowElementList;
class FWidgetStyle;
struct FGeometry;

/** An overlay that displays global information in the track area. */
class SDataflowTimelineOverlay : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SDataflowTimelineOverlay)
		: _DisplayTickLines(true)
		, _DisplayScrubPosition(false)
		{}

		SLATE_ATTRIBUTE(bool, DisplayTickLines)
		SLATE_ATTRIBUTE(bool, DisplayScrubPosition)
		SLATE_ATTRIBUTE(FPaintPlaybackRangeArgs, PaintPlaybackRangeArgs)
	SLATE_END_ARGS()

	/** Construct the widget given args and a slider controller */
	void Construct(const FArguments& InArgs, TSharedRef<FDataflowTimeSliderController> InTimeSliderController)
	{
		bDisplayScrubPosition = InArgs._DisplayScrubPosition;
		bDisplayTickLines = InArgs._DisplayTickLines;
		PaintPlaybackRangeArgs = InArgs._PaintPlaybackRangeArgs;
		TimeSliderController = InTimeSliderController;
	}

private:
	/** SWidget Interface */
	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	
	/** Controller for manipulating time */
	TSharedPtr<FDataflowTimeSliderController> TimeSliderController;
	
	/** Whether or not to display the scrub position */
	TAttribute<bool> bDisplayScrubPosition;
	
	/** Whether or not to display tick lines */
	TAttribute<bool> bDisplayTickLines;
	
	/** User-supplied options for drawing playback range */
	TAttribute<FPaintPlaybackRangeArgs> PaintPlaybackRangeArgs;
};