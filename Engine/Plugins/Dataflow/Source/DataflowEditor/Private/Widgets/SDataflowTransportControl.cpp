// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDataflowTransportControl.h"
#include "DataflowSimulationBinding.h"
#include "Dataflow/DataflowEditorStyle.h"
#include "Delegates/Delegate.h"
#include "Framework/SlateDelegates.h"
#include "Misc/EnumRange.h"
#include "SlateOptMacros.h"
#include "Editor/EditorWidgets/Public/EditorWidgetsModule.h"
#include "Editor/EditorWidgets/Public/ITransportControl.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/SBoxPanel.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "DataflowSimulationTransportControl"

void SDataflowTransportControl::Construct(const FArguments& InArgs, const TSharedRef<FDataflowSimulationBinding>& InBindingAsset)
{
	SimulationBinding = InBindingAsset;
	
	// Skip adding the the Loop button so we can add our own
	TArray<FTransportControlWidget> TransportControlWidgets;
	for (const ETransportControlWidgetType Type : TEnumRange<ETransportControlWidgetType>())
	{
		if ((Type != ETransportControlWidgetType::Custom) && (Type != ETransportControlWidgetType::Loop))
		{
			TransportControlWidgets.Add(FTransportControlWidget(Type));
		}
	}
	const FTransportControlWidget PreviewModeWidget(FOnMakeTransportWidget::CreateSP(this, &SDataflowTransportControl::OnCreateModeButton));
	TransportControlWidgets.Add(PreviewModeWidget);

	FTransportControlArgs TransportControlArgs;
	TransportControlArgs.OnForwardPlay = FOnClicked::CreateSP(this, &SDataflowTransportControl::OnForwardPlay);
	TransportControlArgs.OnBackwardPlay = FOnClicked::CreateSP(this, &SDataflowTransportControl::OnBackwardPlay);
	TransportControlArgs.OnForwardStep = FOnClicked::CreateSP(this, &SDataflowTransportControl::OnForwardStep);
	TransportControlArgs.OnBackwardStep = FOnClicked::CreateSP(this, &SDataflowTransportControl::OnBackwardStep);
	TransportControlArgs.OnForwardEnd = FOnClicked::CreateSP(this, &SDataflowTransportControl::OnForwardEnd);
	TransportControlArgs.OnBackwardEnd = FOnClicked::CreateSP(this, &SDataflowTransportControl::OnBackwardEnd);
	TransportControlArgs.OnRecord = FOnClicked::CreateSP(this, &SDataflowTransportControl::OnRecord);
	TransportControlArgs.OnTickPlayback = FOnTickPlayback::CreateSP(this, &SDataflowTransportControl::OnTickPlayback);
	TransportControlArgs.OnGetPlaybackMode = FOnGetPlaybackMode::CreateSP(this, &SDataflowTransportControl::GetPlaybackMode);
	TransportControlArgs.WidgetsToCreate = TransportControlWidgets;
	
	FEditorWidgetsModule& EditorWidgetsModule = FModuleManager::LoadModuleChecked<FEditorWidgetsModule>("EditorWidgets");

	TSharedPtr<SHorizontalBox> HorizontalBox;
	ChildSlot
	[
		SAssignNew(HorizontalBox, SHorizontalBox)
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Center)
		.FillWidth(1.0)
		[
			EditorWidgetsModule.CreateTransportControl(TransportControlArgs)
		]
	];
}

FReply SDataflowTransportControl::OnForwardStep()
{
	if (const TSharedPtr<FDataflowSimulationBinding> SharedBinding = SimulationBinding.Pin())
	{
		SharedBinding->SetScrubTime(SharedBinding->GetScrubTime() + SharedBinding->GetDeltaTime());
	}
	return FReply::Handled();
}

FReply SDataflowTransportControl::OnForwardEnd()
{
	if (const TSharedPtr<FDataflowSimulationBinding> SharedBinding = SimulationBinding.Pin())
	{
		SharedBinding->SetScrubTime(SharedBinding->GetSequenceLength());
	}
	return FReply::Handled();
}

FReply SDataflowTransportControl::OnBackwardStep()
{
	if (const TSharedPtr<FDataflowSimulationBinding> SharedBinding = SimulationBinding.Pin())
	{
		SharedBinding->SetScrubTime(SharedBinding->GetScrubTime() - SharedBinding->GetDeltaTime());
	}
	return FReply::Handled();
}

FReply SDataflowTransportControl::OnBackwardEnd()
{
	if (const TSharedPtr<FDataflowSimulationBinding> SharedBinding = SimulationBinding.Pin())
	{
		SharedBinding->SetScrubTime(0);
	}
	return FReply::Handled();
}

FReply SDataflowTransportControl::OnForwardPlay()
{
	if (PlaybackMode == EPlaybackMode::PlayingForward)
	{
		PlaybackMode = EPlaybackMode::Stopped;
	}
	else
	{
		PlaybackMode = EPlaybackMode::PlayingForward;
	}
	return FReply::Handled();
}

FReply SDataflowTransportControl::OnBackwardPlay()
{
	if(PlaybackMode == EPlaybackMode::PlayingReverse)
	{
		PlaybackMode = EPlaybackMode::Stopped;
	}
	else
	{
		PlaybackMode = EPlaybackMode::PlayingReverse;
	}

	return FReply::Handled();
}

FReply SDataflowTransportControl::OnPlaybackMode()
{
	if (PreviewMode == EPreviewMode::Default)
	{
		PreviewMode = EPreviewMode::Looping;
	}
	else if (PreviewMode == EPreviewMode::Looping)
	{
		PreviewMode = EPreviewMode::PingPong;
	}
	else
	{
		PreviewMode = EPreviewMode::Default;
	}

	return FReply::Handled();
}

void SDataflowTransportControl::OnTickPlayback(double InCurrentTime, float InDeltaTime)
{
	if (const TSharedPtr<FDataflowSimulationBinding> SharedBinding = SimulationBinding.Pin())
	{
		const float SequenceLength = SharedBinding->GetSequenceLength();
		const float ScrubTime = (PlaybackMode == EPlaybackMode::PlayingForward) ?
			SharedBinding->GetScrubTime() + InDeltaTime : SharedBinding->GetScrubTime() - InDeltaTime;
		
		if(PreviewMode == EPreviewMode::Looping)
		{
			SharedBinding->SetScrubTime(ScrubTime - SequenceLength * FMath::Floor(ScrubTime / SequenceLength));
		}
		else
		{
			if(PreviewMode == EPreviewMode::PingPong)
			{
				if((PlaybackMode == EPlaybackMode::PlayingForward) && (ScrubTime >= SequenceLength))
				{
					PlaybackMode = EPlaybackMode::PlayingReverse;
				}

				if((PlaybackMode == EPlaybackMode::PlayingReverse) && (ScrubTime <= 0))
				{
					PlaybackMode = EPlaybackMode::PlayingForward;
				}
			}
			SharedBinding->SetScrubTime(FMath::Clamp(ScrubTime, 0.f, SequenceLength));
		}
	}
}

FReply SDataflowTransportControl::OnRecord()
{
	if (const TSharedPtr<FDataflowSimulationBinding> SharedBinding = SimulationBinding.Pin())
	{
		SharedBinding->ResetSimulationScene();

		SharedBinding->RecordSimulationCache();
	}
	return FReply::Handled();
}

TSharedRef<SWidget> SDataflowTransportControl::OnCreateResetButton()
{
	if (const TSharedPtr<FDataflowSimulationBinding> SharedBinding = SimulationBinding.Pin())
	{
		ResetSimulationButton = SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "Animation.PlayControlsButton")
				.ToolTipText(FText::FromString("The button reset the dataflow simulation."))
				.OnClicked_Lambda([this, SharedBinding]()
					{
						SharedBinding->ResetSimulationScene();

						return FReply::Handled();
					})
				.ContentPadding(0.0f)
				.IsFocusable(true)
				.HAlign(HAlign_Center)
					[
						SNew(SImage)
							.ColorAndOpacity(FSlateColor::UseSubduedForeground())
							.Image(FDataflowEditorStyle::Get().GetBrush("Dataflow.ResetSimulation"))
					];
	}

	TSharedRef<SHorizontalBox> ResetSimulationWidget = SNew(SHorizontalBox);
	ResetSimulationWidget->AddSlot()
	.AutoWidth()
	[
		ResetSimulationButton.ToSharedRef()
	];

	return ResetSimulationWidget;
}

TSharedRef<SWidget> SDataflowTransportControl::OnCreateLockButton()
{
	if (const TSharedPtr<FDataflowSimulationBinding> SharedBinding = SimulationBinding.Pin())
	{
		LockSimulationBox = SNew(SCheckBox)
				.Style(FAppStyle::Get(), "ToggleButtonCheckbox")
				.ToolTipText(FText::FromString("The button locks the refresh of the values in the panel."))
				.IsChecked_Lambda([this, SharedBinding]()-> ECheckBoxState
					{
						if (SharedBinding->IsSimulationLocked())
						{
							return ECheckBoxState::Checked;
						}
						return ECheckBoxState::Unchecked;
					})
				.OnCheckStateChanged_Lambda([this, SharedBinding](ECheckBoxState State)
					{
						//const bool bIsPlaying = PlaybackMode != EPlaybackMode::Stopped;
						if(State == ECheckBoxState::Checked)
						{
							SharedBinding->SetSimulationLocked(true);
						}
						else
						{
							SharedBinding->SetSimulationLocked(false);
						}
					})
				.Padding(0.0f)
				.HAlign(HAlign_Center)
				[
					SNew(SImage)
						.ColorAndOpacity(FSlateColor::UseForeground())
						.Image(this, &SDataflowTransportControl::GetLockButtonImage)  
				];
	}
	TSharedRef<SHorizontalBox> LockSimulationWidget = SNew(SHorizontalBox);
	LockSimulationWidget->AddSlot()
	.AutoWidth()
	[
		LockSimulationBox.ToSharedRef()
	];

	return LockSimulationWidget;
}

TSharedRef<SWidget> SDataflowTransportControl::OnCreateModeButton()
{
	PreviewModeButton = SNew(SButton)
		.OnClicked(this, &SDataflowTransportControl::OnPlaybackMode)
		.ButtonStyle( FAppStyle::Get(), "Animation.PlayControlsButton" )
		.IsFocusable(false)
		.ToolTipText_Lambda([&]()
		{
			if (PreviewMode == EPreviewMode::Default)
			{
				return LOCTEXT("PlaybackModeDefaultTooltip", "Linear playback");
			}
			else if (PreviewMode == EPreviewMode::Looping)
			{
				return LOCTEXT("PlaybackModeLoopingTooltip", "Looping playback");
			}
			else
			{
				return LOCTEXT("PlaybackModePingPongTooltip", "Ping pong playback");
			}
		})
		.ContentPadding(0.0f);

	TWeakPtr<SButton> WeakButton = PreviewModeButton;

	PreviewModeButton->SetContent(SNew(SImage)
		.ColorAndOpacity(FSlateColor::UseSubduedForeground())
		.Image_Lambda([&, WeakButton]()
		{
			if (PreviewMode == EPreviewMode::Default)
			{
				return FAppStyle::Get().GetBrush("Animation.Loop.Disabled");
			}
			else if (PreviewMode == EPreviewMode::Looping)
			{
				return FAppStyle::Get().GetBrush("Animation.Loop.Enabled");
			}
			else
			{
				return FAppStyle::Get().GetBrush("Animation.Loop.SelectionRange");		// TODO: Replace with a back and forth type icon
			}
		})
	);

	TSharedRef<SHorizontalBox> PreviewPlaybackModeBox = SNew(SHorizontalBox);
	PreviewPlaybackModeBox->AddSlot()
	.AutoWidth()
	[
		PreviewModeButton.ToSharedRef()
	];

	return PreviewPlaybackModeBox;
}

const FSlateBrush* SDataflowTransportControl::GetLockButtonImage() const
{
	if (const TSharedPtr<FDataflowSimulationBinding> SharedBinding = SimulationBinding.Pin())
	{
		if (SharedBinding->IsSimulationLocked())
		{
			return FDataflowEditorStyle::Get().GetBrush("Dataflow.LockSimulation");
		}
		else
		{
			return FDataflowEditorStyle::Get().GetBrush("Dataflow.UnlockSimulation");
		}
	}
	return nullptr;
}

#undef LOCTEXT_NAMESPACE

