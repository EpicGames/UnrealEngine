// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Input/Reply.h"
#include "ITransportControl.h"
#include "Templates/SharedPointer.h"
#include "TickableEditorObject.h"
#include "Editor/EditorWidgets/Public/ITransportControl.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class FDataflowSimulationBinding;
class SButton;
class SCheckBox;

/** Dataflow simulation widget to control an animation/simulation */
class SDataflowTransportControl : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SDataflowTransportControl) {}
	SLATE_END_ARGS()

	/** Construct the widget given args and a simulation binding */
	void Construct(const FArguments& InArgs, const TSharedRef<FDataflowSimulationBinding>& InSimulationBinding);

private:

	/** Data flow preview mode */
	enum class EPreviewMode : uint8
	{
		Default,
		Looping,
		PingPong
	};

	/** Delegate when the simulation controls are pressed*/
	FReply OnForwardPlay();
	FReply OnBackwardPlay();
	FReply OnForwardStep();
	FReply OnBackwardStep();
	FReply OnForwardEnd();
	FReply OnBackwardEnd();
	FReply OnPlaybackMode();
	FReply OnRecord();
	void OnTickPlayback(double InCurrentTime, float InDeltaTime);

	/** Create preview mode widget */
	TSharedRef<SWidget> OnCreateModeButton();

	/** Create lock widget */
	TSharedRef<SWidget> OnCreateLockButton();

	/** Create reset widget */
	TSharedRef<SWidget> OnCreateResetButton();

	/** Get the lock button image */
	const FSlateBrush* GetLockButtonImage() const;

	/** Get the playback mode */
	EPlaybackMode::Type GetPlaybackMode() const {return PlaybackMode;}
	
	/** Simulation binding to extract dataflow information */
	TWeakPtr<FDataflowSimulationBinding> SimulationBinding;

	/** Preview mode (looping...) */
	EPreviewMode PreviewMode = EPreviewMode::Looping;

	/** Playback mode */
	EPlaybackMode::Type PlaybackMode = EPlaybackMode::Stopped;

	/** Playback mode button */
	TSharedPtr<SButton> PreviewModeButton;

	/** Widget showing if a simulation is locked or not */
	TSharedPtr<SCheckBox> LockSimulationBox;

	/** Widget to reset a simulation */
	TSharedPtr<SButton> ResetSimulationButton;
};