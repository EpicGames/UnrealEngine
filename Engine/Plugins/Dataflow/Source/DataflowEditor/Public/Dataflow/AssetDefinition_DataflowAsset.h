// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetDefinitionDefault.h"

#include "AssetDefinition_DataflowAsset.generated.h"

#define UE_API DATAFLOWEDITOR_API

class UDataflow;
class UDataflowInstanceInterface;

namespace UE::DataflowAssetDefinitionHelpers
{
	// Create a brand new empty dataflow asset
	// @return true is the asset is successfully created ( in that case OutDataflowAsset will containt the pointer to this asset )
	DATAFLOWEDITOR_API bool CreateNewDataflowAsset(const UObject* BoundAsset, UObject*& OutDataflowAsset);

	// Take an embedded dataflow asset from BoundAsset and make it shareable by being its own separate asset
	// If the Dataflow asset was already shareable , do nothing 
	// @return true if the operation was a success ( in that case OutDataflowAsset will containt the pointer to the shareable Dataflow asset )
	DATAFLOWEDITOR_API bool MakeShareableDataflowAsset(UObject* BoundAsset, UDataflow*& OutDataflowAsset);

	// Take an shared dataflow asset from BoundAsset and embded it in BoundAsset
	// If the Dataflow asset was already embedded, do nothing 
	// @return true if the operation was a success ( in that case OutDataflowAsset will containt the pointer to the Embdedded Dataflow asset )
	DATAFLOWEDITOR_API bool EmbedDataflowAsset(UObject* BoundAsset, UDataflow*& OutDataflowAsset);

	// Return true if we should proceed, false if we should re-open the dialog
	DATAFLOWEDITOR_API bool OpenDataflowAsset(const UObject* Asset, UObject*& OutDataflowAsset);

	// Return true if we should proceed, false if we should re-open the dialog
	DATAFLOWEDITOR_API bool NewOrOpenDialog(const UObject* Asset, UObject*& OutDataflowAsset);

	// Create a new UDataflow if one doesn't already exist for the Cloth Asset
	DATAFLOWEDITOR_API UObject* NewOrOpenDataflowAsset(const UObject* Asset);
	
	// Set dataflow from a template picked by the user
	DATAFLOWEDITOR_API bool SetDataflowFromTemplatePicker(UObject* Asset, bool bShowDefaultBlankOption = true);

	// Determine whether or not a UDataflow can be opened directly in the Dataflow Editor
	DATAFLOWEDITOR_API bool CanOpenDataflowAssetInEditor(const UObject* DataflowAsset);

	// Register Dataflow related menus
	DATAFLOWEDITOR_API FDelegateHandle RegisterDataflowAssetMenus(UClass* AssetClass);

	DATAFLOWEDITOR_API void UnregisterDataflowAssetMenus(FDelegateHandle Handle);

	// Create a new Dataflow based asset using the specified Dataflow template, an asset with an empty Dataflow if an empty path is provided, or an unbound asset if the path is nullptr.
	// @note The Dataflow template is always duplicated to use with the new asset, even when not embedded.
	// @return A new asset of type AssetClass that references a new Dataflow asset of the specified template.
	DATAFLOWEDITOR_API UObject* FactoryCreateNew(
		UClass* AssetClass,
		UObject* Parent,
		FName Name,
		EObjectFlags Flags,
		const FString* DataflowTemplatePath,
		bool bEmbedDataflow,
		const TCHAR* AssetPrefix);
}

struct FDataflowInput;
struct FDataflowOutput;

//
// Copy/Paste structs
//
USTRUCT()
struct FDataflowNodeData
{
	GENERATED_BODY()

	UPROPERTY()
	FString Type;

	UPROPERTY()
	FString Name;

	UPROPERTY()
	FString Properties;

	UPROPERTY()
	FVector2D Position = FVector2D::ZeroVector;
};

USTRUCT()
struct FDataflowCommentNodeData
{
	GENERATED_BODY()

	UPROPERTY()
	FString Name;

	UPROPERTY()
	FVector2D Size = FVector2D::ZeroVector;

	UPROPERTY()
	FLinearColor Color = FLinearColor::White;

	UPROPERTY()
	FVector2D Position = FVector2D::ZeroVector;

	UPROPERTY()
	int32 FontSize = 18;
};

USTRUCT()
struct FDataflowConnectionData
{
	GENERATED_BODY()

	UPROPERTY()
	FString Out;

	UPROPERTY()
	FString In;

	UE_API void Set(const FDataflowOutput& Output, const FDataflowInput& Input);

	static UE_API FString GetNode(const FString InConnection);
	static UE_API FString GetProperty(const FString InConnection);
	static UE_API void GetNodePropertyAndType(const FString InConnection, FString& OutNode, FString& OutProperty, FString& OutType);
};

USTRUCT()
struct FDataflowSubGraphData
{
	GENERATED_BODY()

	UPROPERTY()
	FGuid SubGraphGuid;

	UPROPERTY()
	FString Name;

	UPROPERTY()
	bool bIsForEach = false;

	UPROPERTY()
	TArray<FDataflowNodeData> NodeData;

	UPROPERTY()
	TArray<FDataflowCommentNodeData> CommentNodeData;

	UPROPERTY()
	TArray<FDataflowConnectionData> ConnectionData;
};

USTRUCT()
struct FDataflowCopyPasteContent
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FDataflowNodeData> NodeData;

	UPROPERTY()
	TArray<FDataflowCommentNodeData> CommentNodeData;

	UPROPERTY()
	TArray<FDataflowConnectionData> ConnectionData;

	UPROPERTY()
	TArray<FDataflowSubGraphData> SubGraphData;
};


UCLASS()
class UAssetDefinition_DataflowAsset : public UAssetDefinitionDefault
{
	GENERATED_BODY()

private:

	virtual FText GetAssetDisplayName() const override;
	virtual TSoftClassPtr<UObject> GetAssetClass() const override;
	virtual FLinearColor GetAssetColor() const override;
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override;
	virtual UThumbnailInfo* LoadThumbnailInfo(const FAssetData& InAssetData) const override;

	virtual	FAssetOpenSupport GetAssetOpenSupport(const FAssetOpenSupportArgs& OpenSupportArgs) const;
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
};

#undef UE_API
