// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/MathFwd.h"
#include "UObject/NameTypes.h"
#include "Templates/SharedPointer.h"

///////////////////////////////////////////////////////////

struct FEdGraphPinType;
struct FDataflowNode;
struct FInstancedPropertyBag;
struct FPropertyBagPropertyDesc;
class IAssetReferenceFilter;
class UDataflow;
class UDataflowEdNode;
class UDataflowSubGraph;
class UEdGraph;
class UEdGraphNode;
class UEdGraphPin;

/**
* Set to functions to modify a dataflow asset programmatically
* all changes are wrapped inside a scoped transaction as well and propertly marked the asset as modified if needed
*/
namespace UE::Dataflow
{
	static const FName DefaultNewVariableBaseName = "NewVariable";
	static const FName DefaultNewSubGraphBaseName = "NewSubGraph";

	struct FEditAssetUtils
	{
		/**
		* Test if a dataflow sub-object ( Node, subgraph ... ) name is unique
		*/
		UE_INTERNAL static DATAFLOWEDITOR_API bool IsUniqueDataflowSubObjectName(UDataflow* DataflowAsset, FName Name);

		UE_INTERNAL static DATAFLOWEDITOR_API TSharedPtr<IAssetReferenceFilter> MakeAssetReferenceFilter(const UEdGraph* Graph);

		/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
		//
		// Nodes API
		//
		/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
		/**
		* Add a new node to the graph
		* @param EdGraph	Main Graph or Subgraph to add the node to
		* @param Location	Location on the graph where to place the node
		* @param NodeName	Name to give to the node ( if the name is not unique a unique name will be created out of the parameter one)
		* @param TypeName	Name of the type of the node to add
		* @param FromPin	Pin to connect automatically to the node
		* @return the newly created EdGraphNode
		*/
		UE_INTERNAL static DATAFLOWEDITOR_API UDataflowEdNode* AddNewNode(UEdGraph* EdGraph, const FVector2D& Location, const FName NodeName, const FName NodeTypeName, UEdGraphPin* FromPin);

		/**
		* Add a new comment node to the graph
		* @param EdGraph	Main Graph or Subgraph to add the node to
		* @param Location	Location on the graph where to place the comment
		* @param Location	Size of the comment box
		* @return the newly created EdGraphNode
		*/
		UE_INTERNAL static DATAFLOWEDITOR_API UEdGraphNode* AddNewComment(UEdGraph* EdGraph, const FVector2D& Location, const FVector2D& Size);

		/**
		* Delete some nodes from a graph
		* @param EdGraph			Main Graph or Subgraph to duplicate the node in
		* @param NodesToDelete		List of node to delete
		*/
		UE_INTERNAL static DATAFLOWEDITOR_API void DeleteNodes(UEdGraph* EdGraph, const TArray<UEdGraphNode*>& NodesToDelete);

		/**
		* Duplicate nodes within the same graph
		* @param EdGraph			Main Graph or Subgraph to duplicate the node in
		* @param NodeToDuplicate	List of nodes to duplicate
		* @param Location			Location on the graph where to place the node
		* @param OutDuplicatedNodes	List of newly created nodes
		* @return the newly created EdGraphNode
		*/
		UE_INTERNAL static DATAFLOWEDITOR_API void DuplicateNodes(UEdGraph* EdGraph, const TArray<UEdGraphNode*>& NodesToDuplicate, const FVector2D& Location, TArray<UEdGraphNode*>& OutDuplicatedNodes);

		/**
		* Duplicate nodes from one graph to another ( assuming same dataflow asset )
		* @param SourceEdGraph		Main Graph or Subgraph to duplicate the node from
		* @param NodeToDuplicate	List of nodes to duplicate
		* @param TargetEdGraph		Main Graph or Subgraph to duplicate the node to
		* @param Location			Location on the graph where to place the node
		* @param OutDuplicatedNodes	List of newly created nodes
		* @param OutNodeGuidMap		Mapping of dup[licated -> new node guids
		* @return the newly created EdGraphNode
		*/
		UE_INTERNAL static DATAFLOWEDITOR_API void DuplicateNodes(UEdGraph* SourceEdGraph, const TArray<UEdGraphNode*>& NodesToDuplicate, UEdGraph* TargetEdGraph, const FVector2D& Location, TArray<UEdGraphNode*>& OutDuplicatedNodes, TMap<FGuid, FGuid>& OutNodeGuidMap);

		/**
		* Copy a list of nodes and their connection to the clipboard
		*/
		UE_INTERNAL static DATAFLOWEDITOR_API void CopyNodesToClipboard(const TArray<const UEdGraphNode*>& NodesToCopy, int32& OutNumCopiedNodes);

		/**
		* Attempt to paste nodes from the clipboard
		*/
		UE_INTERNAL static DATAFLOWEDITOR_API void PasteNodesFromClipboard(UEdGraph* EdGraph, const FVector2D& Location, TArray<UEdGraphNode*>& OutPastedNodes);

		/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
		//
		// Variables API
		//
		/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

		/**
		* Add a new Dataflow asset variable
		* If BaseName already exists, use it to generate a unique name by adding an increment to it in the form of BaseName_##
		*/
		UE_INTERNAL static DATAFLOWEDITOR_API FName AddNewVariable(UDataflow* DataflowAsset, FName BaseName = DefaultNewVariableBaseName);

		/**
		* Add a new Dataflow asset variable from a property bag description template
		* If BaseName already exists, use it to generate a unique name by adding an increment to it in the form of BaseName_##
		*/
		UE_INTERNAL static DATAFLOWEDITOR_API FName AddNewVariable(UDataflow* DataflowAsset, FName BaseName, const FPropertyBagPropertyDesc& TemplateDesc);

		/**
		* Remove a Dataflow asset variable
		*/
		UE_INTERNAL static DATAFLOWEDITOR_API void DeleteVariable(UDataflow* DataflowAsset, FName VariableName);

		/**
		* Rename a Dataflow asset variable
		*/
		UE_INTERNAL static DATAFLOWEDITOR_API void RenameVariable(UDataflow* DataflowAsset, FName OldVariableName, FName NewVariableName);

		/**
		* Duplicate a Dataflow asset variable
		* the new variable name will be generate uniquely based on the name of the original variable
		*/
		UE_INTERNAL static DATAFLOWEDITOR_API FName DuplicateVariable(UDataflow* DataflowAsset, FName VariableName);

		/**
		* Set the type of a Dataflow asset variable
		*/
		UE_INTERNAL static DATAFLOWEDITOR_API void SetVariableType(UDataflow* DataflowAsset, FName VariableName, const FEdGraphPinType& PinType);

		/*
		* Set value of a Dataflow asset variable from an property in a propertybag
		* Name and type must match for the operation to be successful
		*/
		UE_INTERNAL static DATAFLOWEDITOR_API void SetVariableValue(UDataflow* DataflowAsset, FName VariableName, const FInstancedPropertyBag& SourceBag);

		/**
		* Copy a variable to clipboard
		*/
		UE_INTERNAL static DATAFLOWEDITOR_API void CopyVariableToClipboard(UDataflow* DataflowAsset, FName VariableName);

		/**
		* Paste a variable from clipboard and return its name
		*/
		UE_INTERNAL static DATAFLOWEDITOR_API FName PasteVariableFromClipboard(UDataflow* DataflowAsset);

		/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
		//
		// SubGraph API
		//
		/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

		/**
		* Add a new SubGraph to the Dataflow Asset and return its name
		*/
		UE_INTERNAL static DATAFLOWEDITOR_API FName AddNewSubGraph(UDataflow* DataflowAsset, FName BaseName = DefaultNewSubGraphBaseName);

		/**
		* Rename a Dataflow SubGraph
		*/
		UE_INTERNAL static DATAFLOWEDITOR_API void RenameSubGraph(UDataflow* DataflowAsset, FName OldSubGraphName, FName NewSubGraphName);

		/**
		* Delete a Dataflow SubGraph
		*/
		UE_INTERNAL static DATAFLOWEDITOR_API void DeleteSubGraph(UDataflow* DataflowAsset, FGuid SubGraphGuid);

	private:
		/**
		* Create a UDataflowSubGraph that preserves a specific GUID and ForEach state.
		* Restricted to FEditAssetUtils via UDataflowSubGraph friend access so only the
		* paste-from-clipboard path can recreate a subgraph with a non-fresh GUID.
		*/
		static UDataflowSubGraph* CreateSubGraphWithGuid(UDataflow& DataflowAsset, FName UniqueName, const FGuid& Guid, bool bIsForEach);
	};
}
