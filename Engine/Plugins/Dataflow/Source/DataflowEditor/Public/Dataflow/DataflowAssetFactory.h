// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"

#include "DataflowAssetFactory.generated.h"

UCLASS()
class UDataflowAssetFactory : public UFactory
{
	GENERATED_BODY()
public:

	UDataflowAssetFactory();

	/** UFactory Interface */
	bool     CanCreateNew() const override;
	bool     FactoryCanImport(const FString& Filename) override;
	UObject* FactoryCreateNew(UClass*           InClass,
							  UObject*          InParent,
							  FName             InName,
							  EObjectFlags      Flags,
							  UObject*          Context,
							  FFeedbackContext* Warn) override;

	bool ShouldShowInNewMenu() const override;
	bool ConfigureProperties() override;
	/** End UFactory Interface */

private:
};
