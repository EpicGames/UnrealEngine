// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Dataflow/DataflowSelection.h"
#include "DetailLayoutBuilder.h"
#include "Delegates/Delegate.h"
#include "UObject/NameTypes.h"
#include "Styling/StarshipCoreStyle.h"
#include "Dataflow/DataflowTransformOutlinerWidget.h"
#include "Dataflow/DataflowVerticesOutlinerWidget.h"
#include "Dataflow/DataflowFacesOutlinerWidget.h"

struct FManagedArrayCollection;

/**
* Struct to hold OutputType/Selection data for the outputs
* Data is stored in a map using the OutputName as key: TMap<FString, SCollectionInfo>
*/
struct SCollectionInfo
{
//	FString OutputType;
	TSharedPtr<const FManagedArrayCollection> Collection;
};


/** 
* Header
* 1st column possible values: Transform Index/Face Index/Vertex Index based on the OutputType
* 2nd column is SelectionStatus
*/
struct FCollectionSpreadSheetHeader
{
	static const FName IndexColumnName;
	static const FName NoColumnName;

	TArray<FName> ColumnNames;
};


/** 
* Representing a row in the table 
* Index/SelectionStatus
*/
struct FCollectionSpreadSheetItem
{
	TArray<FString> Values;
};


/** 
* 
*/
class SCollectionSpreadSheetRow : public SMultiColumnTableRow<TSharedPtr<const FCollectionSpreadSheetItem>>
{
public:
	SLATE_BEGIN_ARGS(SCollectionSpreadSheetRow) 
	{}
		SLATE_ARGUMENT(TSharedPtr<const FCollectionSpreadSheetHeader>, Header)
		SLATE_ARGUMENT(TSharedPtr<const FCollectionSpreadSheetItem>, Item)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedRef<STableViewBase> OwnerTableView, const TSharedPtr<const FCollectionSpreadSheetHeader>& InHeader, const TSharedPtr<const FCollectionSpreadSheetItem>& InItem);

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

private:
	TSharedPtr<const FCollectionSpreadSheetHeader> Header;
	TSharedPtr<const FCollectionSpreadSheetItem> Item;
};

/** 
* 2xn grid to display Collection data
*/
class SCollectionSpreadSheet : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SCollectionSpreadSheet) {}
		SLATE_ARGUMENT(FName, SelectedOutput)
		SLATE_ARGUMENT(TSharedPtr<SScrollBar>, ExternalVerticalScrollBar)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	TMap<FString, SCollectionInfo>& GetCollectionInfoMap() { return CollectionInfoMap; }

	const FName& GetSelectedOutput() const;
	const FString GetSelectedOutputStr() const;
	void SetSelectedOutput(const FName& InSelectedOutput);
	
	const FName& GetSelectedGroup() const;
	void SetSelectedGroup(const FName& InSelectedGroup);

	int32 GetNumItems() const { return NumItems; }
	void SetNumItems(int32 InNumItems) { NumItems = InNumItems; }

	TSharedRef<ITableRow> GenerateRow(TSharedPtr<const FCollectionSpreadSheetItem> InItem, const TSharedRef<STableViewBase>& OwnerTable);

private:
	FName SelectedOutput = FName(TEXT(""));
	FName SelectedGroup = FName(TEXT(""));
	TMap<FString, SCollectionInfo> CollectionInfoMap; // Map<Name of Output, FManagedArrayCollection>

	TSharedPtr<SListView<TSharedPtr<const FCollectionSpreadSheetItem>>> ListView;
	TArray<TSharedPtr<const FCollectionSpreadSheetItem>> ListItems;

	TSharedPtr<FCollectionSpreadSheetHeader> Header;
	TSharedPtr<SHeaderRow> HeaderRowWidget;

	int32 NumItems = 0;
	void RegenerateHeader();
	void RepopulateListView();
};

class STransformOutliner;
class SVerticesOutliner;
class SFacesOutliner;

/**
* Widget for the CollectionSpreadSheet panel
*/
class SCollectionSpreadSheetWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SCollectionSpreadSheetWidget) {}
	SLATE_END_ARGS()

	/**
	* Slate construction function
	*
	* @param InArgs - Arguments passed from Slate
	*
	*/
	void DATAFLOWEDITOR_API  Construct(const FArguments& InArgs);

	void DATAFLOWEDITOR_API SetData(const FString& InNodeName);
	void DATAFLOWEDITOR_API RefreshWidget();
	TSharedPtr<SCollectionSpreadSheet> GetCollectionTable() { return CollectionTable; }
	void SetStatusText();

	void UpdateCollectionGroups(const FName& InOutputName);

	const FSlateBrush* GetPinButtonImage() const;
	const FSlateBrush* GetLockButtonImage() const;

	/** Gets a multicast delegate which is called whenever the PinnedDown button clicked */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnPinnedDownChanged, const bool);
	FOnPinnedDownChanged& GetOnPinnedDownChangedDelegate() { return OnPinnedDownChangedDelegate; }

	/** Gets a multicast delegate which is called whenever the LockRefresh button clicked */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnRefreshLockedChanged, const bool);
	FOnRefreshLockedChanged& GetOnRefreshLockedChangedDelegate() { return OnRefreshLockedChangedDelegate; }

	TSet<FName> NonListViewGroups = { FName(TEXT("Transform")), FName(TEXT("Vertices")), FName(TEXT("Faces")) };

	EVisibility GetCollectionSpreadSheetVisibility() const
	{
		return NonListViewGroups.Contains(CollectionTable->GetSelectedGroup()) ? EVisibility::Collapsed : EVisibility::Visible;
	}

	EVisibility GetTransformOutlinerVisibility() const
	{
		return CollectionTable->GetSelectedGroup() == FName(TEXT("Transform")) ? EVisibility::Visible : EVisibility::Collapsed;
	}

	EVisibility GetVerticesOutlinerVisibility() const
	{
		return CollectionTable->GetSelectedGroup() == FName(TEXT("Vertices")) ? EVisibility::Visible : EVisibility::Collapsed;
	}

	EVisibility GetFacesOutlinerVisibility() const
	{
		return CollectionTable->GetSelectedGroup() == FName(TEXT("Faces")) ? EVisibility::Visible : EVisibility::Collapsed;
	}

	// UI callbacks
	void NodeOutputsComboBoxSelectionChanged(FName InSelectedOutput, ESelectInfo::Type InSelectInfo);
	void CollectionGroupsComboBoxSelectionChanged(FName InSelectedOutput, ESelectInfo::Type InSelectInfo);
	FText GetNoOutputText();
	FText GetNoGroupText();

private:
	void UpdateTransformOutliner();
	void UpdateVertexOutliner();
	void UpdateFaceOutliner();

	TSharedPtr<STextBlock> NodeNameTextBlock;
	TSharedPtr<SComboBox<FName>> NodeOutputsComboBox;
	TSharedPtr<STextBlock> NodeOutputsComboBoxLabel;
	TSharedPtr<SComboBox<FName>> CollectionGroupsComboBox;
	TSharedPtr<STextBlock> CollectionGroupsComboBoxLabel;
	TSharedPtr<SCollectionSpreadSheet> CollectionTable;

	// Widget to display Transform hierarchy
	TSharedPtr<STransformOutliner> TransformOutliner;

	// Widget to display Vertices hierarchy
	TSharedPtr<SVerticesOutliner> VerticesOutliner;

	// Widget to display Faces hierarchy
	TSharedPtr<SFacesOutliner> FacesOutliner;

	TSharedPtr<STextBlock> StatusTextBlock;

	FString NodeName;
	TArray<FName> NodeOutputs;
	TArray<FName> CollectionGroups;
	bool bIsPinnedDown = false;
	bool bIsRefreshLocked = false;
	// ScrollBar to scroll ListView/TreeView view area
	TSharedPtr<SScrollBar> SpreadSheetHorizontalScrollBar;
	// External scrollbar to scroll CollectionSpreadSheet vertically, only visible when CollectionSpreadSheet is visible
	TSharedPtr<SScrollBar> CollectionSpreadSheetExternalVerticalScrollBar;
	// External scrollbar to scroll TransformOutliner vertically, only visible when TransformOutliner is visible
	TSharedPtr<SScrollBar> TransformOutlinerExternalVerticalScrollBar;
	// External scrollbar to scroll VerticesOutliner vertically, only visible when VerticesOutliner is visible
	TSharedPtr<SScrollBar> VerticesOutlinerExternalVerticalScrollBar;
	// External scrollbar to scroll FacesOutliner vertically, only visible when FacesOutliner is visible
	TSharedPtr<SScrollBar> FacesOutlinerExternalVerticalScrollBar;

protected:
	FOnPinnedDownChanged OnPinnedDownChangedDelegate;
	FOnRefreshLockedChanged OnRefreshLockedChangedDelegate;
};

