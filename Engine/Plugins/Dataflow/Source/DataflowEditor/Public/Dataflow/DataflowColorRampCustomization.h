// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"

namespace UE::Dataflow
{
	class FDataflowColorRampCustomization : public IPropertyTypeCustomization
	{
	public:

		/** Makes a new instance of this detail layout class for a specific detail view requesting it */
		DATAFLOWEDITOR_API static TSharedRef<IPropertyTypeCustomization> MakeInstance();

		//~ Begin - IPropertyTypeCustomization interface
		DATAFLOWEDITOR_API virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
		virtual void CustomizeChildren(TSharedRef<IPropertyHandle> /*StructPropertyHandle*/, IDetailChildrenBuilder& /*ChildBuilder*/, IPropertyTypeCustomizationUtils& /*CustomizationUtils*/) override {}
		//~ End - IPropertyTypeCustomization interface
	};

	class FDataflowLinearColorRampCustomization : public IPropertyTypeCustomization
	{
	public:

		/** Makes a new instance of this detail layout class for a specific detail view requesting it */
		DATAFLOWEDITOR_API static TSharedRef<IPropertyTypeCustomization> MakeInstance();

		//~ Begin - IPropertyTypeCustomization interface
		DATAFLOWEDITOR_API virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
		virtual void CustomizeChildren(TSharedRef<IPropertyHandle> /*StructPropertyHandle*/, IDetailChildrenBuilder& /*ChildBuilder*/, IPropertyTypeCustomizationUtils& /*CustomizationUtils*/) override {}
		//~ End - IPropertyTypeCustomization interface
	};
}
