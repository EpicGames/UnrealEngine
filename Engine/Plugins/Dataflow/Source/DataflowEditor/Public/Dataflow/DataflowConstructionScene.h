// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Dataflow/DataflowContent.h"
#include "Dataflow/DataflowEditorPreviewSceneBase.h"
#include "Dataflow/DataflowNodeParameters.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"

#define UE_API DATAFLOWEDITOR_API

class UDataflowEditor;
class USkeletalMeshComponent;
class UDebugDrawComponent;
class UDynamicMeshComponent;
class UMeshElementsVisualizer;
class UDataflowEdNode;
class UInteractiveToolPropertySet;
class UDataflowRenderableTypeSettings;
class UDataflowDebugMeshRenderSettings;

namespace UE
{
	namespace Geometry
	{
		class FDynamicMesh3;
	}

	namespace Dataflow
	{
		struct FRenderableTypeInstance;
	}
}

/**
 * Dataflow construction scene holding all the dynamic meshes used/displayed in the dataflow graph
 */
class FDataflowConstructionScene : public FDataflowPreviewSceneBase
{
	/*(const void* ObjectPointer, const int32 ObjectIndex)*/
	typedef TPair<TObjectPtr<const UDataflowEdNode>, int32> FDataflowRenderKey;

public:

	inline static FName DebugMeshComponentTag = TEXT("MeshDebugComponent");

	UE_API FDataflowConstructionScene(FPreviewScene::ConstructionValues ConstructionValues, UDataflowEditor* Editor);
	UE_API virtual ~FDataflowConstructionScene();

	// FGCObject interface
	UE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	
	/** Tick data flow scene */
	UE_API virtual void TickDataflowScene(const float DeltaSeconds) override;

	/** Check if the preview scene can run simulation */
	virtual bool CanRunSimulation() const override {return false;}
	
	/** Is there ANYTHING to render */
	UE_API bool HasRenderableGeometry();

	/** Close the construction scene */
	UE_API void ResetConstructionScene();

	/** Update the construction scene */
	UE_API void UpdateConstructionScene();
	
	/** Hide all or a single component */
	UE_API void SetVisibility(bool bVisible, UActorComponent* Component = nullptr);

	/** Array of all DynamicMeshComponents in the scene */
	UE_API TArray<TObjectPtr<UDynamicMeshComponent>> GetDynamicMeshComponents() const;

	/** Get all Debug draw components */
	UE_API TArray<TObjectPtr<UDebugDrawComponent>> GetDebugDrawComponents() const;

	/** Get the render settings for the selected nodes */
	UE_API TArray<UObject*> GetRenderSettings() const;

	/** select only the components that are relates to a specific node */
	UE_API void SelectNodeComponents(const UDataflowEdNode* EdNode);

	/** Get the render settings for matching the selected nodes */
	UE_API TArray<UDataflowRenderableTypeSettings*> GetRenderableTypeSettings() const;

	DECLARE_MULTICAST_DELEGATE(FOnConstructionSceneUpdated)
	FOnConstructionSceneUpdated OnConstructionSceneUpdated;

private:

	/** Reset the scene mesh visualizer */
	UE_API void ResetWireframeMeshElementsVisualizer();

	/** Update the scene mesh visualizer */
	UE_API void UpdateWireframeMeshElementsVisualizer();

	/** Add a mesh visualizer to the scene */
	UE_API void AddWireframeMeshElementsVisualizer();

	/** Reset all the scene components (dynamic mesh / primitive)*/
    UE_API void ResetSceneComponents();
	
	/** Build the dynamic mesh components from the dataflow graph render targets*/
	UE_API void UpdateDynamicMeshComponents();

	/** Build the primitive components from the dataflow graph selected node */
    UE_API void UpdatePrimitiveComponents();
	
	/** Get the materials stored in the rendering facade for a speicfic mesh */
	TArray<UMaterialInterface*> GetMaterialsFromRenderingFacade(const GeometryCollection::Facades::FRenderingFacade& RenderingFacade, int32 MeshIndex);

	/** Build the primitive components from the dataflow graph selected node */
	void UpdateComponentsFromNode(const UDataflowEdNode* Node, bool bWireframe);

	/** Add a dynamic mesh component to the scene */
	UE_API TObjectPtr<UDynamicMeshComponent>& AddDynamicMeshComponent(
		FDataflowRenderKey InKey, const FString& MeshName, UE::Geometry::FDynamicMesh3&& DynamicMesh,  const TArray<UMaterialInterface*>& MaterialSet, const FTransform& InMeshTransform = FTransform::Identity);

	/** Remove the scene component from selection and destroy it */
	UE_API void RemoveSceneComponent(USelection* SelectedComponents, UPrimitiveComponent* PrimitiveComponent);

	/** update floor visibility based on the view type */
	void UpdateFloorVisibility();

	/** Dynamic mesh components generated by the tools for edit/manipulation*/
	typedef TPair<FDataflowRenderKey, TObjectPtr<UDynamicMeshComponent>> FRenderElement;
	TMap<FDataflowRenderKey, TObjectPtr<UDynamicMeshComponent>> DynamicMeshComponents;

	/** Rest space wireframe. They have to get ticked to be able to respond to setting changes. [ticked] */
	typedef TPair<TObjectPtr<UDynamicMeshComponent>, TObjectPtr<UMeshElementsVisualizer>> FRenderWireElement;
	TMap<TObjectPtr<UDynamicMeshComponent>,TObjectPtr<UMeshElementsVisualizer> > WireframeElements;

	/** Source meshes for Wireframe rendering */
	TArray<UDynamicMeshComponent*> MeshComponentsForWireframeRendering;

	/** Property objects (visible or not) that get ticked. [ticked] */
	TArray<TObjectPtr<UInteractiveToolPropertySet>> PropertyObjectsToTick;

	/** Last Rendered Timestamp */
	UE::Dataflow::FTimestamp LastRenderedTimestamp = UE::Dataflow::FTimestamp::Invalid;

	/** Show the wireframe on render */ 
	bool bConstructionViewWireframe = true;
	
	/** Primitive components generated by the nodes and to be used by the tools */
	TArray<TObjectPtr<UPrimitiveComponent>> PrimitiveComponents;

	/** list of renderable type instance, there's one per node/output that can be rendered */
	TArray<UE::Dataflow::FRenderableTypeInstance> RenderableTypeInstances;

	TObjectPtr<UDataflowDebugMeshRenderSettings> DebugMeshRenderSettings = nullptr;
};

#undef UE_API
