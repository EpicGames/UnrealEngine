// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Dataflow/DataflowEditorViewportClientBase.h"
#include "Dataflow/DataflowContent.h"
#include "Delegates/Delegate.h"

#define UE_API DATAFLOWEDITOR_API


class FDataflowEditorToolkit;
class FDataflowConstructionScene;
class USelection;
class IClickDragBehaviorTarget;
class FEditor2DMouseWheelZoomBehaviorTarget;
namespace UE::Dataflow
{
	class IDataflowConstructionViewMode;
}

class FDataflowConstructionViewportClient : public FDataflowEditorViewportClientBase
{
public:
	using Super = FDataflowEditorViewportClientBase;

	UE_API FDataflowConstructionViewportClient(FEditorModeTools* InModeTools, TWeakPtr<FDataflowConstructionScene> InConstructionScenePtr,  const bool bCouldTickScene,
								  const TWeakPtr<SEditorViewport> InEditorViewportWidget = nullptr);

	UE_API virtual ~FDataflowConstructionViewportClient();

	UE_API void SetConstructionViewMode(const UE::Dataflow::IDataflowConstructionViewMode* InViewMode);

	UE_API USelection* GetSelectedComponents() const;

	/** Set the data flow toolkit used to create the client*/
	UE_API void SetDataflowEditorToolkit(TWeakPtr<FDataflowEditorToolkit> DataflowToolkit);
	
	/** Get the data flow toolkit  */
	const TWeakPtr<FDataflowEditorToolkit>& GetDataflowEditorToolkit() const { return DataflowEditorToolkitPtr; }

	/** Set the tool command list */
	UE_API void SetToolCommandList(TWeakPtr<FUICommandList> ToolCommandList);

	// FGCObject Interface
	virtual FString GetReferencerName() const override { return TEXT("FDataflowConstructionViewportClient"); }

	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnSelectionChangedMulticast, const TArray<UPrimitiveComponent*>&, const TArray<FDataflowBaseElement*>&)
	FOnSelectionChangedMulticast OnSelectionChangedMulticast;

	UE_API FString GetOverlayString() const;

	/** Return the widget mode only used for tooling */
	UE::Widget::EWidgetMode GetToolWidgetMode() const {return WidgetMode;}

	UE_API void ShowMeshEdges(bool bShow);

	UE_API void FocusViewportOnBoundingBox(const FBox& BoundingBox, bool bInstant);

protected:

	// FGCObject
	UE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;

private:

	// FDataflowEditorViewportClientBase
	UE_API virtual void OnViewportClicked(HHitProxy* HitProxy) override;

	// FEditorViewportClient interface
	UE_API virtual void Tick(float DeltaSeconds) override;
	UE_API virtual bool InputKey(const FInputKeyEventArgs& EventArgs) override;
	UE_API virtual void ProcessClick(FSceneView& View, HHitProxy* HitProxy, FKey Key, EInputEvent Event, uint32 HitX, uint32 HitY) override;
	UE_API virtual void Draw(const FSceneView* View, FPrimitiveDrawInterface* PDI) override;
	UE_API virtual void DrawCanvas(FViewport& InViewport, FSceneView& View, FCanvas& Canvas) override;
	UE_API virtual float GetMinimumOrthoZoom() const override;
	UE_API virtual bool CanSetWidgetMode(UE::Widget::EWidgetMode NewMode) const override;
	UE_API virtual UE::Widget::EWidgetMode GetWidgetMode() const override;
	UE_API virtual void SetWidgetMode(UE::Widget::EWidgetMode NewMode) override;
	UE_API virtual bool ShouldOrbitCamera() const override;

	UE_API void UpdateFromSettings();

	TUniquePtr<IClickDragBehaviorTarget> OrthoScrollBehaviorTarget;
	TUniquePtr<FEditor2DMouseWheelZoomBehaviorTarget> ZoomBehaviorTarget;
	TArray<TObjectPtr<UInputBehavior>> BehaviorsFor2DMode;

	/** Toolkit used to create the viewport client */
	TWeakPtr<FDataflowEditorToolkit> DataflowEditorToolkitPtr = nullptr;

	/** Dataflow preview scene from the toolkit */
	TWeakPtr<FDataflowConstructionScene> ConstructionScene;

	// @todo(brice) : Is this needed?
	TWeakPtr<FUICommandList> ToolCommandList;

	/** Construction view mode */
	const UE::Dataflow::IDataflowConstructionViewMode* ConstructionViewMode = nullptr;

	/** Flag to enable scene ticking from the client */
	bool bEnableSceneTicking = false;

	// Saved view transforms for the currently inactive view modes (e.g. store the 3D camera here while in 2D mode and vice-versa)
	TMap<FName, FViewportCameraTransform> SavedInactiveViewTransforms;

	/** Currently active transform widget type. */
	UE::Widget::EWidgetMode WidgetMode = UE::Widget::WM_Translate;

	FDelegateHandle OnSettingsChangeHandle;
};

#undef UE_API
