// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"
#include "Templates/SharedPointer.h"

class FPrimitiveDrawInterface;
class FSceneView;
class FCanvas;
class FMenuBuilder;
class FDataflowConstructionScene;
class FDataflowConstructionViewportClient;

namespace UE::Dataflow
{

	class IDataflowConstructionVisualization
	{
	public:
		virtual ~IDataflowConstructionVisualization() = default;

		virtual FName GetName() const = 0;
		virtual void ExtendViewportShowMenu(const TSharedPtr<FDataflowConstructionViewportClient>& ViewportClient, FMenuBuilder& MenuBuilder) {};
		virtual void Draw(const FDataflowConstructionScene* ConstructionScene, FPrimitiveDrawInterface* PDI, const FSceneView* View = nullptr) {};
		virtual void DrawCanvas(const FDataflowConstructionScene* ConstructionScene, FCanvas* Canvas, const FSceneView* SceneView) {};
	};

	class FDataflowConstructionVisualizationRegistry
	{
	public:

		// FLazySingleton
		static DATAFLOWEDITOR_API FDataflowConstructionVisualizationRegistry& GetInstance();
		static DATAFLOWEDITOR_API void TearDown();

		DATAFLOWEDITOR_API void RegisterVisualization(TUniquePtr<IDataflowConstructionVisualization>&& Visualization);
		DATAFLOWEDITOR_API void DeregisterVisualization(const FName& VisualizationName);

		DATAFLOWEDITOR_API const TMap<FName, TUniquePtr<IDataflowConstructionVisualization>>& GetVisualizations() const;
		DATAFLOWEDITOR_API const IDataflowConstructionVisualization* GetVisualization(const FName& VisualizationOption) const;

	private:

		TMap<FName, TUniquePtr<IDataflowConstructionVisualization>> VisualizationMap;
	};

}

