// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseCharacterFXEditor.h"

#include "Dataflow/DataflowEngine.h"
#include "Dataflow/AssetDefinition_DataflowAsset.h"
#include "Dataflow/DataflowAssetFactory.h"
#include "Dataflow/DataflowEditorCommands.h"
#include "Dataflow/DataflowContent.h"
#include "Dataflow/DataflowSchema.h"
#include "Dataflow/DataflowSNode.h"
#include "Dataflow/DataflowSNodeFactories.h"
#include "Templates/SharedPointer.h"

#include "DataflowEditor.generated.h"

#define UE_API DATAFLOWEDITOR_API

class FDataflowEditorToolkit;
class UDataflowBaseContent;

/**
* Base class defining settings for Dataflow
* Each part of the dataflow editor can define settings and assets can define instance of those settings
* The interface does not define any method as it is only used as based class for those settings to be discovered and also recognized by the editor
*/

UCLASS(MinimalAPI)
class UDataflowEditorSettings : public UObject
{
	GENERATED_BODY()
};

UCLASS(MinimalAPI, config = EditorPerProjectUserSettings)
class UDataflowEditorUserSettings : public UObject
{
	GENERATED_BODY()

public:
	/** When true use the task graph based evaluation system */
	UPROPERTY(config, EditAnywhere, Category = "Dataflow")
	bool bUseAsyncEvaluation = false;

	/** Whether to show the performance data on top of the nodes */
	UPROPERTY(config, EditAnywhere, Category = "Dataflow")
	bool bShowPerformanceData = false;
};

/** 
 * The actual asset editor class doesn't have that much in it, intentionally. 
 * 
 * Our current asset editor guidelines ask us to place as little business logic as possible
 * into the class, instead putting as much of the non-UI code into the subsystem as possible,
 * and the UI code into the toolkit (which this class owns).
 *
 * However, since we're using a mode and the Interactive Tools Framework, a lot of our business logic
 * ends up inside the mode and the tools, not the subsystem. The front-facing code is mostly in
 * the asset editor toolkit, though the mode toolkit has most of the things that deal with the toolbar
 * on the left.
 */

UCLASS(MinimalAPI)
class UDataflowEditor : public UBaseCharacterFXEditor
{
	GENERATED_BODY()

public:
	UE_API UDataflowEditor();

	// UBaseCharacterFXEditor interface
	UE_API virtual TSharedPtr<FBaseAssetToolkit> CreateToolkit() override;

	/** Initialize editor contents given a list of objects */
	UE_API virtual void Initialize(const TArray<TObjectPtr<UObject>>& InObjects, const TSubclassOf<AActor>& InPreviewClass = {}) override;

	/** Update the terminal contents */
	UE_API void UpdateTerminalContents(const UE::Dataflow::FTimestamp TimeStamp);
	
	/** Update the editor content */
	UE_API void UpdateEditorContent();

	/** Dataflow editor content accessors */
	TObjectPtr<UDataflowBaseContent>& GetEditorContent() { return EditorContent; }
	const TObjectPtr<UDataflowBaseContent>& GetEditorContent() const { return EditorContent; }

	/** Dataflow terminal contents accessors */
	TArray<TObjectPtr<UDataflowBaseContent>>& GetTerminalContents() { return TerminalContents; }
	const TArray<TObjectPtr<UDataflowBaseContent>>& GetTerminalContents() const { return TerminalContents; }

	/** Get the registered tool categories */
	const TArray<FName>& GetToolCategories() const { return ToolCategories; }

	/** Get the editor settings */
	template<typename T>
	inline const T* FindEditorSettings() const 
	{ 
		for (const UDataflowEditorSettings* Settings : EditorSettings)
		{
			if (const T* TypedSettings = Cast<T>(Settings))
			{
				return TypedSettings;
			}
		}
		return nullptr;
	}

	/** Registere tool categories available for that construction scene */
	void RegisterToolCategories(const TArray<FName>& InToolCategrories) { ToolCategories = InToolCategrories; }

	/** set the settings for this editor - this needs to be called before initialize for it to be applied */
	UE_API void AddEditorSettings(const UDataflowEditorSettings* Settings);

private :

	friend class FDataflowEditorToolkit;

	using ValidTerminalsType = TMap<TSharedPtr<FDataflowNode>,TObjectPtr<UDataflowBaseContent>>;

	/** Remove invalid terminal contents from the container */
	UE_API void RemoveTerminalContents(const TSharedPtr<UE::Dataflow::FGraph>& DataflowGraph, ValidTerminalsType& ValidTerminals);
	
	/** Add valid terminal nodes to the container */
	UE_API void AddTerminalContents(const TSharedPtr<UE::Dataflow::FGraph>& DataflowGraph, ValidTerminalsType& ValidTerminals);
	
	// Dataflow editor is the owner of the object list to edit/process and the dataflow mode
	// is the one holding the dynamic mesh components to be rendered in the viewport
	// It is why the data flow asset/owner/skelmesh have been added here. Could be added
	// in the subsystem if necessary
	UPROPERTY()
	TObjectPtr<UDataflowBaseContent> EditorContent;

	/** List of dataflow contents available in the graph and coming from all the terminal nodes */
	UPROPERTY()
	TArray<TObjectPtr<UDataflowBaseContent>> TerminalContents;

	/** array of settings for this instance */
	UPROPERTY()
	TArray<TObjectPtr<const UDataflowEditorSettings>> EditorSettings;

	/** List of tool categories registered for that editor */
	TArray<FName> ToolCategories;
};

DECLARE_LOG_CATEGORY_EXTERN(LogDataflowEditor, Log, All);

#undef UE_API
