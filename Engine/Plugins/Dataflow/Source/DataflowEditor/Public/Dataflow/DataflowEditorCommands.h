// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Dataflow/DataflowObjectInterface.h"
#include "BaseCharacterFXEditorCommands.h"
#include "Styling/AppStyle.h"

#define UE_API DATAFLOWEDITOR_API

class FDragDropEvent;
struct FDataflowOutput;
struct FGeometry;
class IStructureDetailsView;
class UDataflow;
class UDataflowEdNode;
struct FDataflowNode;
class UEdGraph;
class UEdGraphNode;
class UEdGraphNode_Comment;
class SDataflowGraphEditor;
class UDataflowBaseContent;

typedef TSet<class UObject*> FGraphPanelSelectionSet;

/*
* FDataflowEditorCommandsImpl
* 
*/
class DATAFLOWEDITOR_API FDataflowEditorCommandsImpl : public TBaseCharacterFXEditorCommands<FDataflowEditorCommandsImpl>
{
public:

	FDataflowEditorCommandsImpl();

	// Note: We need to explicitly disable warnings on these constructors/operators for clang to be happy with deprecated variables
	// (See Confluence page "How to Deprecate Code in UE > How to deprecate variables inside a UStruct")
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	~FDataflowEditorCommandsImpl() = default;
	FDataflowEditorCommandsImpl(const FDataflowEditorCommandsImpl&) = default;
	FDataflowEditorCommandsImpl(FDataflowEditorCommandsImpl&&) = default;
	FDataflowEditorCommandsImpl& operator=(const FDataflowEditorCommandsImpl&) = default;
	FDataflowEditorCommandsImpl& operator=(FDataflowEditorCommandsImpl&&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// TBaseCharacterFXEditorCommands<> interface
	 virtual void RegisterCommands() override;

	// TInteractiveToolCommands<>
	// Each tool will have its own TInteractiveToolCommands<> object stored in the DataflowToolRegistry, so this should not return anything
	virtual void GetToolDefaultObjectList(TArray<UInteractiveTool*>& ToolCDOs) override {}

	/**
	* Add or remove commands relevant to Tool to the given UICommandList.
	* Call this when the active tool changes (eg on ToolManager.OnToolStarted / OnToolEnded)
	* @param bUnbind if true, commands are removed, otherwise added
	*/
	 static void UpdateToolCommandBinding(UInteractiveTool* Tool, TSharedPtr<FUICommandList> UICommandList, bool bUnbind = false);
	
	TSharedPtr<FUICommandInfo> EvaluateNode;
	TSharedPtr<FUICommandInfo> EvaluateGraph;
	TSharedPtr<FUICommandInfo> ToggleSimulation;
	TSharedPtr<FUICommandInfo> StartSimulation;
	TSharedPtr<FUICommandInfo> PauseSimulation;
	TSharedPtr<FUICommandInfo> StepSimulation;
	TSharedPtr<FUICommandInfo> ResetSimulation;
	TSharedPtr<FUICommandInfo> StopSimulation;
	TSharedPtr<FUICommandInfo> EvaluateGraphAutomatic;
	TSharedPtr<FUICommandInfo> EvaluateGraphManual;
	TSharedPtr<FUICommandInfo> ClearGraphCache;
	TSharedPtr<FUICommandInfo> TogglePerfData;
	TSharedPtr<FUICommandInfo> FreezeNodes;
	TSharedPtr<FUICommandInfo> UnfreezeNodes;
	TSharedPtr<FUICommandInfo> CreateComment;
	TSharedPtr<FUICommandInfo> ToggleEnabledState;
	TSharedPtr<FUICommandInfo> ToggleObjectSelection;
	TSharedPtr<FUICommandInfo> ToggleFaceSelection;
	TSharedPtr<FUICommandInfo> ToggleVertexSelection;
	TSharedPtr<FUICommandInfo> AddOptionPin;
	TSharedPtr<FUICommandInfo> RemoveOptionPin;
	TSharedPtr<FUICommandInfo> ZoomToFitGraph;
	TSharedPtr<FUICommandInfo> AddNewVariable;
	TSharedPtr<FUICommandInfo> AddNewSubGraph;
	TSharedPtr<FUICommandInfo> ConvertToBasicSubGraph;
	TSharedPtr<FUICommandInfo> ConvertToForEachSubGraph;
	TSharedPtr<FUICommandInfo> ToggleAsyncEvaluation;
	TSharedPtr<FUICommandInfo> CollapseToSubGraph;
	TSharedPtr<FUICommandInfo> ConvertToEmbeddedDataflow;
	TSharedPtr<FUICommandInfo> ConvertToExternalDataflow;
	TSharedPtr<FUICommandInfo> ToggleDataflowEditingLock;

	TMap<FName, TSharedPtr<FUICommandInfo>> SetConstructionViewModeCommands;
	TMap<FName, TSharedPtr<FUICommandInfo>> ToolPaletteCommands;

	UE_DEPRECATED(5.5, "Dataflow Tool commands are now stored in FDataflowToolRegistry")
	const static FString BeginWeightMapPaintToolIdentifier;
	UE_DEPRECATED(5.5, "Dataflow Tool commands are now stored in FDataflowToolRegistry")
	TSharedPtr<FUICommandInfo> BeginWeightMapPaintTool;
	
	const static FString AddWeightMapNodeIdentifier;
	TSharedPtr<FUICommandInfo> AddWeightMapNode;
};

//@todo(brice) Merge this into the above class
class FDataflowEditorCommands
{
public:
	UE_DEPRECATED(5.8, "Use SDataflowGraphEditor::FGraphEvaluationCallback instead ")
	typedef TFunction<void(const FDataflowNode*, const FDataflowOutput*)> FGraphEvaluationCallback;

	UE_DEPRECATED(5.8, "Use SDataflowGraphEditor::FOnDragDropEventCallback instead ")
	typedef TFunction<void(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)> FOnDragDropEventCallback;

	static UE_API void Register();
	static UE_API void Unregister();
	static UE_API bool IsRegistered();

	static UE_API const FDataflowEditorCommandsImpl& Get();

	/*
	* Node evaluation utility function.
	* 
	* @param Context  The evaluation context.
	* @param InOutLastNodeTimestamp  The last evaluation time used to trigger the evaluation when the node's timestamp is more recent than this value. If the node is evaluated, the value also gets updated with the evaluated node's current timestamp.
	* @param Dataflow  The dataflow asset used to search for the NodeName when Node is nullptr.
	* @param Node  The node to evaluate. When null, a node with the given NodeName will be evaluated instead if it exists.
	* @param Output  The node's output to evaluate. When no output are specified, all outputs will be evaluated.
	* @param NodeName  When no node is specified, then the node will be searched instead within the Dataflow's graph using NodeName, otherwise NodeName is ignored.
	* @param Asset  When Asset is non null, if the node is a terminal node, and if the node timestamp is more recent than InOutLastNodeTimestamp. then the node SetAssetValue method will be called on this asset.
	* @return  The node that has been evaluated if any.
	*/
	static UE_API const FDataflowNode* EvaluateNode(UE::Dataflow::FContext& Context, UE::Dataflow::FTimestamp& InOutLastNodeTimestamp,
		const UDataflow* Dataflow, const FDataflowNode* Node, const FDataflowOutput* Output = nullptr, 
		const FString& NodeName = FString(), UObject* Asset = nullptr);

	static UE_API void EvaluateNode(UE::Dataflow::FContext& Context, const FDataflowNode& Node, const FDataflowOutput* Output, UObject* Asset, UE::Dataflow::FTimestamp& InOutLastNodeTimestamp, UE::Dataflow::FOnPostEvaluationFunction OnEvaluationCompleted);

	/*
	*  DeleteNodes
	*/
	static UE_API void DeleteNodes(UEdGraph* EdGraph, const FGraphPanelSelectionSet& SelectedNodes);

	/*
	*  FreezeNodes
	*/
	static UE_API void FreezeNodes(UE::Dataflow::FContext& Context, const FGraphPanelSelectionSet& SelectedNodes);

	/*
	*  UnfreezeNodes
	*/
	static UE_API void UnfreezeNodes(UE::Dataflow::FContext& Context, const FGraphPanelSelectionSet& SelectedNodes);

	/*
	* OnNodeVerifyTitleCommit
	*/
	static UE_API bool OnNodeVerifyTitleCommit(const FText& NewText, UEdGraphNode* GraphNode, FText& OutErrorMessage);

	/*
	* OnNodeTitleCommitted
	*/
	static UE_API void OnNodeTitleCommitted(const FText& InNewText, ETextCommit::Type InCommitType, UEdGraphNode* GraphNode);

	/*
	* OnNotifyPropertyPreChange
	*/
	static UE_API void OnNotifyPropertyPreChange(TSharedPtr<IStructureDetailsView> PropertiesEditor, UDataflow* Graph, class FEditPropertyChain* PropertyAboutToChange);

	/*
	*  OnPropertyValueChanged
	*/
	static UE_API void OnPropertyValueChanged(UDataflow* Graph, TSharedPtr<UE::Dataflow::FEngineContext>& Context, UE::Dataflow::FTimestamp& OutLastNodeTimestamp, const FPropertyChangedEvent& PropertyChangedEvent, const TSet<TObjectPtr<UDataflowEdNode>>& NewSelection = {});
	static UE_API void OnCommentPropertyValueChanged(const FPropertyChangedEvent& PropertyChangedEvent, const TSet<TObjectPtr<UEdGraphNode_Comment>>& CommentSelection = {});
	static UE_API void OnAssetPropertyValueChanged(TObjectPtr<UDataflowBaseContent> Content, const FPropertyChangedEvent& PropertyChangedEvent);

	/*
	*  OnSelectedNodesChanged
	*/
	static UE_API void OnSelectedNodesChanged(TSharedPtr<IStructureDetailsView> PropertiesEditor, UObject* Asset, UDataflow* Graph, const TSet<TObjectPtr<UObject>>& NewSelection);

	/*
	*  ToggleEnabledState
	*/
	static UE_API void ToggleEnabledState(UDataflow* Graph);

	/*
	*  DuplicateNodes
	*/
	static UE_API void DuplicateNodes(UEdGraph* EdGraph, const TSharedPtr<SDataflowGraphEditor>& DataflowGraphEditor, const FGraphPanelSelectionSet& SelectedNodes);

	/*
	*  CopyNodes
	*/
	static UE_API void CopyNodes(UEdGraph* EdGraph, const TSharedPtr<SDataflowGraphEditor>& DataflowGraphEditor, const FGraphPanelSelectionSet& SelectedNodes);

	/*
	*  PasteSelectedNodes
	*/
	static UE_API void PasteNodes(UEdGraph* EdGraph, const TSharedPtr<SDataflowGraphEditor>& DataflowGraphEditor);

	/*
	*  RenameNode
	*/
	static UE_API void RenameNode(const TSharedPtr<SDataflowGraphEditor>& DataflowGraphEditor, UEdGraphNode* EdNode);
};


#undef UE_API
