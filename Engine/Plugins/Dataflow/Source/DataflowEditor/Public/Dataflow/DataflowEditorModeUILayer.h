// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseCharacterFXEditorModeUILayer.h"
#include "DataflowEditorModeUILayer.generated.h"

#define UE_API DATAFLOWEDITOR_API

/** Interchange layer to manage built in tab locations within the editor's layout. **/
UCLASS(MinimalAPI)
class UDataflowEditorUISubsystem : public UAssetEditorUISubsystem
{
	GENERATED_BODY()
public:

	UE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	UE_API virtual void Deinitialize() override;
	UE_API virtual void RegisterLayoutExtensions(FLayoutExtender& Extender) override;

	// Identifier for the Layout extension in DataflowEditorToolkit's StandaloneDefaultLayout
	static UE_API const FName EditorSidePanelAreaName;

};

/** Handles the hosting of additional toolkits, such as the mode toolkit, within the DataflowEditor's toolkit. **/
class FDataflowEditorModeUILayer : public FBaseCharacterFXEditorModeUILayer
{
public:
	FDataflowEditorModeUILayer(const IToolkitHost* InToolkitHost) : FBaseCharacterFXEditorModeUILayer(InToolkitHost) {}
};

#undef UE_API
