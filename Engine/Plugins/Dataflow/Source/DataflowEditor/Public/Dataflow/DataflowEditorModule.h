// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BaseCharacterFXEditorModule.h"

#define UE_API DATAFLOWEDITOR_API


class FDataflowSNodeFactory;

/**
 * The public interface to this module
 */
class FDataflowEditorModule : public FBaseCharacterFXEditorModule
{
public:

	static UE_API const FColor SurfaceColor;

	/** IModuleInterface implementation */
	UE_API virtual void StartupModule();
	UE_API virtual void ShutdownModule();

private:

	TSharedPtr<FDataflowSNodeFactory> DataflowSNodeFactory;
	FDelegateHandle AssetViewExtraStateGeneratorHandle;
	FDelegateHandle SkeletalMeshDataflowMenusHandle;

};

#undef UE_API
