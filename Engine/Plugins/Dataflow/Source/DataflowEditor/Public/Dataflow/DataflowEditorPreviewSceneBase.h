// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AdvancedPreviewScene.h"
#include "DataStorage/CommonTypes.h"
#include "Dataflow/DataflowDebugDraw.h"
#include "Dataflow/DataflowDebugDrawObject.h"

#include "DataflowEditorPreviewSceneBase.generated.h"

#define UE_API DATAFLOWEDITOR_API

struct FDataflowBaseElement;
class UDataflowBaseContent;
class UDataflowEditor;
class FAssetEditorModeManager;
class USelection;

/**
 * TEDS tag added to any object that belongs to dataflow construction scene
 */
USTRUCT(meta = (DisplayName = "Dataflow construction object"))
struct FDataflowConstructionObjectTag final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

/**
 * TEDS tag added to any object that belongs to dataflow simulation scene
 */
USTRUCT(meta = (DisplayName = "Dataflow simulation object"))
struct FDataflowSimulationObjectTag final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

/**
 * TEDS tag added to any object that belongs to dataflow scene
 */
USTRUCT(meta = (DisplayName = "Dataflow scene object", EditorDataStorage_DynamicColumnTemplate))
struct FDataflowSceneObjectTag final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

/**
 * TEDS tag added to any struct that belongs to dataflow scene
 */
USTRUCT(meta = (DisplayName = "Dataflow scene struct", EditorDataStorage_DynamicColumnTemplate))
struct FDataflowSceneStructTag final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

/**
 * TEDS column added to any struct/scene to display the type (construction/simulation/evaluation)
 */
USTRUCT(meta = (DisplayName = "Dataflow scene type"))
struct FDataflowSceneTypeColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

public:

	int32 SceneType = 0;
};


/** Dataflow focus request delegate */
DECLARE_MULTICAST_DELEGATE_OneParam(FDataflowFocusRequestDelegate, const FBox&)

/**
 * Dataflow preview scene base
 * @brief the scene is holding all the objects that will be
 * visible and potentially editable within the viewport
 */
class FDataflowPreviewSceneBase : public FAdvancedPreviewScene
{
public:

	UE_API FDataflowPreviewSceneBase(FPreviewScene::ConstructionValues ConstructionValues, UDataflowEditor* Editor, const FName& InActorName);
	UE_API virtual ~FDataflowPreviewSceneBase();
	
	// FGCObject interface
	UE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;

	/** Dataflow editor content accessors */
	UE_API TObjectPtr<UDataflowBaseContent>& GetEditorContent();
	UE_API const TObjectPtr<UDataflowBaseContent>& GetEditorContent() const;

	/** Dataflow terminal contents accessors */
	UE_API TArray<TObjectPtr<UDataflowBaseContent>>& GetTerminalContents();
	UE_API const TArray<TObjectPtr<UDataflowBaseContent>>& GetTerminalContents() const;

	/** Root scene actor accessors */
	TObjectPtr<AActor> GetRootActor() { return RootSceneActor; }
	const TObjectPtr<AActor> GetRootActor() const { return RootSceneActor; }

	/** Dataflow mode manager accessors */
	TSharedPtr<FAssetEditorModeManager>& GetDataflowModeManager() { return DataflowModeManager; }
	const TSharedPtr<FAssetEditorModeManager>& GetDataflowModeManager() const { return DataflowModeManager; }
	
	/** Build the scene bounding box */
	UE_API virtual FBox GetBoundingBox() const;

	/** Get const scene elements */
	const IDataflowDebugDrawInterface::FDataflowElementsType& GetSceneElements() const {return SceneElements;}

	/** Get non const scene elements */
	IDataflowDebugDrawInterface::FDataflowElementsType& ModifySceneElements() {return SceneElements;}

	/** Get const debug draw component */
	const TObjectPtr<class UDataflowDebugDrawComponent>& GetDebugDrawComponent() const {return DebugDrawComponent;}

	/** Get non const debug draw component */
	TObjectPtr<class UDataflowDebugDrawComponent>& ModifyDebugDrawComponent() {return DebugDrawComponent;}

	/** Tick data flow scene */
	virtual void TickDataflowScene(const float DeltaSeconds) {}

	/** Check if a primitive component is selected */
	UE_API bool IsComponentSelected(const UPrimitiveComponent* InComponent) const;

	/** Check if the preview scene can run simulation */
	virtual bool CanRunSimulation() const {return false;}

	/** Update the currently selected scene profile */
	UE_API void SetCurrentProfileIndex(int32 NewProfileIndex);

	/** Return true if the preview scene is dirty */
	bool IsSceneDirty() const {return bPreviewSceneDirty;}

	/** Set the dirty flag */
	void SetDirtyFlag() { bPreviewSceneDirty = true; }

	/** Reset the dirty flag */
	void ResetDirtyFlag() {bPreviewSceneDirty = false;}

	/** Event triggered when an object is focused in the scene (double-click in the scene outliner)*/
	FDataflowFocusRequestDelegate& OnFocusRequest() { return FocusRequestDelegate; }

	/** Register all the scene elements to TEDs */
	UE_API void RegisterSceneElements(const bool bIsConstruction);

	/** Unregister all the scene elements to TEDs */
	UE_API void UnregisterSceneElements();

	/** Return the scene selected components */
	UE_API USelection* GetSelectedComponents() const;

	/** Get the dataflow editor */
	const UDataflowEditor* GetDataflowEditor() const { return DataflowEditor; }
	
protected:

	/** Store Scene object into the TEDS database */
	UE_API void AddSceneObject(UObject* SceneObject, const bool bIsConstruction) const;

	/** Store Scene struct into the TEDS database */
	UE_API void AddSceneStruct(void* SceneStruct, const TWeakObjectPtr<const UScriptStruct> TypeInfo, const bool bIsConstruction) const;

	/** Remove Scene object from the TEDS database */
	UE_API void RemoveSceneObject(UObject* SceneObject) const;

	/** Remove Scene struct from the TEDS database */
	UE_API void RemoveSceneStruct(void* SceneStruct) const;

	/** Respond to changes in the scene profile settings */
	UE_API void OnAssetViewerSettingsRefresh(const FName& InPropertyName);

	/** Root scene actor */
	TObjectPtr<AActor> RootSceneActor = nullptr; 

	/** Dataflow editor linked to that preview scene */
	UDataflowEditor* DataflowEditor = nullptr;

	/** Mode Manager for selection */
	TSharedPtr<FAssetEditorModeManager> DataflowModeManager;

	/** Boolean to check if the preview scene is dirty or not */
	bool bPreviewSceneDirty = false;

	/** Delegate to focus the viewport */
	FDataflowFocusRequestDelegate FocusRequestDelegate;

	/** List of scene elements that could be used in the editor (outliner/viewport...) */
	IDataflowDebugDrawInterface::FDataflowElementsType SceneElements;
	
	/** Persistent component used for debug drawing */
	TObjectPtr<class UDataflowDebugDrawComponent> DebugDrawComponent;
};

#undef UE_API
