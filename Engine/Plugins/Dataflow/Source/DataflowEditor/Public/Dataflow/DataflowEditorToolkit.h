// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseCharacterFXEditorToolkit.h"
#include "Misc/NotifyHook.h"
#include "TickableEditorObject.h"
#include "Dataflow/DataflowContent.h"
#include "Dataflow/DataflowEditor.h"

#include "DataflowEditorToolkit.generated.h"

#define UE_API DATAFLOWEDITOR_API

class FCanvas;
class FDocumentTracker;
class FSceneView;
class IDetailsView;
class FTabManager;
class FTabInfo;
class IDataflowViewListener;
class IStructureDetailsView;
class UDataflow;
class UDataflowSubGraph;
class SDataflowGraphEditor;
class FDataflowCollectionSpreadSheet;
class FDataflowConstructionScene;
class FDataflowSimulationViewportClient;
class UDataflowBaseContent;
class FDataflowSimulationScene;
class FDataflowSkeletonView;
class FDataflowOutlinerView;
class UDataflowEditor;
class FDataflowSimulationSceneProfileIndexStorage;
class FDataflowOutputLog;
class FDataflowSelectionView;
class SDataflowMembersWidget;
class SDataflowConstructionViewport;
class SDataflowSimulationViewport;
class SGraphEditor;
class SDataflowSimulationTimeline;
struct FDataflowPath;
class FDataflowSimulationBinding;
class UDataflowPropertyGizmoManager;

enum class EDataflowEditorEvaluationMode : uint8;

namespace UE::Dataflow
{
	class FDataflowNodeDetailExtensionHandler;
	namespace Private
	{
		// stores the log message per node and 
		struct FLogMessageStorage
		{
		public:
			void Add(const UE::Dataflow::FContext::FLogMessage& Message);
			void RemoveForNode(const FDataflowNode& Node, const FTimestamp& Timestamp = FTimestamp::Invalid);
			void GetMessageForNode(const FDataflowNode& Node, TArray<UE::Dataflow::FContext::FLogMessage>& OutMessages) const;
			EMessageSeverity::Type GetMaxSeverityForNode(const FDataflowNode& Node) const;
			int32 Num() const { return Entries.Num(); }
			void GetMessage(int32 Index, UE::Dataflow::FContext::FLogMessage& OutMessage) const;
			EMessageSeverity::Type GetMaxSeverity();

		private:
			struct FEntry
			{
				UE::Dataflow::FTimestamp Timestamp;
				UE::Dataflow::FContext::FLogMessage Message;
			};

			TArray<FEntry> Entries;
		};
	}
}
namespace EMessageSeverity { enum Type : int; }


UCLASS(MinimalAPI)
class UDataflowEvaluationSettings : public UDataflowEditorSettings
{
public:
	GENERATED_BODY()

	UPROPERTY()
	bool bAllowEvaluationInPIE = false;
};

class FDataflowEditorToolkit final : public FBaseCharacterFXEditorToolkit, public FTickableEditorObject, public FNotifyHook, public FGCObject
{
	using FBaseCharacterFXEditorToolkit::ObjectScene;

public:

	UE_API explicit FDataflowEditorToolkit(UAssetEditor* InOwningAssetEditor);
	UE_API ~FDataflowEditorToolkit();

	static UE_API bool CanOpenDataflowEditor(UObject* ObjectToEdit);
	static UE_API bool HasDataflowAsset(UObject* ObjectToEdit);
	static UE_API UDataflow* GetDataflowAsset(UObject* ObjectToEdit);
	static UE_API const UDataflow* GetDataflowAsset(const UObject* ObjectToEdit);
	UE_API bool IsSimulationDataflowAsset() const;
	UE_API FName GetGraphLogName() const;
	UE_API void LogMessage(const EMessageSeverity::Type Severity, const FDataflowNode* Node, const FDataflowOutput* Output, const FString& Message) const;
	UE_API void LogMessage(const EMessageSeverity::Type Severity, const FDataflowNode* Node, const FDataflowOutput* Output, const FText& Message) const;

	/** Editor dataflow content accessors */
	UE_API const TObjectPtr<UDataflowBaseContent>& GetEditorContent() const;
	UE_API TObjectPtr<UDataflowBaseContent>& GetEditorContent();

	/** Terminal dataflow contents accessors */
	UE_API const TArray<TObjectPtr<UDataflowBaseContent>>& GetTerminalContents() const;
	UE_API TArray<TObjectPtr<UDataflowBaseContent>>& GetTerminalContents();
	
	/** Dataflow graph editor accessor */
	const TSharedPtr<SDataflowGraphEditor> GetDataflowGraphEditor() const { return GraphEditor; }

	// IToolkit interface
	UE_API virtual FName GetToolkitFName() const override;
	UE_API virtual FText GetToolkitName() const override;
	UE_API virtual FText GetBaseToolkitName() const override;
	UE_API virtual FText GetToolkitToolTipText() const override;
	UE_API virtual FString GetWorldCentricTabPrefix() const override;
	UE_API virtual FLinearColor GetWorldCentricTabColorScale() const override;
	UE_API virtual void RegisterTabSpawners(const TSharedRef<FTabManager>& TabManager) override;
	UE_API virtual void UnregisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;

	/** Dataflow preview scenes accessor */
	const TSharedPtr<FDataflowSimulationScene>& GetSimulationScene() const { return SimulationScene; }
	const TSharedPtr<FDataflowConstructionScene>& GetConstructionScene() const { return ConstructionScene; }
	const TSharedPtr<FDataflowSimulationSceneProfileIndexStorage>& GetSimulationSceneProfileIndexStorage() const { return SimulationSceneProfileIndexStorage; }

	// FSerializableObject interface
	UE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FDataflowEditorToolkit");
	}
	// End of FSerializableObject interface

	UE_API void OpenSubGraphTab(FName SubGraphName);
	UE_API void OpenSubGraphTab(const UDataflowSubGraph* SubGraph);
	UE_API void CloseSubGraphTab(const UDataflowSubGraph* SubGraph);
	UE_API void ReOpenSubGraphTab(const UDataflowSubGraph* SubGraph);
	UE_API void SetSubGraphTabActiveState(TSharedPtr<SDataflowGraphEditor> SubGraphEditor, bool bActive);
	UE_API UDataflowSubGraph* GetSubGraph(const FGuid& SubGraphGuid) const;
	UE_API UDataflowSubGraph* GetSubGraph(FName SubGraphName) const;

	UE_API FGuid FocusOnNextVariableNode(FName VariableName, const FGuid& LastTimeNodeGuid);
	void FindAllVariableNodeInGraph(UEdGraph* EdGraph, FName VariableName, TArray<UDataflowEdNode*>& OutEdNodes);

	UE_API const FString& GetDebugDrawOverlayString() const;

	/** Draw property name labels at gizmo origins for the currently selected node. Call from viewport DrawCanvas. */
	UE_API void DrawPropertyGizmoLabels(FCanvas* Canvas, const FSceneView* SceneView) const;

	UE_API void OnConstructionViewModeChanged(const UE::Dataflow::IDataflowConstructionViewMode* NewConstructionViewMode);
	
	UE_API void OnDataflowAssetChanged();

	/** Get the toolkit evaluation mode */
	const EDataflowEditorEvaluationMode& GetEvaluationMode() const { return EvaluationMode; }

	/** Reset dataflow simulation */
	UE_API void ResetDataflowSimulation() const;
	
	/** Toggle dataflow simulation */
	UE_API void ToggleDataflowSimulation() const;
	
	/** Start dataflow simulation */
	UE_API void StartDataflowSimulation() const;

	/** Stop dataflow simulation */
	UE_API void StopDataflowSimulation() const;
	
	/** Step dataflow simulation */
	UE_API void StepDataflowSimulation() const;

	/** Pause dataflow simulation */
	UE_API void PauseDataflowSimulation() const;

	/** Check if the simulation can be toggled or not */
	bool HasSimulationManager() const;

	/** Check if the simulation is enabled or not */
	bool IsSimulationEnabled() const;

	/** Check if the simulation is disabled or not */
	bool IsSimulationDisabled() const;

protected:

	UDataflowEditor* DataflowEditor = nullptr;

	// List of dataflow actions callbacks
	UE_API void OnPropertyValueChanged(const FPropertyChangedEvent& PropertyChangedEvent);
	UE_API bool OnNodeVerifyTitleCommit(const FText& NewText, UEdGraphNode* GraphNode, FText& OutErrorMessage);
	UE_API void OnNodeTitleCommitted(const FText& InNewText, ETextCommit::Type InCommitType, UEdGraphNode* GraphNode);
	UE_API void OnNodeDoubleClicked(UEdGraphNode* ClickedNode);
	UE_API void OnNodeSelectionChanged(const TSet<UObject*>& NewSelection);
	UE_API void OnNodeInvalidated(UDataflow& DataflowAsset, FDataflowNode& Node);
	UE_API void OnNodeDeleted(const TSet<UObject*>& NewSelection);
	UE_API void OnAssetPropertyValueChanged(const FPropertyChangedEvent& PropertyChangedEvent);
	UE_API void OnConstructionViewSelectionChanged(const TArray<UPrimitiveComponent*>& SelectedComponents, const TArray<FDataflowBaseElement*>& SelectedElements);
	UE_API void OnSimulationViewSelectionChanged(const TArray<UPrimitiveComponent*>& SelectedComponents, const TArray<FDataflowBaseElement*>& SelectedElements);
	// Callback to remove the closed one from the listener views
	UE_API void OnTabClosed(TSharedRef<SDockTab> Tab);

	// Node evaluation
	UE_API void EvaluateTerminalNode(const FDataflowTerminalNode& TerminalNode);
	UE_API void EvaluateNode(const FDataflowNode* Node, const FDataflowOutput* Output, UE::Dataflow::FTimestamp& InOutTimestamp);
	UE_API void EvaluateGraph();
	UE_API void RefreshViewsIfNeeded(bool bForce = false);
	UE_API void OnNodeBeginEvaluate(const FDataflowNode* Node, const FDataflowOutput* Output);
	UE_API void OnNodeFinishEvaluate(const FDataflowNode* Node, const FDataflowOutput* Output);
	UE_API void OnOutputLogMessageTokenClicked(const FString TokenString);
	UE_API void OnContextLog(const UE::Dataflow::FContext::FLogMessage& InLogMessage);
	UE_API void RefreshLogMessageView();

private:
	
	// Spawning of all the additional tabs (viewport,details ones are coming from the base asset toolkit)
	UE_API TSharedRef<SDockTab> SpawnTab_GraphCanvas(const FSpawnTabArgs& Args);
	UE_API TSharedRef<SDockTab> SpawnTab_SubGraphTab(const FSpawnTabArgs& Args);
	UE_API TSharedRef<SDockTab> SpawnTab_NodeDetails(const FSpawnTabArgs& Args);
	UE_API TSharedRef<SDockTab> SpawnTab_SkeletonView(const FSpawnTabArgs& Args);
	UE_API TSharedRef<SDockTab> SpawnTab_OutlinerView(const FSpawnTabArgs& Args);
	UE_API TSharedRef<SDockTab> SpawnTab_SelectionView(const FSpawnTabArgs& Args);
	UE_API TSharedRef<SDockTab> SpawnTab_CollectionSpreadSheet(const FSpawnTabArgs& Args);
	UE_API TSharedRef<SDockTab> SpawnTab_AssetDetails(const FSpawnTabArgs& Args);
	UE_API TSharedRef<SDockTab> SpawnTab_SimulationViewport(const FSpawnTabArgs& Args);
	UE_API TSharedRef<SDockTab> SpawnTab_PreviewScene(const FSpawnTabArgs& Args);
	UE_API TSharedRef<SDockTab> SpawnTab_SimulationVisualization(const FSpawnTabArgs& Args);
	UE_API TSharedRef<SDockTab> SpawnTab_MembersWidget(const FSpawnTabArgs& Args);
	UE_API TSharedRef<SDockTab> SpawnTab_OutputLog(const FSpawnTabArgs& Args);
	UE_API TSharedRef<SDockTab> SpawnTab_SimulationTimeline(const FSpawnTabArgs& Args);

	// FTickableEditorObject interface
	UE_API virtual void Tick(float DeltaTime) override;
	virtual bool IsTickable() const override { return true; }
	UE_API virtual TStatId GetStatId() const override;

	// FBaseCharacterFXEditorToolkit interface
	UE_API virtual FEditorModeID GetEditorModeId() const override;
	UE_API virtual void InitializeEdMode(UBaseCharacterFXEditorMode* EdMode) override;
	UE_API virtual void CreateEditorModeUILayer() override;

	// FAssetEditorToolkit interface
	UE_API virtual bool OnRequestClose(EAssetEditorCloseReason InCloseReason) override;
	UE_API virtual void OnClose() override;
	UE_API virtual void PostInitAssetEditor() override;
	UE_API virtual void GetSaveableObjects(TArray<UObject*>& OutObjects) const override;
	UE_API virtual void OnAssetsSavedAs(const TArray<UObject*>& SavedObjects) override;
	UE_API virtual bool ShouldReopenEditorForSavedAsset(const UObject* Asset) const override;

	// FBaseAssetToolkit interface
	UE_API virtual void CreateWidgets() override;
	UE_API virtual AssetEditorViewportFactoryFunction GetViewportDelegate() override;
	UE_API virtual TSharedPtr<FEditorViewportClient> CreateEditorViewportClient() const override;
	UE_API virtual void CreateEditorModeManager() override;

	UE_API void UpdateGizmos();

	// FNotifyHook
	UE_API virtual void NotifyPreChange(class FEditPropertyChain* PropertyAboutToChange) override;

	// List of all the tab names ids that will be used to identify the editor widgets
	static UE_API const FName GraphCanvasTabId;
	static UE_API const FName SubGraphCanvasTabId;
	static UE_API const FName NodeDetailsTabId;
	static UE_API const FName SkeletonViewTabId;
	static UE_API const FName OutlinerViewTabId;
	static UE_API const FName SelectionViewTabId_1;
	static UE_API const FName SelectionViewTabId_2;
	static UE_API const FName SelectionViewTabId_3;
	static UE_API const FName SelectionViewTabId_4;
	static UE_API const FName CollectionSpreadSheetTabId_1;
	static UE_API const FName CollectionSpreadSheetTabId_2;
	static UE_API const FName CollectionSpreadSheetTabId_3;
	static UE_API const FName CollectionSpreadSheetTabId_4;
	static UE_API const FName SimulationViewportTabId;
	static UE_API const FName PreviewSceneTabId;
	static UE_API const FName SimulationVisualizationTabId;
	static UE_API const FName SimulationTimelineTabId;
	static UE_API const FName MembersWidgetTabId;
	static UE_API const FName OutputLogTabId;

	// List of all the widgets shared ptr that will be built in the editor
	TSharedPtr<SDataflowConstructionViewport> DataflowConstructionViewport;
	TSharedPtr<SDataflowSimulationViewport> DataflowSimulationViewport;
	TSharedPtr<SDataflowGraphEditor> GraphEditor;
	TSharedPtr<SDockTab> GraphEditorTab;
	TSharedPtr<IStructureDetailsView> NodeDetailsEditor;
	TSharedPtr<SDataflowMembersWidget> MembersWidget;
	TSharedPtr<UE::Dataflow::FDataflowNodeDetailExtensionHandler> NodeDetailsExtensionHandler;
	TSharedPtr<FDataflowSkeletonView> SkeletonEditorView;
	TSharedPtr<FDataflowOutlinerView> DataflowOutlinerView;
	TSharedPtr<IDetailsView> AssetDetailsEditor;
	TSharedPtr<FDataflowSelectionView> DataflowSelectionView_1;
	TSharedPtr<FDataflowSelectionView> DataflowSelectionView_2;
	TSharedPtr<FDataflowSelectionView> DataflowSelectionView_3;
	TSharedPtr<FDataflowSelectionView> DataflowSelectionView_4;
	TSharedPtr<FDataflowCollectionSpreadSheet> DataflowCollectionSpreadSheet_1;
	TSharedPtr<FDataflowCollectionSpreadSheet> DataflowCollectionSpreadSheet_2;
	TSharedPtr<FDataflowCollectionSpreadSheet> DataflowCollectionSpreadSheet_3;
	TSharedPtr<FDataflowCollectionSpreadSheet> DataflowCollectionSpreadSheet_4;
	TSharedPtr<SWidget> AdvancedPreviewSettingsWidget;
	TSharedPtr<SWidget> SimulationVisualizationWidget;
	TSharedPtr<SDataflowSimulationTimeline> SimulationTimelineWidget;
	TSharedPtr<FDataflowOutputLog> DataflowOutputLog;

	/** Customize preview scene with editor/terminal contents */
	UE_API TSharedRef<class IDetailCustomization> CustomizePreviewSceneDescription() const;

	UE_API TSharedRef<class IDetailCustomization> CustomizeAssetViewer() const;

	// Utility factory functions to build the widgets
	UE_API TSharedRef<SDataflowGraphEditor> CreateGraphEditorWidget(UEdGraph* GraphToEdit, TSharedPtr<IStructureDetailsView> PropertiesEditor);
    UE_API TSharedPtr<IDetailsView> CreateAssetDetailsEditorWidget(const TArray<UObject*>& ObjectsToEdit);
	UE_API TSharedPtr<SWidget> CreateSimulationVisualizationWidget();
    UE_API TSharedPtr<IStructureDetailsView> CreateNodeDetailsEditorWidget(UObject* ObjectToEdit);
	UE_API TSharedPtr<SDataflowMembersWidget> CreateDataflowMembersWidget();
	UE_API TSharedRef<SGraphEditor> CreateSubGraphEditorWidget(TSharedRef<FTabInfo> InTabInfo, UDataflowSubGraph* InGraph);

	UE_API void AddEvaluationWidget(FToolMenuSection& Section);
	UE_API TSharedRef<SWidget> GenerateEvaluationOptionsMenu();
	UE_API FSlateIcon GetEvaluationStatusImage() const;
	UE_API bool IsGraphDirty() const;
	UE_API bool IsEvaluateButtonEnabled() const;

	UE_API void SetEvaluateGraphMode(EDataflowEditorEvaluationMode Mode);
	UE_API void ClearGraphCache();
	UE_API bool CanClearGraphCache() const;
	UE_API void TogglePerfData();
	UE_API bool IsPerfDataEnabled() const;
	UE_API void ToggleAsyncEvaluation();
	UE_API bool IsAsyncEvaluationEnabled() const;

	/** Create the simulation viewport client */
	UE_API void CreateSimulationViewportClient();

	UE_API void SetDataflowPathFromNodeAndOutput(const FDataflowNode* Node, const FDataflowOutput* Output, FDataflowPath& OutPath) const;

	UE_API void RegisterContextHandlers();
	UE_API void UnregisterContextHandlers();

	/** Update the debug draw based on a change of currently selected or pinned nodes */
	UE_API void UpdateDebugDraw();

	/** reset the editor. Used when the dataflow graph as changed */
	void ResetEditor();

	/** This adds the widget to switch from embdedded or external dataflow as well as read - only lock when applicable */
	void AddDataflowActionWidget(FToolMenuSection& Section);

	/** return true if the Dataflow is not an asset and is an object embedded into the owner asset */
	bool IsEmbeddedDataflow() const;

	/** return true if the Dataflow is a separate asset or no dataflow is assigned to the owner asset */
	bool IsExternalDataflow() const;

	/** return true if the bound asset is the reference asset or if the reference asset is null */
	bool IsReferenceAssetNullOrSameAsBoundAsset() const;

	/** Get Reference asset from the dataflow graph */
	UObject* GetReferenceAsset() const;

	/** return true the current dataflow can be edited - condition are must be External and match reference asset (if reference asset is defined)*/
	bool AllowExternalDataflowEditing() const;

	/** whether the Lock/Unlock editing button is visible or not */
	bool IsEditingLockButtonVisible() const;

	bool IsEditable() const;

	/** Duplicate the current external dataflow and embed it into the owner asset */
	void ConvertToEmbeddedDataflow();

	/** 
	* Duplicate the embedded dataflow object and store it into a separate asset  
	* this methid is interactive and will prompt a dialog for the user to choose a path to save the duplicated Dataflow to 
	*/
	void ConvertToExternalDataflow();

	void ToggleDataflowEditingLock();

	void UpdateViewsFromNode(UDataflowEdNode* Node = nullptr);
	static TSet<TObjectPtr<UDataflowEdNode>> FilterDataflowEdNodesFromSet(const TSet<UObject*>& Set);
	static const TObjectPtr<UDataflowEdNode> GetOnlyFromSet(const TSet<TObjectPtr<UDataflowEdNode>>& Set);

	// List of editor commands used  for the dataflow asset
	TSharedPtr<FUICommandList> GraphEditorCommands;

	// List of selection view / collection spreadsheet widgets that are listening to any changed in the graph
	TArray<IDataflowViewListener*> ViewListeners;

	// Graph delegates used to update the UI
	FDelegateHandle OnSelectionChangedMulticastDelegateHandle;
    FDelegateHandle OnNodeDeletedMulticastDelegateHandle;
	FDelegateHandle OnEvaluateSelectedNodesDelegateHandle;
    FDelegateHandle OnFinishedChangingPropertiesDelegateHandle;
	FDelegateHandle OnFinishedChangingAssetPropertiesDelegateHandle;
	FDelegateHandle OnConstructionSelectionChangedDelegateHandle;
	FDelegateHandle OnSimulationSelectionChangedDelegateHandle;
	FDelegateHandle OnSimulationSceneChangedDelegateHandle;

	// Delegates to communicate with Context
	FDelegateHandle OnNodeBeginEvaluateMulticastDelegateHandle;
	FDelegateHandle OnNodeFinishEvaluateMulticastDelegateHandle;
	FDelegateHandle OnOutputLogMessageTokenClickedDelegateHandle;

	FDelegateHandle OnContextLogDelegateHandle;

	// The currently selected set of dataflow nodes.
	TSet<TObjectPtr<UDataflowEdNode>> SelectedDataflowNodes;
	TSet<TObjectPtr<UEdGraphNode_Comment>> SelectedComments;

	/** Manages viewport gizmos for GizmoType-annotated properties on the currently selected node. */
	TObjectPtr<UDataflowPropertyGizmoManager> PropertyGizmoManager;

	/** Pointer to the construction viewport scene. Note this is an alias of ObjectScene in FBaseCharacterFXEditorToolkit but with the specific type */
	TSharedPtr<FDataflowConstructionScene> ConstructionScene;

	/** PreviewScene showing the objects being simulated */
	TSharedPtr<FDataflowSimulationScene> SimulationScene;

	TSharedPtr<FDataflowSimulationSceneProfileIndexStorage> SimulationSceneProfileIndexStorage;

	/** The editor mode manager used by the simulation preview scene */
	TSharedPtr<FEditorModeTools> SimulationModeManager;

	/** Simulation tab content */
	TSharedPtr<class FEditorViewportTabContent> SimulationTabContent;

	/** Simulation viewport delegate */
	AssetEditorViewportFactoryFunction SimulationViewportDelegate;

	/** Simulation Viewport client */
	TSharedPtr<FDataflowSimulationViewportClient> SimulationViewportClient;

	/** Simulation default layout */
	TSharedPtr<FTabManager::FLayout> SimulationDefaultLayout;
	
	/** Simulation default layout */
	TSharedPtr<FTabManager::FLayout> ConstructionDefaultLayout;

	/** Cached value of the p.Dataflow.EnableGraphEval cvar, to avoid calling FindConsoleVariable too often */
	bool bDataflowEnableGraphEval;

	/** Allow editing of external dataflow */
	bool bAllowExternalDataflowEditing = false;

	/** Cached value from the editor evaluation settings */
	bool bAllowEvaluationInPIE = false;

	EDataflowEditorEvaluationMode EvaluationMode;

	/** Delegate for updating the cached value of p.Dataflow.EnableGraphEval */
	FDelegateHandle GraphEvalCVarChangedDelegateHandle;

	TWeakPtr<SDataflowGraphEditor> ActiveSubGraphEditorWeakPtr;

	/** Document tracker for dynamic tabs ( like subgraphs ) */
	TSharedPtr<FDocumentTracker> DocumentManager;

	TWeakPtr<SDockTab> WeakOutputLogDockTab;

	TSet<FGuid> NodesToEvaluateOnTick;

	/** relevant message to display */
	TSharedRef<UE::Dataflow::Private::FLogMessageStorage> LogMessages;

	bool bViewsNeedRefresh = false;

	FString DebugDrawOverlayString;

	/** Simulation Binding to control the simulation scene from the timeline */
	TSharedPtr<FDataflowSimulationBinding> SimulationBinding;
};

#undef UE_API
