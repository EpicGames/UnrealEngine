// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "EditorViewportClient.h"
#include "InputBehaviorSet.h"

#define UE_API DATAFLOWEDITOR_API

struct FDataflowBaseElement;
class FEditorModeTools;
class SEditorViewport;
class HHitProxy;
class FDataflowPreviewSceneBase;

class FDataflowEditorViewportClientBase : public FEditorViewportClient, public IInputBehaviorSource
{
public:
	using Super = FEditorViewportClient;

	UE_API FDataflowEditorViewportClientBase(FEditorModeTools* InModeTools, TWeakPtr<FDataflowPreviewSceneBase> InPreviewScene,  const bool bCouldTickScene, const TWeakPtr<SEditorViewport> InEditorViewportWidget = nullptr);
	
	UE_API virtual ~FDataflowEditorViewportClientBase();

protected:

	virtual void OnViewportClicked(HHitProxy* HitProxy) = 0;

	// FGCObject
	UE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;

	//~ Begin FEditorViewportClient interface
	UE_API virtual void Draw(const FSceneView* View, FPrimitiveDrawInterface* PDI) override;
	//~ End FEditorViewportClient interface

	/** Dataflow preview scene from the toolkit */
	TWeakPtr<FDataflowPreviewSceneBase> DataflowPreviewScene;

	/** Behaviors defined by this base class */
	TArray<TObjectPtr<UInputBehavior>> BaseBehaviors;

	/** All behaviors available to the current viewport (subclasses can add to this set) */
	TObjectPtr<UInputBehaviorSet> BehaviorSet;

protected :

	/** Get all the scene selected elements */
	UE_API void GetSelectedElements(HHitProxy* HitProxy, TArray<FDataflowBaseElement*>& SelectedElements) const;

private:
	/** Handle the focus request onto the bounding box */
	UE_API void HandleFocusRequest(const FBox& BoundingBox);

	// IInputBehaviorSource
	UE_API virtual const UInputBehaviorSet* GetInputBehaviors() const override;

	/** Called from constructor/destructor to manage callbacks when settings change */
	UE_API void RegisterDelegates();
	UE_API void DeregisterDelegates();

	/** Handle for callback when preview scene profile settings change */
	FDelegateHandle OnAssetViewerSettingsChangedDelegateHandle;

	/** Handle for callback when preview scene focus request is triggered */
	FDelegateHandle OnFocusRequestDelegateHandle;
};

#undef UE_API
