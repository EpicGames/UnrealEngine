// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Dataflow/DataflowSelection.h"
#include "DetailLayoutBuilder.h"
#include "Delegates/Delegate.h"
#include "UObject/NameTypes.h"
#include "Styling/StarshipCoreStyle.h"
#include "Widgets/Views/STreeView.h"

class FGeometryCollection;
class FGeometryCollectionFacesTreeItem;
class FGeometryCollectionFacesTreeItemFace;
class SFacesOutliner;
struct FManagedArrayCollection;

typedef TArray<TSharedPtr<FGeometryCollectionFacesTreeItem>> FGeometryCollectionFacesTreeItemList;
typedef TSharedPtr<FGeometryCollectionFacesTreeItem> FGeometryCollectionFacesTreeItemPtr;
typedef TSharedPtr<class FGeometryCollectionFacesTreeItemFace> FGeometryCollectionFacesTreeItemFacePtr;

class FGeometryCollectionFacesTreeItem : public TSharedFromThis<FGeometryCollectionFacesTreeItem>
{
public:
	virtual ~FGeometryCollectionFacesTreeItem() = default;
	virtual TSharedRef<ITableRow> MakeTreeRowWidget(const TSharedRef<STableViewBase>& InOwnerTable, bool bIsPinned = false) = 0;
	virtual void GetChildren(FGeometryCollectionFacesTreeItemList& OutChildren) = 0;

	virtual int32 GetFaceIndex() const { return INDEX_NONE; }

	void GenerateContextMenu(UToolMenu* Menu, SFacesOutliner& Outliner);
};

class FGeometryCollectionFacesTreeItemGeometry : public FGeometryCollectionFacesTreeItem
{
public:
	FGeometryCollectionFacesTreeItemGeometry(TSharedPtr<const FManagedArrayCollection> InCollection, int32 InGeoemtryIdx, TSharedPtr<STreeView<FGeometryCollectionFacesTreeItemPtr>> InTreeView)
		: Collection(InCollection)
		, GeometryIdx(InGeoemtryIdx)
		, TreeView(InTreeView)
	{
		RegenerateChildren();
	}

	virtual ~FGeometryCollectionFacesTreeItemGeometry() = default;

	// FGeometryCollectionFacesTreeItem interface
	virtual TSharedRef<ITableRow> MakeTreeRowWidget(const TSharedRef<STableViewBase>& InOwnerTable, bool bIsPinned = false);
	virtual void GetChildren(FGeometryCollectionFacesTreeItemList& OutChildren) override;

	FGeometryCollectionFacesTreeItemPtr GetItemFromFaceIndex(int32 InFaceIndex) const;

	void GetChildrenForFace(FGeometryCollectionFacesTreeItemFace& FaceItem, FGeometryCollectionFacesTreeItemList& OutChildren) const;
	bool HasChildrenForFace(const FGeometryCollectionFacesTreeItemFace& FaceItem) const;

	void ExpandAll();
	void RegenerateChildren();
	void RequestTreeRefresh();

	TSharedPtr<const FManagedArrayCollection> GetCollection() { return Collection; }

	bool IsValid() const;

	// Mark item as unused/invalid; helpful because slate defers destroying tree items and can still run callbacks on them until tick
	void Invalidate()
	{
		bInvalidated = true;
	}

private:
	bool FilterBoneIndex(int32 BoneIndex) const;

	TSharedPtr<const FManagedArrayCollection> Collection;
	int32 GeometryIdx;

	TSharedPtr<STreeView<FGeometryCollectionFacesTreeItemPtr>> TreeView;

	// The direct children under this component
	TArray<FGeometryCollectionFacesTreeItemPtr> ChildItems;
	TMap<int32, FGeometryCollectionFacesTreeItemPtr> ItemsByFaceIndex;
	int32 RootIndex;

	// track whether the item has been explicitly invalidated
	bool bInvalidated = false;
};

class FGeometryCollectionFacesTreeItemFace : public FGeometryCollectionFacesTreeItem
{
public:
	FGeometryCollectionFacesTreeItemFace(const int32 InFaceIndex, FGeometryCollectionFacesTreeItemGeometry* InParentCollectionItem)
		: FaceIndex(InFaceIndex)
		, ParentCollectionItem(InParentCollectionItem)
		, ItemColor(FSlateColor::UseForeground())
	{}

	// FGeometryCollectionFacesTreeItem interface
	TSharedRef<ITableRow> MakeTreeRowWidget(const TSharedRef<STableViewBase>& InOwnerTable, bool bIsPinned = false);
	TSharedRef<SWidget> MakeIndexColumnWidget() const;
	TSharedRef<SWidget> MakeEmptyColumnWidget() const;

	virtual void GetChildren(FGeometryCollectionFacesTreeItemList& OutChildren) override;
	bool IsValidFace() const;
	virtual int32 GetFaceIndex() const override { return FaceIndex; }
	FSlateColor GetItemColor() const { return ItemColor; }
	bool HasChildren() const;
	TSharedPtr<const FManagedArrayCollection> GetCollection() const;

protected:

private:
	void UpdateItemColorFromCollection();

	const int32 FaceIndex;
	FGeometryCollectionFacesTreeItemGeometry* ParentCollectionItem;
	FSlateColor ItemColor;
};

class SFacesOutlinerRow : public SMultiColumnTableRow<FGeometryCollectionFacesTreeItemFacePtr>
{
protected:
	FGeometryCollectionFacesTreeItemFacePtr Item;

public:
	SLATE_BEGIN_ARGS(SFacesOutlinerRow) {}
	SLATE_END_ARGS()

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

	void Construct(const FArguments& InArgs, TSharedRef<STableViewBase> OwnerTableView, FGeometryCollectionFacesTreeItemFacePtr InItemToEdit)
	{
		Item = InItemToEdit;
		SMultiColumnTableRow<FGeometryCollectionFacesTreeItemFacePtr>::Construct(
			FSuperRowType::FArguments()
			.Style(&FStarshipCoreStyle::GetCoreStyle().GetWidgetStyle<FTableRowStyle>("TableView.AlternatingRow"))
			, OwnerTableView);
	}
};

class SFacesOutliner : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SFacesOutliner) {}
		SLATE_ARGUMENT(TSharedPtr<SScrollBar>, ExternalVerticalScrollBar)
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs);

	void RegenerateItems();
	void RegenerateHeader();

	TSharedRef<ITableRow> MakeTreeRowWidget(FGeometryCollectionFacesTreeItemPtr InInfo, const TSharedRef<STableViewBase>& OwnerTable);
	TSharedRef<ITableRow> OnGeneratePinnedRowWidget(FGeometryCollectionFacesTreeItemPtr InItem, const TSharedRef<STableViewBase>& InOwnerTable, bool bPinned);
	void OnGetChildren(TSharedPtr<FGeometryCollectionFacesTreeItem> InInfo, TArray< TSharedPtr<FGeometryCollectionFacesTreeItem> >& OutChildren);
	TSharedPtr<SWidget> OnOpenContextMenu();

	void UpdateGeometryCollection();
	void SetCollection(const TSharedPtr<const FManagedArrayCollection>& InCollection);
	int32 GetBoneSelectionCount() const;

	void ExpandAll();
	void ExpandRecursive(TSharedPtr<FGeometryCollectionFacesTreeItem> TreeItem, bool bInExpansionState) const;

private:
	void OnSelectionChanged(FGeometryCollectionFacesTreeItemPtr Item, ESelectInfo::Type SelectInfo);

	TSharedPtr<STreeView<FGeometryCollectionFacesTreeItemPtr>> TreeView;
	TSharedPtr<SHeaderRow> HeaderRowWidget;
	TArray<TSharedPtr<FGeometryCollectionFacesTreeItemGeometry>> RootNodes;
	bool bPerformingSelection;
	TSharedPtr<const FManagedArrayCollection> Collection;
};

