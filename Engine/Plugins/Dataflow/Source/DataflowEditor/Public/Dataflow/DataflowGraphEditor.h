// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Dataflow/DataflowEditorCommands.h"
#include "Dataflow/DataflowSEditorInterface.h"
#include "EdGraphUtilities.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/Commands/UICommandList.h"
#include "GraphEditor.h"
#include "GraphEditorActions.h"
#include "Layout/SlateRect.h"
#include "NodeFactory.h"
#include "SGraphNode.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Layout/SConstraintCanvas.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/SWidget.h"

#define UE_API DATAFLOWEDITOR_API


class FDataflowEditorToolkit;
class FDataflowGraphEditorNodeFactory;
class FDataflowSNodeFactory;
class SGraphEditorActionMenu;
class UDataflow;
class UDataflowEditor;
struct FDataflowConnection;
namespace UE::Dataflow {
	class FContext;
	namespace Private
	{
		struct FLogMessageStorage;
	}
}
/**
 * The SDataflowGraphEditpr class is a specialization of SGraphEditor
 * to display and manipulate the actions of a Dataflow asset
 * 
 * see(SDataprepGraphEditor for reference)
 */
class SDataflowGraphEditor : public SGraphEditor, public FGCObject, public FDataflowSEditorInterface
{
public:

	using FGraphEvaluationCallback = TFunction<void(const FDataflowNode*, const FDataflowOutput*)>;
	using FOnDragDropEventCallback = TFunction<void(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)>;
	using FIsEditableCallback = TFunction<bool()>;

	SLATE_BEGIN_ARGS(SDataflowGraphEditor)
		: _AdditionalCommands(static_cast<FUICommandList*>(nullptr))
		, _GraphToEdit(nullptr)
	{}

	SLATE_ARGUMENT(TSharedPtr<FUICommandList>, AdditionalCommands)
	SLATE_ATTRIBUTE(FGraphAppearanceInfo, Appearance)
	SLATE_ARGUMENT_DEFAULT(UEdGraph*, GraphToEdit) = nullptr;
	SLATE_ARGUMENT(FGraphEditorEvents, GraphEvents)
	SLATE_ARGUMENT(TSharedPtr<IStructureDetailsView>, DetailsView)
	SLATE_ARGUMENT(FGraphEvaluationCallback, EvaluateGraph)
	SLATE_ARGUMENT(FOnDragDropEventCallback, OnDragDropEvent)
	SLATE_ARGUMENT(FIsEditableCallback, IsEditable)
	SLATE_ARGUMENT_DEFAULT(UDataflowEditor*, DataflowEditor) = nullptr;
	SLATE_END_ARGS()

	// This delegate exists in SGraphEditor but it is not multicast, and we are going to bind it to OnSelectedNodesChanged().
	// This new multicast delegate will be broadcast from the OnSelectedNodesChanged handler in case another class wants to be notified.
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnSelectionChangedMulticast, const FGraphPanelSelectionSet&)
	FOnSelectionChangedMulticast OnSelectionChangedMulticast;

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnNodeDeletedMulticast, const FGraphPanelSelectionSet&)
	FOnNodeDeletedMulticast OnNodeDeletedMulticast;

	UE_API virtual ~SDataflowGraphEditor();

	// SWidget overrides
	UE_API virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	UE_API virtual FReply OnKeyUp(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	UE_API virtual FReply OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	UE_API virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	UE_API bool IsControlDown() const;
	UE_API bool IsAltDown() const;
	//virtual void OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	//virtual void OnDragLeave(const FDragDropEvent& DragDropEvent) override;
	// end SWidget

	/** */
	UE_API void Construct(const FArguments& InArgs, UObject* AssetOwner);

	/** return whether or not the graph is editable */
	UE_API bool IsEditable() const;

	/** */
	UE_API void EvaluateNode();

	/** */
	UE_API void FreezeNodes();

	/** */
	UE_API void UnfreezeNodes();

	/** */
	UE_API void DeleteNode();

	/** */
	UE_API void OnSelectedNodesChanged(const TSet<UObject*>& NewSelection);

	/** */
	UE_API void CreateComment();

	/** */
	UE_API void AlignTop();

	/** */
	UE_API void AlignMiddle();

	/** */
	UE_API void AlignBottom();

	/** */
	UE_API void AlignLeft();

	/** */
	UE_API void AlignCenter();

	/** */
	UE_API void AlignRight();

	/** */
	UE_API void StraightenConnections();

	/** */
	UE_API void DistributeHorizontally();

	/** */
	UE_API void DistributeVertically();

	/** */
	UE_API void ToggleEnabledState();

	/** */
	UE_API void DuplicateSelectedNodes();

	/** */
	UE_API void ZoomToFitGraph();

	/** */
	UE_API void CopySelectedNodes();

	/** */
	UE_API void CutSelectedNodes();

	/** */
	UE_API void PasteSelectedNodes();

	/** */
	UE_API void RenameNode();
	UE_API bool CanRenameNode() const;

	/** Add a new variable for this dataflow graph */
	UE_API void AddNewVariable() const;

	/** Add a new SubGraph for this dataflow graph */
	UE_API void AddNewSubGraph() const;

	SGraphEditor* GetGraphEditor() { return (SGraphEditor*)this; }

	/** FGCObject interface */
	UE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override { return TEXT("SDataflowGraphEditor"); }

	//~ Begin FDataflowSEditorInterface 
	UE_API virtual TSharedPtr<UE::Dataflow::FContext> GetDataflowContext() const override;
	UE_API virtual void OnRenderToggleChanged(UDataflowEdNode* UpdatedEdNode) const override;
	UE_API virtual FText GetMessageText(const UDataflowEdNode& Node) const override;
	UE_API virtual EMessageSeverity::Type GetMessageMaxSeverity(const UDataflowEdNode& Node) const override;
	//~ End FDataflowSEditorInterface 

	const TSharedPtr<FUICommandList> GetCommands() const { return GraphEditorCommands; }

	/** Return the currently selected editor. Only valid for the duration of the OnSelectedNodesChanged callback where the property editor is updated. */
	static const TWeakPtr<SDataflowGraphEditor>& GetSelectedGraphEditor() { return SelectedGraphEditor; }

	static const TWeakPtr<SDataflowGraphEditor>& GetLastActionMenuGraphEditor() { return LastActionMenuGraphEditor; }

	bool GetFilterActionMenyByAssetType() const { return bFilterActionMenyByAssetType; }

	void SetLogMessageStorage(TSharedPtr<UE::Dataflow::Private::FLogMessageStorage> InLogMessages);

private:
	/** Add an additional option pin to all selected Dataflow nodes for those that overrides the AddPin function. */
	UE_API void OnAddOptionPin();
	/** Return whether all currently selected Dataflow nodes can execute the AddPin function. */
	UE_API bool CanAddOptionPin() const;

	/** Remove an option pin from all selected Dataflow nodes for those that overrides the RemovePin function. */
	UE_API void OnRemoveOptionPin();
	/** Return whether all currently selected Dataflow nodes can execute the RemovePin function. */
	UE_API bool CanRemoveOptionPin() const;

	UE_API void OnStartWatchingPin();
	UE_API bool CanStartWatchingPin() const;
	UE_API void OnStopWatchingPin();
	UE_API bool CanStopWatchingPin() const;
	UE_API void OnPromoteToVariable();
	UE_API bool CanPromoteToVariable() const;

	UE_API bool GetPinVisibility(SGraphEditor::EPinVisibility InVisibility) const;

	/** Create a widget to display an overlaid message in the graph editor panel */
	UE_API void InitGraphEditorMessageBar();

	/** Create a widget to display the progress of the evaluation of the graph */
	UE_API void InitEvaluationProgressBar();

	/** Text for overlay message in graph editor panel */
	UE_API FText GetGraphEditorOverlayText() const;

	/** get the dataflow asset from the edGraph being edited */
	UE_API UDataflow* GetDataflowAsset() const;

	FOnDragDropEventCallback OnDragDropEventCallback;
	FGraphEvaluationCallback EvaluateGraphCallback;
	FIsEditableCallback IsEditableCallback;

	UE_API FActionMenuContent OnCreateActionMenu(UEdGraph* Graph, const FVector2f& Position, const TArray<UEdGraphPin*>& DraggedPins, bool bAutoExpandActionMenu, SGraphEditor::FActionMenuClosed OnClosed);
	UE_API void OnActionMenuFilterByAssetTypeChanged(ECheckBoxState NewState, const TWeakPtr<SGraphEditorActionMenu> WeakActionMenu);
	UE_API ECheckBoxState IsActionMenuFilterByAssetTypeChecked() const;

	/** The asset that ownes this dataflow graph */
	TWeakObjectPtr<UObject> AssetOwner;

	/** The dataflow asset associated with this graph */
	TWeakObjectPtr<UEdGraph> EdGraphWeakPtr;

	/** Command list associated with this graph editor */
	TSharedPtr<FUICommandList> GraphEditorCommands;

	/** The details view that responds to this widget. */
	TSharedPtr<IStructureDetailsView> DetailsView;

	/** Factory to create the associated SGraphNode classes for Dataprep graph's UEdGraph classes */
	static UE_API TSharedPtr<FDataflowGraphEditorNodeFactory> NodeFactory;

	/** The current graph editor when the selection callback is invoked. */
	static UE_API TWeakPtr<SDataflowGraphEditor> SelectedGraphEditor;

	/** The last graph editor used when a action context  menu was brough up in the graph. */
	static UE_API TWeakPtr<SDataflowGraphEditor> LastActionMenuGraphEditor;

	/** Editor for the content */
	UDataflowEditor* DataflowEditor = nullptr;

	bool VKeyDown = false;
	bool LeftControlKeyDown = false;
	bool RightControlKeyDown = false;
	bool LeftAltKeyDown = false;
	bool RightAltKeyDown = false;

	bool bFilterActionMenyByAssetType = true;

	FDelegateHandle CVarChangedDelegateHandle;
	TSharedPtr<SWidget> MessageBar;
	TSharedPtr<SWidget> EvaluationProgressBar;
	FText MessageBarText;

	TSharedPtr<UE::Dataflow::Private::FLogMessageStorage> LogMessages;
};

#undef UE_API
