// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GraphEditorDragDropAction.h"
#include "Textures/SlateIcon.h"

#include "DataflowGraphSchemaAction.generated.h"

#define UE_API DATAFLOWEDITOR_API

struct FPropertyBagPropertyDesc;
class IDataflowInstanceInterface;
class UDataflow;
class UEdGraphNode;
class UEdGraphPin;
struct FDataflowNode;

namespace UE::Dataflow
{
	/* Enums to use when grouping action in widgets 
	* see SDataflowMembersWidget
	*/
	enum class ESchemaActionSectionID: int32
	{
		NONE = 0,
		SUBGRAPHS,
		VARIABLES,
	};
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
* Action to add a node to the graph
*/
USTRUCT()
struct FAssetSchemaAction_Dataflow_CreateNode_DataflowEdNode : public FEdGraphSchemaAction
{
	GENERATED_USTRUCT_BODY();

public:
	FAssetSchemaAction_Dataflow_CreateNode_DataflowEdNode() : FEdGraphSchemaAction() {}

	FAssetSchemaAction_Dataflow_CreateNode_DataflowEdNode(const FName& InName, const FName& InType, FText InNodeCategory, FText InMenuDesc, FText InToolTip, FText InKeywords, FName InIconName);
	FAssetSchemaAction_Dataflow_CreateNode_DataflowEdNode(const FName& InName, const FName& InType, FText InNodeCategory, FText InMenuDesc, FText InToolTip, FText InKeywords, const FSlateIcon& InIcon);

	static UE_API TSharedPtr<FAssetSchemaAction_Dataflow_CreateNode_DataflowEdNode> CreateAction(const UEdGraph* ParentGraph, const FName& NodeTypeName, const FName& InOverrideNodeName = NAME_None);

	using FEdGraphSchemaAction::PerformAction; // Prevent hiding of deprecated base class function with FVector2D
	UE_API virtual UEdGraphNode* PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;

	virtual FSlateBrush const* GetPaletteIcon() const override;

	FName NodeName;
	FName NodeTypeName;
	FSlateIcon Icon;
};

/**
* Action that refers to a dataflow variable
* It is used to display the variables in the SDataflowMembersWidget
*/
USTRUCT()
struct FEdGraphSchemaAction_DataflowVariable : public FEdGraphSchemaAction
{
	GENERATED_USTRUCT_BODY()

public:
	FEdGraphSchemaAction_DataflowVariable();

	FEdGraphSchemaAction_DataflowVariable(UDataflow* InDataflowAsset, const FPropertyBagPropertyDesc& PropertyDesc);

	// Simple type info
	static FName StaticGetTypeId() { static FName Type("FEdGraphSchemaAction_DataflowVariable"); return Type; }
	virtual FName GetTypeId() const override { return StaticGetTypeId(); }

	const FString& GetVariableName() const
	{
		return VariableName;
	}

	FName GetFullVariableName() const
	{
		return FullVariableName;
	}

	const FEdGraphPinType& GetVariableType() const
	{
		return VariableType;
	}

	// FEdGraphSchemaAction interface
	virtual bool IsAVariable() const { return true; }
	virtual UEdGraphNode* PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
	// End of FEdGraphSchemaAction interface

	bool CanRenameItem(FText NewName) const;
	void RenameItem(FText NewName);
	void SetVariableType(const FEdGraphPinType& PinType);

	void CopyItemToClipboard();
	void PasteItemFromClipboard();
	void DeleteItem();
	void DuplicateItem();

private:
	static FString CategoryFromFullName(FName FullName);
	static FString NameFromFullName(FName FullName);

private:
	/**
	* Name of the variable
	* the name can contain a category is using a separating |
	* example : Category.VariableName
	*/
	FName FullVariableName;

	/**
	* Variable short name ( extracted from FullName )
	*/
	FString VariableName;

	/**
	* Variable category name ( extracted from FullName )
	*/
	FString VariableCategory;

	/**
	* type of the variable (as a Pin type)
	*/
	FEdGraphPinType VariableType;

	/** Dataflow asset associated with this action */
	TWeakObjectPtr<UDataflow> DataflowAssetWeakPtr;
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
* Dataflow variable drag and drop action
* see FEdGraphSchemaAction_DataflowVariable
*/
struct FGraphSchemaActionDragDropAction_DataflowVariable : public FGraphSchemaActionDragDropAction
{
public:
	DRAG_DROP_OPERATOR_TYPE(FGraphSchemaActionDragDropAction_DataflowVariable, FGraphSchemaActionDragDropAction)

	static TSharedRef<FGraphSchemaActionDragDropAction_DataflowVariable> New(TSharedPtr<FEdGraphSchemaAction_DataflowVariable>& InAction);

private:
	FGraphSchemaActionDragDropAction_DataflowVariable();

	TSharedPtr<FEdGraphSchemaAction_DataflowVariable> VariableAction;
};


////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
* Action that refers to a dataflow subgraph
* It is used to display the variables in the SDataflowMembersWidget
*/
USTRUCT()
struct FEdGraphSchemaAction_DataflowSubGraph : public FEdGraphSchemaAction
{
	GENERATED_USTRUCT_BODY()

public:
	FEdGraphSchemaAction_DataflowSubGraph();

	FEdGraphSchemaAction_DataflowSubGraph(UDataflow* InDataflowAsset, const FGuid& InSubGraphGuid);

	// Simple type info
	static FName StaticGetTypeId() { static FName Type("FEdGraphSchemaAction_DataflowSubGraph"); return Type; }
	virtual FName GetTypeId() const override { return StaticGetTypeId(); }

	DATAFLOWEDITOR_API const FName GetSubGraphName() const;
	DATAFLOWEDITOR_API bool IsForEachSubGraph() const;
	DATAFLOWEDITOR_API void SetForEachSubGraph(bool bValue);

	// FEdGraphSchemaAction interface
	virtual bool IsAVariable() const { return false; }
	virtual UEdGraphNode* PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
	// End of FEdGraphSchemaAction interface

	bool CanRenameItem(FText NewName) const;
	void RenameItem(FText NewName);

	void CopyItemToClipboard();
	void PasteItemFromClipboard();
	void DeleteItem();
	void DuplicateItem();

private:
	/** Guid of the subgraph */
	FGuid SubGraphGuid;

	/** Dataflow asset associated with this action */
	TWeakObjectPtr<UDataflow> DataflowAssetWeakPtr;
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
* Dataflow variable drag and drop action
* see FEdGraphSchemaAction_DataflowSubGraph
*/
struct FGraphSchemaActionDragDropAction_DataflowSubGraph : public FGraphSchemaActionDragDropAction
{
public:
	DRAG_DROP_OPERATOR_TYPE(FGraphSchemaActionDragDropAction_DataflowSubGraph, FGraphSchemaActionDragDropAction)

	static TSharedRef<FGraphSchemaActionDragDropAction_DataflowSubGraph> New(TSharedPtr<FEdGraphSchemaAction_DataflowSubGraph>& InAction);

private:
	FGraphSchemaActionDragDropAction_DataflowSubGraph();

	TSharedPtr<FEdGraphSchemaAction_DataflowSubGraph> SubGraphAction;
};

#undef UE_API
