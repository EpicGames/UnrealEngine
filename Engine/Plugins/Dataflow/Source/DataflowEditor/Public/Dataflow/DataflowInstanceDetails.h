// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "IDetailCustomization.h"
#include "IDetailPropertyChildrenCustomizationHandler.h"

class IPropertyHandle;
class USmartObjectDefinition;
class IPropertyUtilities;

/**
 * Type customization for FDataflowVariableOverrides.
 */
class FDataflowVariableOverridesDetails : public IPropertyTypeCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> InStructPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& InChildrenBuilder, IPropertyTypeCustomizationUtils& InCustomizationUtils) override;
};

class FDataflowInstanceDetailCustomization : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	DATAFLOWEDITOR_API FDataflowInstanceDetailCustomization(bool bOnlyShowVariableOverrides);

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

private:
	bool bOnlyShowVariableOverrides = false;
};
