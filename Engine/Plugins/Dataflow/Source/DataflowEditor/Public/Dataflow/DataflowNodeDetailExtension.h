// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"
#include "Templates/SharedPointer.h"
#include "IDetailPropertyExtensionHandler.h"

namespace UE::Dataflow
{
	/**
	* IDetailPropertyExtensionHandler that can be registered with FDataflowNodeDetailExtensionRegistry to be aggregated and applied to the Dataflow Node Details panel.
	*/
	class IDataflowNodeDetailExtension : public IDetailPropertyExtensionHandler
	{
	public:
		virtual ~IDataflowNodeDetailExtension() = default;

		virtual FName GetName() const = 0;
	};

	/**
	* Registry for IDataflowNodeDetailExtension. Register/Deregister extensions to extend the Dataflow Node Details rows via IDetailPropertyExtensionHandler.
	* The only reason why this isn't an IDetailPropertyExtensionHandler is because DetailsViews want a SharedPtr of an IDetailPropertyExtensionHandler, but this is a singleton registry.
	*/
	class FDataflowNodeDetailExtensionRegistry
	{
	public:

		// FLazySingleton
		static DATAFLOWEDITOR_API FDataflowNodeDetailExtensionRegistry& GetInstance();
		static DATAFLOWEDITOR_API void TearDown();

		DATAFLOWEDITOR_API void RegisterExtension(TUniquePtr<IDataflowNodeDetailExtension>&& Extension);
		DATAFLOWEDITOR_API void DeregisterExtension(const FName& ExtensionName);

		// These methods aggregate calls on the registered extensions.
		DATAFLOWEDITOR_API bool IsPropertyExtendable(const UClass* InObjectClass, const IPropertyHandle& PropertyHandle) const;
		DATAFLOWEDITOR_API void ExtendWidgetRow(FDetailWidgetRow& InWidgetRow, const IDetailLayoutBuilder& InDetailBuilder, const UClass* InObjectClass, TSharedPtr<IPropertyHandle> PropertyHandle);

	private:

		TMap<FName, TUniquePtr<IDataflowNodeDetailExtension>> ExtensionMap;
	};


	/**
	* This is the IDetailPropertyExtensionHandler that the DataflowToolkit actually uses.It simply calls the equivalent methods on the FDataflowNodeDetailExtensionRegistry singleton.
	*/
	class FDataflowNodeDetailExtensionHandler : public IDetailPropertyExtensionHandler
	{
	public:
		// IDetailPropertyExtensionHandler
		DATAFLOWEDITOR_API virtual bool IsPropertyExtendable(const UClass* InObjectClass, const IPropertyHandle& PropertyHandle) const override;
		DATAFLOWEDITOR_API virtual void ExtendWidgetRow(FDetailWidgetRow& InWidgetRow, const IDetailLayoutBuilder& InDetailBuilder, const UClass* InObjectClass, TSharedPtr<IPropertyHandle> PropertyHandle) override;
	};
}

