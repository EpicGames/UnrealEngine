// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosLog.h"
#include "CoreMinimal.h"
#include "Dataflow/DataflowNodeParameters.h"
#include "Dataflow/DataflowInputOutput.h"
#include "Dataflow/DataflowNode.h"
#include "Dataflow/DataflowObjectInterface.h"
#include "Dataflow/DataflowRenderingViewMode.h"

namespace GeometryCollection::Facades
{
	class FRenderingFacade;
}
namespace UE::Dataflow
{
	class FContext;
	typedef TPair<FString, FName> FRenderKey;

	struct FGraphRenderingState 
	{
		FGraphRenderingState(const FGuid InGuid, const FDataflowNode* InNode, const FRenderingParameter& InParameters, UE::Dataflow::FContext& InContext,
			const UE::Dataflow::IDataflowConstructionViewMode& ViewMode, const bool bInEvaluateOutputs = true)
			: NodeGuid(InGuid)
			, Node(InNode)
			, RenderName(InParameters.Name)
			, RenderType(InParameters.Type)
			, RenderOutputs(InParameters.Outputs)
			, Context(InContext)
			, ViewMode(ViewMode)
			, bEvaluateOutputs(bInEvaluateOutputs)
		{}

		const FGuid& GetGuid() const { return NodeGuid; }
		FName GetNodeName() const { return Node?Node->GetName():FName(); }
		FRenderKey GetRenderKey() const { return { RenderName,RenderType }; }
		const TArray<FName>& GetRenderOutputs() const { return RenderOutputs; }

		template<class T>
		const T& GetValue(FName OutputName, const T& Default) const
		{
			if (Node)
			{
				if (const FDataflowOutput* Output = Node->FindOutput(OutputName))
				{
					if (Context.IsThreaded() || !bEvaluateOutputs)
					{
						return Output->ReadValue<T>(Context, Default);
					}
					return Output->GetValue<T>(Context, Default);
				}
			}
			return Default;
		}

		const UE::Dataflow::IDataflowConstructionViewMode& GetViewMode() const { return ViewMode; }

	private:
		const FGuid NodeGuid;
		const FDataflowNode* Node = nullptr;

		FString RenderName;
		FName RenderType;
		TArray<FName> RenderOutputs;

		UE::Dataflow::FContext& Context;

		const UE::Dataflow::IDataflowConstructionViewMode& ViewMode;

		/** Boolean to check if we need to reevaluate the outputs or read the cached value */
		bool bEvaluateOutputs = true;
	};


	class FRenderingFactory
	{
	public:

		class ICallbackInterface
		{
		public:
			virtual ~ICallbackInterface() = default;
			virtual FRenderKey GetRenderKey() const = 0;
			virtual bool CanRender(const UE::Dataflow::IDataflowConstructionViewMode& ViewMode) const = 0;
			virtual bool CanRenderFromState(const FGraphRenderingState& State) const { return CanRender(State.GetViewMode()); }
			virtual void Render(GeometryCollection::Facades::FRenderingFacade& RenderingFacade, const FGraphRenderingState& State) = 0;
		};

		~FRenderingFactory() { delete Instance; }

		DATAFLOWEDITOR_API static FRenderingFactory* GetInstance();

		DATAFLOWEDITOR_API void RegisterCallbacks(TUniquePtr<ICallbackInterface> InCallbacks);
		DATAFLOWEDITOR_API void DeregisterCallbacks(const FRenderKey& Key);

		bool Contains(const FRenderKey& InKey) const { return CallbackMap.Contains(InKey); }
		DATAFLOWEDITOR_API void RenderNodeOutput(GeometryCollection::Facades::FRenderingFacade& RenderingFacade, const FGraphRenderingState& State);
		DATAFLOWEDITOR_API bool CanRenderNodeOutput(const FGraphRenderingState& State) const;

	private:

		FRenderingFactory() {}

		static FRenderingFactory* Instance;

		TMap<FRenderKey, TUniquePtr<ICallbackInterface>> CallbackMap;
	};

}




