// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"
#include "Editor/UnrealEdTypes.h"
#include "ChaosLog.h"

namespace UE::Dataflow
{
	//
	// Dataflow Construction View Modes allow the user to choose different views on the same node output. The View Mode system serves two main purposes: 
	// 
	// 1) In the Dataflow Editor's Construction Viewport, the currently active IDataflowConstructionViewMode will determine which ELevelViewportType should be used for rendering.
	//    In other words, the the ViewMode will determine whether we have a 3D perspective camera or a 2D orthographic camera.
	// 2) In the Datflow Rendering Factory (UE::Dataflow::FRenderingFactory), the rendering callbacks that are registered will also be able to return different 
	//    render data based on what the currently active IDataflowConstructionViewMode in the viewport is.
	//
	// For example, suppose you have created a new type FMyMeshType that contains both a 3D Mesh representation and a 2D Mesh representation (for, say, UVs.) If you want to
	// write custom ViewModes for it, here is the process:
	// 
	// 1) Define two new classes, e.g. FMyMeshType2DViewMode (inheriting from FDataflowConstruction2DViewModeBase) and FMyMeshType3DViewMode (inheriting from FDataflowConstruction3DViewModeBase)
	// 2) In your module initialization, register both view modes with the UE::Dataflow::FRenderingViewModeFactory
	// 3) Create a render function callback class and register it with UE::Dataflow::FRenderingFactory. 
	//    - The CanRender function should return true if the view mode is either of your two new custom view modes.
	//    - The Render function should return different RenderCollections depending on what the current view mode is (which can be determined from the FGraphRenderingState parameter.)
	//

	class IDataflowConstructionViewMode
	{
	public:
		virtual ~IDataflowConstructionViewMode() = default;

		virtual FName GetName() const = 0;
		virtual FText GetButtonText() const = 0;
		virtual FText GetTooltipText() const = 0;
		virtual ELevelViewportType GetViewportType() const = 0;
		virtual FRotator GetViewportOrientation() const { return FRotator::ZeroRotator; }
		virtual bool CanDisplayGizmo() const { return false; }
		
		DATAFLOWEDITOR_API bool IsPerspective() const;
	};

	// Base 2D and 3D types

	class FDataflowConstruction2DViewModeBase : public IDataflowConstructionViewMode
	{
	public:
		DATAFLOWEDITOR_API virtual ELevelViewportType GetViewportType() const override;
	};

	class FDataflowConstruction3DViewModeBase : public IDataflowConstructionViewMode
	{
	public:
		DATAFLOWEDITOR_API virtual ELevelViewportType GetViewportType() const override;
	};

	// Concrete default 2D and 3D types

	class FDataflowConstruction2DViewMode : public FDataflowConstruction2DViewModeBase
	{
	public:
		DATAFLOWEDITOR_API static FName Name;
		virtual ~FDataflowConstruction2DViewMode() = default;
	private:
		DATAFLOWEDITOR_API virtual FName GetName() const override;
		DATAFLOWEDITOR_API virtual FText GetButtonText() const override;
		DATAFLOWEDITOR_API virtual FText GetTooltipText() const override;
		DATAFLOWEDITOR_API virtual FRotator GetViewportOrientation() const override;
	};

	class FDataflowConstruction3DViewMode : public FDataflowConstruction3DViewModeBase
	{
	public:
		DATAFLOWEDITOR_API static FName Name;
		virtual ~FDataflowConstruction3DViewMode() = default;
	private:
		DATAFLOWEDITOR_API virtual FName GetName() const override;
		DATAFLOWEDITOR_API virtual FText GetButtonText() const override;
		DATAFLOWEDITOR_API virtual FText GetTooltipText() const override;
		DATAFLOWEDITOR_API virtual FRotator GetViewportOrientation() const override;
		DATAFLOWEDITOR_API virtual bool CanDisplayGizmo() const { return true; }
	};

	// UV view mode (same as 2D but with a different name and button text)
	class FDataflowConstructionUVViewMode : public FDataflowConstruction2DViewModeBase
	{
	public:
		DATAFLOWEDITOR_API static FName Name;
		virtual ~FDataflowConstructionUVViewMode() = default;
	private:
		DATAFLOWEDITOR_API virtual FName GetName() const override;
		DATAFLOWEDITOR_API virtual FText GetButtonText() const override;
		DATAFLOWEDITOR_API virtual FText GetTooltipText() const override;
		DATAFLOWEDITOR_API virtual FRotator GetViewportOrientation() const override;
	};

	class FDataflowConstructionMaterialViewMode : public FDataflowConstruction3DViewModeBase
	{
	public:
		DATAFLOWEDITOR_API static FName Name;
		virtual ~FDataflowConstructionMaterialViewMode() = default;
	private:
		DATAFLOWEDITOR_API virtual FName GetName() const override;
		DATAFLOWEDITOR_API virtual FText GetButtonText() const override;
		DATAFLOWEDITOR_API virtual FText GetTooltipText() const override;
	};

	//
	// ViewMode registry/factory
	//

	class FRenderingViewModeFactory
	{
	public:

		FRenderingViewModeFactory();

		// FLazySingleton
		static DATAFLOWEDITOR_API FRenderingViewModeFactory& GetInstance();
		static DATAFLOWEDITOR_API void TearDown();

		DATAFLOWEDITOR_API void RegisterViewMode(TUniquePtr<IDataflowConstructionViewMode>&& ViewMode);
		DATAFLOWEDITOR_API void DeregisterViewMode(const FName& ViewModeName);

		DATAFLOWEDITOR_API const IDataflowConstructionViewMode* GetViewMode(const FName& ViewModeName) const;

		DATAFLOWEDITOR_API const TMap<FName, TUniquePtr<IDataflowConstructionViewMode>>& GetViewModes() const;

	private:

		TMap<FName, TUniquePtr<IDataflowConstructionViewMode>> ViewModeMap;
	};
}

