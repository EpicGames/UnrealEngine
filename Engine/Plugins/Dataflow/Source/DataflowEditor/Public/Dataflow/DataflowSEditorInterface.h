// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointerFwd.h"
#include "Logging/TokenizedMessage.h"

namespace UE::Dataflow
{
	class FContext;
}

class UDataflowEdNode;

/**
* FDataflowSEditorInterface
* 
*/
class FDataflowSEditorInterface
{
public:
	/** Dataflow editor content accessors */
	virtual TSharedPtr<UE::Dataflow::FContext> GetDataflowContext() const = 0;

	virtual bool NodesHaveToggleWidget() const
	{
		return true;
	}

	virtual bool NodesHaveFreezeWidget() const
	{
		return true;
	}

	virtual void OnRenderToggleChanged(UDataflowEdNode* UpdatedEdNode) const
	{
	}

	virtual FText GetMessageText(const UDataflowEdNode& Node) const
	{
		return FText();
	}

	virtual EMessageSeverity::Type GetMessageMaxSeverity(const UDataflowEdNode& Node) const
	{
		return EMessageSeverity::Info;
	}

protected:
	FDataflowSEditorInterface() = default;
};
