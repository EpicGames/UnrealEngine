// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraphUtilities.h"
#include "SGraphNode.h"
#include "UObject/GCObject.h"
#include "Dataflow/DataflowCore.h"
#include "Dataflow/DataflowSEditorInterface.h"

#define UE_API DATAFLOWEDITOR_API

class UDataflowEdNode;
class SCheckBox;
class SImage;
class STextBlock;

//
// SDataflowEdNode
//

class SDataflowEdNode : public SGraphNode , public FGCObject
{
	typedef SGraphNode Super;

public:
	typedef TFunction<void(UEdGraphNode* InNode, bool InEnabled)> FToggleRenderCallback;

	SLATE_BEGIN_ARGS(SDataflowEdNode)
		: _GraphNodeObj(nullptr)
	{}
	SLATE_ARGUMENT(UDataflowEdNode*, GraphNodeObj)
	SLATE_ARGUMENT(FDataflowSEditorInterface*, DataflowInterface)
	SLATE_END_ARGS()
	
	UE_API void Construct(const FArguments& InArgs, UDataflowEdNode* InNode);

	// SGraphNode interface
	UE_API virtual FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;
	UE_API virtual TArray<FOverlayWidgetInfo> GetOverlayWidgets(bool bSelected, const FVector2f& WidgetSize) const override;
	UE_API virtual FText GetNodeTooltip() const override;
	UE_API virtual void UpdateErrorInfo() override;
	virtual void RequestRenameOnSpawn() override { /* No auto rename on spawn, because it can interfere with Copy/Paste and cause a crash */ }

	static UE_API void CopyDataflowNodeSettings(TSharedPtr<FDataflowNode> SourceDataflowNode, TSharedPtr<FDataflowNode> TargetDataflowNode);

	//~ Begin FGCObject interface
	UE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("SDataflowEdNode");
	}
	//~ End FGCObject interface


private:
	// SGraphNode interface
	/** Override this to create a button to add pins on the input side of the node */
	UE_API virtual void CreateInputSideAddButton(TSharedPtr<SVerticalBox> InputBox) override;
	/** Override this to create a button to add pins on the output side of the node */
	UE_API virtual void CreateOutputSideAddButton(TSharedPtr<SVerticalBox> OutputBox) override;
	/** Callback function executed when Add pin button is clicked */
	UE_API virtual FReply OnAddPin() override;
	/** Checks whether Add pin button should currently be visible */
	UE_API virtual EVisibility IsAddPinButtonVisible() const;

	/** when clicking on the show/hide input pins button */
	UE_API FReply OnShowHideInputs();

	UE_API const FSlateBrush* GetPinButtonImage() const;

	UE_API virtual TSharedPtr<SGraphPin> CreatePinWidget(UEdGraphPin* Pin) const override;
	UE_API virtual TSharedRef<SWidget> CreateTitleWidget(TSharedPtr<SNodeTitle> NodeTitle);
	UE_API virtual TSharedRef<SWidget> CreateTitleRightWidget() override;

	TObjectPtr<UDataflowEdNode> DataflowGraphNode = nullptr;	

	FCheckBoxStyle CheckBoxStyle;
	TSharedPtr<SCheckBox> RenderCheckBoxWidget;
	TSharedPtr<SImage> FreezeImageWidget;
	TSharedPtr<STextBlock> PerfWidget;
	TSharedPtr<STextBlock> WatchWidget;

	//FCheckBoxStyle CacheStatusStyle;
	//TSharedPtr<SCheckBox> CacheStatus;

	FDataflowSEditorInterface* DataflowInterface;

};

#undef UE_API
