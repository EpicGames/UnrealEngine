// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraphUtilities.h"
#include "EdGraph/EdGraphNode.h"
#include "Dataflow/DataflowSEditorInterface.h"
#include "NodeFactory.h"
#include "SGraphNode.h"

class UDataflowEditor;

class FDataflowGraphNodeFactory : public FGraphNodeFactory, public TSharedFromThis<FDataflowGraphNodeFactory>
{
public:
	virtual ~FDataflowGraphNodeFactory() = default;

	FDataflowGraphNodeFactory(FDataflowSEditorInterface* InDataflowInterface)
		: DataflowInterface(InDataflowInterface)
	{}

	/** Create a widget for the supplied node */
	virtual TSharedPtr<SGraphNode> CreateNodeWidget(UEdGraphNode* InNode) override;

private:
	FDataflowSEditorInterface* DataflowInterface;
};