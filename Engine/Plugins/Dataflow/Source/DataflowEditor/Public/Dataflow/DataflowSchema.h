// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraph/EdGraphSchema.h"
#include "ConnectionDrawingPolicy.h"
#include "Dataflow/DataflowGraph.h"

#include "DataflowSchema.generated.h"

#define UE_API DATAFLOWEDITOR_API

class UDataflow; 

namespace UE::Dataflow
{
	struct FFactoryParameters;
}

UCLASS(MinimalAPI)
class UDataflowSchema : public UEdGraphSchema
{
	GENERATED_BODY()
public:
	UE_API UDataflowSchema();

	//~ Begin UEdGraphSchema Interface
	UE_API virtual const FPinConnectionResponse CanCreateConnection(const UEdGraphPin* PinA, const UEdGraphPin* PinB) const override;
	UE_API virtual void GetContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const override;
	UE_API virtual void GetGraphContextActions(FGraphContextMenuBuilder& ContextMenuBuilder) const override;

	UE_API virtual FLinearColor GetPinTypeColor(const FEdGraphPinType& PinType) const override;
	UE_API virtual FConnectionDrawingPolicy* CreateConnectionDrawingPolicy(int32 InBackLayerID, int32 InFrontLayerID, float InZoomFactor, const FSlateRect& InClippingRect, class FSlateWindowElementList& InDrawElements, class UEdGraph* InGraphObj) const override;
	UE_API virtual void OnPinConnectionDoubleCicked(UEdGraphPin* PinA, UEdGraphPin* PinB, const FVector2f& GraphPosition) const override;
	UE_API virtual void BreakPinLinks(UEdGraphPin& TargetPin, bool bSendsNodeNotifcation) const override;
	UE_API virtual bool TryCreateConnection(UEdGraphPin* PinA, UEdGraphPin* PinB) const override;
	UE_API virtual bool CreateAutomaticConversionNodeAndConnections(UEdGraphPin* InSourcePin, UEdGraphPin* InTargetPin) const override;

	UE_API virtual void GetAssetsGraphHoverMessage(const TArray<FAssetData>& Assets, const UEdGraph* HoverGraph, FString& OutTooltipText, bool& OutOkIcon) const override;
	UE_API virtual void DroppedAssetsOnGraph(const TArray<FAssetData>& Assets, const FVector2f& GraphPosition, UEdGraph* Graph) const override;

	UE_API virtual void GetAssetsNodeHoverMessage(const TArray<struct FAssetData>& Assets, const UEdGraphNode* HoverNode, FString& OutTooltipText, bool& OutOkIcon) const override;
	UE_API virtual void DroppedAssetsOnNode(const TArray<struct FAssetData>& Assets, const FVector2f& GraphPosition, UEdGraphNode* Node) const override;

	virtual void SetPinBeingDroppedOnNode(UEdGraphPin* InSourcePin) const override { PinBeingDropped = InSourcePin; }
	UE_API virtual bool SupportsDropPinOnNode(UEdGraphNode* InTargetNode, const FEdGraphPinType& InSourcePinType, EEdGraphPinDirection InSourcePinDirection, FText& OutMessage) const override;
	UE_API virtual UEdGraphPin* DropPinOnNode(UEdGraphNode* InTargetNode, const FName& InSourcePinName, const FEdGraphPinType& InSourcePinType, EEdGraphPinDirection InSourcePinDirection) const override;
	//~ End UEdGraphSchema Interface

	static UE_API FLinearColor GetTypeColor(const FName& Type);
	UE_API float GetPinTypeWireThickness(const FName& Type) const;
	UE_API TOptional<FLinearColor> GetPinColorOverride(TSharedPtr<FDataflowNode> DataflowNode, UEdGraphPin* Pin) const;


private:
	static UE_API bool CanPinBeConnectedToNode(const UEdGraphPin* Pin, const UE::Dataflow::FFactoryParameters& NodeParameters);
	static UE_API bool CanConnectPins(const UEdGraphPin& OutputPin, const UEdGraphPin& InputPin);
	static UE_API bool CanConvertPins(const UEdGraphPin& OutputPin, const UEdGraphPin& InputPin);
	static UE_API bool IsCategorySupported(FName NodeCategory, FName AssetType);
	static UE_API FName GetEditedAssetType();
	static UE_API UE::Dataflow::FPin::EDirection GetDirectionFromPinDirection(EEdGraphPinDirection InPinDirection);

	static TSharedPtr<UE::Dataflow::FGraph> GetDataflowGraphFromPin(const UEdGraphPin& Pin);
	static TSharedPtr<const FDataflowNode> GetDataflowNodeFromPin(const UEdGraphPin& Pin);
	static const FDataflowInput* GetDataflowInputFromPin(const UEdGraphPin& Pin);
	static const FDataflowOutput* GetDataflowOutputFromPin(const UEdGraphPin& Pin);

	// used by SupportsDropPinOnNode to know which original connection is being dropped
	mutable UEdGraphPin* PinBeingDropped = nullptr;

	UE_API virtual bool IsCacheVisualizationOutOfDate(int32 InVisualizationCacheID) const override;
	UE_API virtual int32 GetCurrentVisualizationCacheID() const override;
	UE_API virtual void ForceVisualizationCacheClear() const override;
	static int32 CurrentCacheRefreshID;
};

class FDataflowConnectionDrawingPolicy : public FConnectionDrawingPolicy, public FGCObject
{
public:
	FDataflowConnectionDrawingPolicy(int32 InBackLayerID, int32 InFrontLayerID, float InZoomFactor, const FSlateRect& InClippingRect, FSlateWindowElementList& InDrawElements, UEdGraph* InGraph);
	virtual void DetermineWiringStyle(UEdGraphPin* OutputPin, UEdGraphPin* InputPin, /*inout*/ FConnectionParams& Params) override;

	const UDataflowSchema* GetSchema() { return Schema; }

	/** FGCObject interface */
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override { return TEXT("FDataflowConnectionDrawingPolicy"); }

private:
	TObjectPtr<UDataflowSchema> Schema = nullptr;
};

#undef UE_API
