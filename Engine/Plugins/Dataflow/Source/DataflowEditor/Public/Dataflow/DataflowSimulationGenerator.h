// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/ObjectPtr.h"
#include "TickableEditorObject.h"
#include "Logging/LogMacros.h"
#include "Misc/AsyncTaskNotification.h"
#include "Dataflow/DataflowContent.h"
#include "Dataflow/DataflowPreview.h"
#include "StructUtils/PropertyBag.h"

DECLARE_LOG_CATEGORY_EXTERN(LogDataflowSimulationGenerator, Log, All);

class UChaosCacheCollection;
class AChaosCacheManager;

namespace UE::Dataflow
{
	/** Simulation Task to be run on the async thread */
	class FDataflowSimulationTask : public FNonAbandonableTask
	{
		public:
		FDataflowSimulationTask()
		{}

		/** Run the simulation */
		void DoWork();

		/** Can abandon check */
		bool CanAbandon()
		{
			return true;
		}

		TStatId GetStatId() const
		{
			RETURN_QUICK_DECLARE_CYCLE_STAT(TTaskRunner, STATGROUP_ThreadPoolAsyncTasks);
		}
	
		/** Task Manager */
		TSharedPtr<struct FDataflowTaskManager> TaskManager = nullptr;
		
		/** Simulation delta time */
		float DeltaTime = 0.0f;

		/** Simulation min time */
		float MinTime = TNumericLimits<float>::Max();

		/** Simulation max time */
		float MaxTime = TNumericLimits<float>::Lowest();

		/** Simulation world */
		UWorld* SimulationWorld = nullptr;

		/** Boolean to check if we are running the task in the background */
		bool bAsyncCaching = true;

		/** Boolean to check if we should restart the simulation in the range */
		bool bRestartSimulation = false;
	};

	/** Async simulation resource that will be used while simulating */
	struct FDataflowSimulationResource
	{
		/** Number of simulated frames */
		std::atomic<int32>* NumSimulatedFrames = nullptr;

		/** Simulation time of current frames */
		std::atomic<int64>* RecentDateTimeTicks = nullptr;

		/** Async cancel boolean */
		std::atomic<bool>* bCancelled = nullptr;

		/** Check if the simulation is canceled or not */
		bool IsCancelled() const
		{
			return !bCancelled || bCancelled->load();
		}
		
		/** Finish simulating the current frame */
		void FinishFrame()
		{
			if (NumSimulatedFrames)
			{
				++(*NumSimulatedFrames);
			}
			if (RecentDateTimeTicks)
			{
				RecentDateTimeTicks->store(FDateTime::UtcNow().GetTicks());
			}
		}
	};

	/** Simulation Task manager */
	struct FDataflowTaskManager
	{
		/** Simulation resource */
		TSharedPtr<FDataflowSimulationResource> SimulationResource;

		/** Simulation task */
		TUniquePtr<FAsyncTask<FDataflowSimulationTask>> SimulationTask;

		/** Async notification*/
		TUniquePtr<FAsyncTaskNotification> AsyncNotification;

		/** Number of frames to simulate */
		int32 NumFrames;

		/** Start time of the simulation */
		FDateTime StartTime;

		/** Last updated time */
		FDateTime LastUpdateTime;

		/** Allocate the simulation resource from the properties */
		bool AllocateSimulationResource(const FVector2f& TimeRange, const int32 FrameRate,
			const TObjectPtr<UChaosCacheCollection>& CacheAsset, const TSubclassOf<AActor>& ActorClass,
			const TObjectPtr<UDataflowBaseContent>& DataflowContent, const FTransform& BlueprintTransform,
			const FInstancedPropertyBag& BlueprintVariables, const bool bSkeletalMeshVisibility, float DeltaTime);

		UE_DEPRECATED(5.8, "Use BlueprintVariables version instead.")
		bool AllocateSimulationResource(const FVector2f& TimeRange, const int32 FrameRate,
			const TObjectPtr<UChaosCacheCollection>& CacheAsset, const TSubclassOf<AActor>& ActorClass,
			const TObjectPtr<UDataflowBaseContent>& DataflowContent, const FTransform& BlueprintTransform,
			const bool bSkeletalMeshVisibility, float DeltaTime);

		/** Free the simulation resource */
		void FreeSimulationResource();

		/** Cancel the simulation generation */
		void CancelSimulationGeneration();

		/** Number of simulated frames */
		std::atomic<int32> NumSimulatedFrames = 0;
		
		/** Simulation time of current frames */
		std::atomic<int64> RecentDateTimeTicks = 0;

		/** Boolean to check if the simulation has been cancelled */
		std::atomic<bool> bCancelled = false;
		
		/** Temporary world created to run the simulation */
		UWorld* SimulationWorld = nullptr;

		/** Temporary cache manager created to run the simulation */
		TObjectPtr<AChaosCacheManager> CacheManager = nullptr;

		/** Temporary cache manager created to run the simulation */
		TObjectPtr<AActor> PreviewActor = nullptr;
	};

	/** Enum for all the generator actions */
	enum class EDataflowGeneratorActions
	{
		NoAction,
		StartGenerate,
		TickGenerate
	};

	/** Dataflow simulation generator */
	class  FDataflowSimulationGenerator : public FTickableEditorObject
	{
	public:
		FDataflowSimulationGenerator();
		virtual ~FDataflowSimulationGenerator();

		//~ Begin FTickableEditorObject Interface
		virtual void Tick(float DeltaTime) override;
		virtual TStatId GetStatId() const override;
		virtual ETickableTickType GetTickableTickType() const override { return ETickableTickType::Always; }
		//~ End FTickableEditorObject Interface

		/** Set the blueprint class for cache recording */
		void SetBlueprintClass(const TSubclassOf<AActor>& InBlueprintClass);

		/** Set the blueprint transform for cache recording */
		void SetBlueprintTransform(const FTransform& InBlueprintTransform);
		
		/** Set the blueprint variables' values used to customize the blueprint */
		void SetBlueprintVariables(const FInstancedPropertyBag& InBlueprintVariables);

		/** Set the cache params for cache recording */
		void SetCacheParams(const FDataflowPreviewCacheParams& InCacheParams);

		/** Set the cache asset for cache recording */
		void SetCacheAsset(const TObjectPtr<UChaosCacheCollection>& InCacheAsset);

		/** Set the dataflow content */
		void SetDataflowContent(const TObjectPtr<UDataflowBaseContent>& InDataflowContent);

		/** Set the skeletal mesh visibility */
		void SetSkeletalMeshVisibility(const bool bInSkeletalMeshVisibility);

		/** Set delta time */
		void SetDeltaTime(float InDeltaTime) { GeneratorDeltaTime = InDeltaTime; };

		/** Enqueue a generator action to be processed on the async thread */
		void RequestGeneratorAction(EDataflowGeneratorActions Action);
		
		/** If the generator is running simulation */
		bool IsSimulating() { return PendingAction != EDataflowGeneratorActions::NoAction; };

	private:

		/** Start generating the simulation */
		void StartGenerateSimulation();
		
		/** Tick the generated simulation */
		void TickGenerateSimulation();

		/** Free the task resource used while generating the simulation */
		void FreeTaskResource(bool bCancelled);

		/** Cache asset to store the caches */
		TObjectPtr<UChaosCacheCollection> CacheAsset = nullptr;

		/** Cache params used to record simulation */
		FDataflowPreviewCacheParams CacheParams;

		/** Blueprint class used to spawn the actor */
		TSubclassOf<AActor> BlueprintClass;

		/** Blueprint transform used to spawn the actor */
		FTransform BlueprintTransform = FTransform::Identity;

		/** Blueprint variables' values used to customize the blueprint */
		FInstancedPropertyBag BlueprintVariables;

		/** Dataflow content */
		TObjectPtr<UDataflowBaseContent> DataflowContent;

		/** Skeletal mesh visibility */
		bool bSkeletalMeshVisibility = true;

		/** Delta time for a frame */
		float GeneratorDeltaTime = 0.f;

		/** Pending action to be send to the async thread */
		EDataflowGeneratorActions PendingAction = EDataflowGeneratorActions::NoAction;

		/** Task manager to run the async tasks */
		TSharedPtr<FDataflowTaskManager> TaskManager = nullptr;
	};
};


