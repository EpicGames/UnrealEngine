// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/ContainersFwd.h"
#include "GeometryCollection/ManagedArray.h"

class USkinnedAsset;
class UStaticMesh;
class UGeometryCache;

DECLARE_LOG_CATEGORY_EXTERN(LogDataflowSimulationGeometryCache, Log, All);

namespace UE::DataflowSimulationGeometryCache
{
	/** Saves skinned asset positions into the geometry cache*/
	DATAFLOWEDITOR_API void SaveGeometryCache(UGeometryCache& GeometryCache, float FrameRate, const USkinnedAsset& Asset, TConstArrayView<uint32> ImportedVertexNumbers, TArrayView<TArray<FVector3f>> PositionsToMoveFrom);

	/** Saves static mesh positions into the geometry cache */
	DATAFLOWEDITOR_API void SaveGeometryCache(UGeometryCache& GeometryCache, float FrameRate, const UStaticMesh& StaticMesh, TArrayView<TArray<FVector3f>> PositionsToMoveFrom);

	/** Saves the geometry cache */
	DATAFLOWEDITOR_API void SavePackage(UObject& Object);
};