// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/CacheCollection.h"
#include "Dataflow/DataflowEditor.h"
#include "Dataflow/DataflowEditorPreviewSceneBase.h"
#include "Dataflow/DataflowSimulationGenerator.h"
#include "StructUtils/PropertyBag.h"

#include "DataflowSimulationScene.generated.h"

#define UE_API DATAFLOWEDITOR_API

class AActor;
class FFleshCollection;
class UDataflowEditor;
class UDeformableTetrahedralComponent;
class UGeometryCache;
class UFleshDynamicAsset;
class USkeletalMesh;
class USkeletalMeshComponent;
class UStaticMesh;

DECLARE_EVENT(UDataflowSimulationSceneDescription, FDataflowSimulationSceneDescriptionChanged)

UCLASS(MinimalAPI)
class UDataflowSimulationSettings : public UDataflowEditorSettings
{
public:
	GENERATED_BODY()

	UPROPERTY()
	bool bIsSimulationPlayingByDefault = false;

	UPROPERTY()
	bool bIsAsyncCachingSupported = false;

	UPROPERTY()
	bool bIsAsyncCachingEnabledByDefault = false;

	UPROPERTY()
	bool bIsGeometryCacheOutputSupported = false;
};

UCLASS(MinimalAPI)
class UDataflowSimulationSceneDescription : public UObject
{
public:
	GENERATED_BODY()

	FDataflowSimulationSceneDescriptionChanged DataflowSimulationSceneDescriptionChanged;

	UDataflowSimulationSceneDescription()
	{
		SetFlags(RF_Transactional);
	}

	/** Set the simulation scene */
	UE_API void SetSimulationScene(class FDataflowSimulationScene* SimulationScene);

	/** Set the Blueprint. */
	UE_API void SetBlueprintClass(TSubclassOf<AActor> InBlueprintClass);

	/** Return the Blueprint. */
	UE_API const TSubclassOf<AActor>& GetBlueprintClass() const;

	/** Caching blueprint actor class to spawn */
	UE_DEPRECATED(5.8, "Use the SetBlueprintClass setter/getter instead as to get correct operations.")
	UPROPERTY(EditAnywhere, Setter = SetBlueprintClass, Getter = GetBlueprintClass, Category = "Scene")
	TSubclassOf<AActor> BlueprintClass;

	UPROPERTY(EditAnywhere, Category = "Scene|BlueprintVariables", Meta = (FixedLayout, ShowOnlyInnerProperties))
	FInstancedPropertyBag BlueprintVariables;

	/** Blueprint actor transform */
	UPROPERTY(EditAnywhere, Category = "Scene")
	FTransform BlueprintTransform = FTransform::Identity;

	UPROPERTY(EditAnywhere, Category = "Viewport")
	bool bPauseSimulationViewportWhenPlayingInEditor = true;

	UPROPERTY(EditAnywhere, Category = "Viewport")
	bool bPauseSimulationViewportWhenNotFocused = true;

	/** Caching asset to be used to record the simulation  */
	UPROPERTY(EditAnywhere, Category="Caching", DisplayName="Cache Collection")
	TObjectPtr<UChaosCacheCollection> CacheAsset = nullptr;

	/** Caching params used to record the simulation */
	UPROPERTY(EditAnywhere, Category="Caching")
	FDataflowPreviewCacheParams CacheParams;

	UPROPERTY()
	bool bIsGeometryCacheOutputSupported = false;

	/** Geometry cache asset used to extract skeletal mesh results from simulation */
	UPROPERTY(EditAnywhere, Category = "Geometry Cache", DisplayName="Geometry Cache", meta=(EditCondition = "bIsGeometryCacheOutputSupported && CacheAsset != nullptr", EditConditionHides))
	TObjectPtr<UGeometryCache> GeometryCacheAsset = nullptr;

	/** Skeletal mesh interpolated from simulation. This should match the SkeletalMesh used in GenerateSurfaceBindings node */
	UPROPERTY(EditAnywhere, Category = "Geometry Cache", meta = (EditCondition = "bIsGeometryCacheOutputSupported && CacheAsset != nullptr && EmbeddedStaticMesh == nullptr", EditConditionHides))
	TObjectPtr<USkeletalMesh> EmbeddedSkeletalMesh = nullptr;

	/** Static mesh interpolated from simulation. This should match the Static mesh used in GenerateSurfaceBindings node */
	UPROPERTY(EditAnywhere, Category = "Geometry Cache", meta = (EditCondition = "bIsGeometryCacheOutputSupported && CacheAsset != nullptr && EmbeddedSkeletalMesh == nullptr", EditConditionHides))
	TObjectPtr<UStaticMesh> EmbeddedStaticMesh = nullptr;

	/** Interpolates and saves geometry cache from Chaos cache */
	UFUNCTION(CallInEditor, Category = "Geometry Cache")
	UE_API void GenerateGeometryCache();

	/** Creates a new geometry cache file */
	UFUNCTION(CallInEditor, Category = "Geometry Cache")
	UE_API void NewGeometryCache();

	/** Visibility of the skeletal mesh */
	UPROPERTY(EditAnywhere, Category = "Skeletal Mesh")
	bool bSkeletalMeshVisibility = true;
private:

	//~ UObject Interface
	virtual void PreEditChange(class FProperty* PropertyAboutToChange) override;
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PostTransacted(const FTransactionObjectEvent& TransactionEvent) override;
	virtual void BeginDestroy() override;

	void OnBlueprintClassPreChanged();
	void OnBlueprintClassPostChanged();
	void UpdateBlueprintVariables(class UBlueprint* Blueprint);
	void RegisterOnBlueprintCompiledDelegate();
	void UnregisterOnBlueprintCompiledDelegate();

	/** Simulation scene linked to that descriptor */
	class FDataflowSimulationScene* SimulationScene;

	/** Render geometry positions from interpolation */
	TArray<TArray<FVector3f>> RenderPositions;
};

/**
 * Dataflow simulation scene holding all the dataflow content components
 */
class FDataflowSimulationScene : public FDataflowPreviewSceneBase
{
public:

	UE_API FDataflowSimulationScene(FPreviewScene::ConstructionValues ConstructionValues, UDataflowEditor* Editor);
	
	// FGCObject interface
	UE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	
	UE_API virtual ~FDataflowSimulationScene();

	/** Functions that will be triggered when objects will be reinstanced (BP compilation) */
	UE_API void OnObjectsReinstanced(const TMap<UObject*, UObject*>& ObjectsMap);

	/** Tick data flow scene */
	UE_API virtual void TickDataflowScene(const float DeltaSeconds) override;
	
	/** Check if the preview scene can run simulation */
	virtual bool CanRunSimulation() const { return true; }

	/** Get the bounding box for the selected components or the entire sim scene */
	UE_API virtual FBox GetBoundingBox() const override;

	/** Get the scene description used in the preview scene widget */
	UDataflowSimulationSceneDescription* GetPreviewSceneDescription() const { return SceneDescription; }

	/** Create all the simulation world components and instances */
	UE_API void CreateSimulationScene();

	/** Reset all the simulation world components and instances */
	UE_API void ResetSimulationScene();

	/** Pause the simulation */
	UE_API void PauseSimulationScene() const;

	/** Start the simulation */
	UE_API void StartSimulationScene() const;

	/** Step the simulation */
	UE_API void StepSimulationScene() const;
	
	/** Gets whether the simulation is running */
	UE_API bool IsSimulationEnabled() const;

	/** Sets whether the simulation should be enabled or not */
	UE_API void SetSimulationEnabled(bool bEnable) const;

	/** Check if caching is enabled or not */
	UE_API bool HasCachingEnabled() const;

	/** Rebuild the simulation scene */
	UE_API void RebuildSimulationScene();

	/** Check if there is something to render */
	bool HasRenderableGeometry() { return true; }

	/** Update Scene in response to the SceneDescription changing */
	UE_API void SceneDescriptionPropertyChanged(const FName& PropertyName);

	/** Record the simulation cache */
	UE_API void RecordSimulationCache();

	/** Update the simulation cache */
	UE_API void UpdateSimulationCache();

	/** Get the simulation time range */
	const FVector2f& GetTimeRange() const {return TimeRange;}

	/** Set the simulation time range */
	void SetTimeRange(const FVector2f& NewTimeRange);

	/** Get the number of frames */
	const int32& GetNumFrames() const {return NumFrames;}

	/** Get the frame rate */
	const int32 GetFrameRate() const { return SceneDescription->CacheParams.FrameRate; }

	/** Get the subframe rate */
	const int32 GetSubframeRate() const { return SceneDescription->CacheParams.SubframeRate; }

	/** Get delta time */
	float GetDeltaTime() const { return DeltaTime; }

	/** Simulation time used to drive the cache loading */
	float SimulationTime = 0.0f;

	/** Previous simulation time */
	float PreviousTime = 0.0f;
	
	/** Preview actor accessors */
	TObjectPtr<AActor> GetPreviewActor() { return PreviewActor; }
	const TObjectPtr<AActor> GetPreviewActor() const { return PreviewActor; }

	/** LOD for preview actor components */
	UE_API void SetPreviewLOD(int32 InLOD);
	UE_API int32 GetPreviewLOD() const;

	/** Check if the simulation is locked or not */
	bool IsSimulationLocked() const  {return bIsSimulationLocked;}

	/** Set the locked simulation flag */
	void SetSimulationLocked(const bool bSimulationLocked, const bool bIsPlaying);

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnTimeRangeUpdated, const FDataflowSimulationScene&);
	FOnTimeRangeUpdated OnTimeRangeUpdated;

private:
	friend UDataflowSimulationSceneDescription;

	/** Bind the scene selection to the components */
	UE_API void BindSceneSelection();

	/** Unbind the scene selection from the components */
	UE_API void UnbindSceneSelection();

	/** Sync the blueprint preview properties with the edited asset. */
	void ReadPreviewPropertiesFromDataflowOwner();

	/** Sync the blueprint preview properties with the edited asset. */
	void WritePreviewPropertiesToDataflowOwner() const;

	void UpdateTimelinedataFromAnimations();

	/** Recompute NumFrames from TimeRange and broadcast OnTimeRangeUpdated. */
	void BroadcastTimeRangeUpdated();

#if WITH_EDITOR
	/** Called when any UObject property changes - filters for animation changes on our editor content */
	void OnEditorContentPropertyChanged(UObject* Object, struct FPropertyChangedEvent& Event);
#endif

	/** Simulation scene description */
	TObjectPtr<UDataflowSimulationSceneDescription> SceneDescription;

	/** Simulation generator to record the simulation result */
	TSharedPtr<UE::Dataflow::FDataflowSimulationGenerator> SimulationGenerator;

	/** Cache time range in seconds */
	FVector2f TimeRange;

	/** Number of cache frames */
	int32 NumFrames;

	/** Delta time (1/fps) */
	float DeltaTime;

	/** Last context time stamp for which we regenerated the world */
	UE::Dataflow::FTimestamp LastTimeStamp = UE::Dataflow::FTimestamp::Invalid;

	/** Preview actor that will will be used to visualize the result of the simulation graph */
	TObjectPtr<AActor> PreviewActor;

	/** Handle for the delegate */
	FDelegateHandle OnObjectsReinstancedHandle;

#if WITH_EDITOR
	/** Handle for the object property changed delegate to detect animation changes */
	FDelegateHandle OnObjectPropertyChangedHandle;
#endif

	/** Preview LOD used in the simulation viewport */
	int32 CurrentPreviewLOD = INDEX_NONE;

	/** Boolean to check if we are recording the cache or not */
	bool bIsRecordingCache = false;

	/** Boolean to check if the simulation is locked onto the timeline */
	bool bIsSimulationLocked = false;
};




#undef UE_API
