// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Dataflow/DataflowEditorViewportClientBase.h"
#include "Dataflow/DataflowNodeParameters.h"
#include "Dataflow/DataflowComponentSelectionState.h"
#include "Dataflow/DataflowContent.h"
#include "InputBehaviorSet.h"

#define UE_API DATAFLOWEDITOR_API

class FDataflowEditorToolkit;
class FDataflowSimulationScene;

class FDataflowSimulationViewportClient : public FDataflowEditorViewportClientBase
{

public:
	using Super = FDataflowEditorViewportClientBase;

	UE_API FDataflowSimulationViewportClient(FEditorModeTools* InModeTools, TWeakPtr<FDataflowSimulationScene> InSimulationScene,  const bool bCouldTickScene,
								  const TWeakPtr<SEditorViewport> InEditorViewportWidget = nullptr);

	UE_API virtual ~FDataflowSimulationViewportClient();

	/** Set the data flow toolkit used to create the client*/
	UE_API void SetDataflowEditorToolkit(TWeakPtr<FDataflowEditorToolkit> DataflowToolkit);
	
	/** Get the data flow toolkit  */
	const TWeakPtr<FDataflowEditorToolkit>& GetDataflowEditorToolkit() const { return DataflowEditorToolkitPtr; }

	/** Set the tool command list */
	UE_API void SetToolCommandList(TWeakPtr<FUICommandList> ToolCommandList);

	// FGCObject Interface
	virtual FString GetReferencerName() const override { return TEXT("FDataflowSimulationViewportClient"); }
	
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnSelectionChangedMulticast, const TArray<UPrimitiveComponent*>&, const TArray<FDataflowBaseElement*>&)
	FOnSelectionChangedMulticast OnSelectionChangedMulticast;

private:
	bool IsDataflowEditorTabActive() const;

	// FDataflowEditorViewportClientBase
	UE_API virtual void OnViewportClicked(HHitProxy* HitProxy) override;

	// FEditorViewportClient interface
	UE_API virtual void Tick(float DeltaSeconds) override;
	UE_API virtual void ProcessClick(FSceneView& View, HHitProxy* HitProxy, FKey Key, EInputEvent Event, uint32 HitX, uint32 HitY) override;
	UE_API virtual void Draw(const FSceneView* View, FPrimitiveDrawInterface* PDI) override;
	UE_API virtual void DrawCanvas(FViewport& InViewport, FSceneView& View, FCanvas& Canvas) override;

	/** Toolkit used to create the viewport client */
	TWeakPtr<FDataflowEditorToolkit> DataflowEditorToolkitPtr = nullptr;

	/** strongly typed simulation scene */
	TWeakPtr<FDataflowSimulationScene> DataflowSimulationScenePtr;

	// @todo(brice) : Is this needed?
	TWeakPtr<FUICommandList> ToolCommandList;
	
	/** Flag to enable scene ticking from the client */
	bool bEnableSceneTicking = false;
};

#undef UE_API
