// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"
#include "Templates/SharedPointer.h"
#include "Internationalization/Text.h"

class FPrimitiveDrawInterface;
class FSceneView;
class FCanvas;
class FMenuBuilder;
class FDataflowSimulationScene;
class FDataflowSimulationViewportClient;

namespace UE::Dataflow
{

	class IDataflowSimulationVisualization
	{
	public:
		virtual ~IDataflowSimulationVisualization() = default;

		virtual FName GetName() const = 0;
		virtual void ExtendSimulationVisualizationMenu(const TSharedPtr<FDataflowSimulationViewportClient>& ViewportClient, FMenuBuilder& MenuBuilder) {};
		virtual void Draw(const FDataflowSimulationScene* SimulationScene, FPrimitiveDrawInterface* PDI) {};
		virtual void DrawCanvas(const FDataflowSimulationScene* SimulationScene, FCanvas* Canvas, const FSceneView* SceneView) {};
		virtual FText GetDisplayString(const FDataflowSimulationScene* SimulationScene) const { return FText(); };
		virtual void SimulationSceneUpdated(const FDataflowSimulationScene* SimulationScene) {};
	};

	class FDataflowSimulationVisualizationRegistry
	{
	public:

		// FLazySingleton
		static DATAFLOWEDITOR_API FDataflowSimulationVisualizationRegistry& GetInstance();
		static DATAFLOWEDITOR_API void TearDown();

		DATAFLOWEDITOR_API void RegisterVisualization(TUniquePtr<IDataflowSimulationVisualization>&& Visualization);
		DATAFLOWEDITOR_API void DeregisterVisualization(const FName& VisualizationName);

		DATAFLOWEDITOR_API const TMap<FName, TUniquePtr<IDataflowSimulationVisualization>>& GetVisualizations() const;
		DATAFLOWEDITOR_API const IDataflowSimulationVisualization* GetVisualization(const FName& VisualizationOption) const;

	private:

		TMap<FName, TUniquePtr<IDataflowSimulationVisualization>> VisualizationMap;
	};

}

