// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AssetRegistry/AssetData.h"
#include "Textures/SlateIcon.h"

/** One template entry */
struct FDataflowTemplateOption
{
	FText              DisplayName;
	FText              Tooltip;
	/** Icon shown in the tile. Pass nullptr to fall back to a generic dataflow icon. */
	const FSlateBrush* Icon = nullptr;
	/** Opaque identifier returned to the caller in the result. */
	FName              TemplateId;
};

/**
 * Describes a single UDataflow asset to expose as a named template for a specific asset class.
 * Passed to FDataflowTemplateRegistry::RegisterTemplateAsset at module startup.
 *
 * Example:
 *   FDataflowTemplateRegistry::Get().RegisterTemplateAsset(
 *       UGroomAsset::StaticClass(),
 *       { FSoftObjectPath(TEXT("/GroomEditor/Dataflow/Templates/DF_BasicHair")),
 *         LOCTEXT("BasicHair", "Basic Hair"), {}, FSlateIcon("GroomEditorStyle", "ClassThumbnail.GroomAsset") });
 */
struct FDataflowTemplateAssetRegistration
{
	/** Soft path to the specific UDataflow asset. */
	FSoftObjectPath AssetPath;

	/** Tile label. If empty, derived from the asset name (strips "DF_", replaces underscores with spaces). */
	FText DisplayName;

	/** Tooltip shown on hover. If empty, defaults to the asset object path. */
	FText Tooltip;

	/**
	 * Tile icon. Falls back to the generic Dataflow icon if not set.
	 * e.g. FSlateIcon("GroomEditorStyle", "ClassThumbnail.GroomAsset").
	 */
	FSlateIcon Icon;
};

/**
 * Describes a content folder containing UDataflow template assets for a specific asset class.
 * Passed to FDataflowTemplateRegistry::RegisterTemplateFolder at module startup.
 */
struct FDataflowTemplateFolderRegistration
{
	/** Content path to scan recursively, e.g. "/Game/Dataflow/Templates/Cloth". */
	FString ContentFolder;

	/**
	 * Icon for every template found in this folder. Using FSlateIcon allows referencing brushes
	 * from any style set, not just FAppStyle — required for class icons that live in plugin style
	 * sets (e.g. FSlateIcon("GroomEditorStyle", "ClassIcon.GroomAsset")).
	 * A default Dataflow icon is used when the icon is not set.
	 */
	FSlateIcon Icon;
};

/**
 * Registry mapping asset classes to content folders containing UDataflow template assets.
 *
 * Modules register their template folders in StartupModule. When the Dataflow picker dialog
 * opens for an asset, it queries this registry to populate the template tile list.
 * Registrations for parent classes are automatically included (class hierarchy walk,
 * most-derived first).
 *
 * Example (in a plugin's editor module StartupModule):
 *
 *   FDataflowTemplateRegistry::Get().RegisterTemplateFolder(
 *       UChaosClothAsset::StaticClass(),
 *       { TEXT("/ChaosClothEditor/Dataflow/Templates"), FSlateIcon("ChaosClothEditorStyle", "ClassIcon.ChaosClothAsset") });
 */
class DATAFLOWEDITOR_API FDataflowTemplateRegistry
{
public:
	static FDataflowTemplateRegistry& Get();

	/**
	 * Register a single UDataflow asset as a template for the given asset class.
	 * Registrations are visible for subclasses of AssetClass.
	 * Safe to call from any module's StartupModule (game thread only).
	 */
	void RegisterTemplateAsset(const UClass* AssetClass, FDataflowTemplateAssetRegistration Registration);

	/**
	 * Register a content folder to scan for UDataflow template assets for a given asset class.
	 * Registrations are also visible when the picker is opened for any subclass of AssetClass.
	 * Safe to call from any module's StartupModule (game thread only).
	 */
	void RegisterTemplateFolder(const UClass* AssetClass, FDataflowTemplateFolderRegistration Registration);

	/**
	 * Build template options for the given asset class by scanning all registered folders
	 * (including those registered for parent classes, most-derived first).
	 * A "Blank" option is always prepended.
	 * Queries the asset registry synchronously — call from the game thread only.
	 */
	TArray<FDataflowTemplateOption> GetTemplateOptions(const UClass* AssetClass, bool bAddDefaultBlankOption = true) const;

private:
	// Keyed by class path so we don't hold raw UClass* pointers across module reloads.
	TMap<FTopLevelAssetPath, TArray<FDataflowTemplateAssetRegistration>>  AssetRegistrations;
	TMap<FTopLevelAssetPath, TArray<FDataflowTemplateFolderRegistration>> Registrations;
};
