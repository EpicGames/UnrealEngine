// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"
#include "Templates/SharedPointer.h"
#include "UObject/ObjectPtr.h"
#include "Textures/SlateIcon.h"

#define UE_API DATAFLOWEDITOR_API

class FUICommandInfo;
class UInteractiveToolBuilder;
class FUICommandList;
class UInteractiveTool;

namespace UE::Dataflow
{
	class FDataflowToolRegistry
	{
	public:

		// Interface for binding/unbinding tool actions. When a tool begins or ends we switch out the currently available FUICommandList. This allows multiple tools to have
		// individual hotkey actions with the same key chords, for example.
		class IDataflowToolActionCommands
		{
		public:
			virtual ~IDataflowToolActionCommands() = default;
			virtual void UnbindActiveCommands(const TSharedPtr<FUICommandList>& UICommandList) const = 0;
			virtual void BindCommandsForCurrentTool(const TSharedPtr<FUICommandList>& UICommandList, UInteractiveTool* Tool) const = 0;
		};

		static UE_API FDataflowToolRegistry& Get();
		static UE_API void TearDown();

		UE_API void AddNodeToToolMapping(const FName& NodeName, 
			TObjectPtr<UInteractiveToolBuilder> ToolBuilder, 
			const TSharedRef<const IDataflowToolActionCommands>& ToolActionCommands, 
			const FSlateIcon& AddNodeButtonIcon,
			const FText& AddNodeButtonText, 
			const FName& ToolCategory = FName("General"),
			const FName& AddNodeConnectionType = FName("FManagedArrayCollection"),
			const FName& AddNodeConnectionName = FName("Collection")
		);

		UE_DEPRECATED(5.6, "Please use the version of AddNodeToToolMapping taking AddNodeButtonIcon and AddNodeButtonText")
		void AddNodeToToolMapping(const FName& NodeName,
			TObjectPtr<UInteractiveToolBuilder> ToolBuilder,
			const TSharedRef<const IDataflowToolActionCommands>& ToolActionCommands)
		{
			AddNodeToToolMapping(NodeName, ToolBuilder, ToolActionCommands, FSlateIcon(), FText());
		}

		UE_API void RemoveNodeToToolMapping(const FName& NodeName);

		UE_API TArray<FName> GetNodeNames() const;

		bool HasToolInfoForNodeType(const FName& NodeType) const
		{
			return NodeTypeToToolMap.Contains(NodeType);
		}

		UE_API TSharedPtr<FUICommandInfo>& GetAddNodeCommandForNode(const FName& NodeType);
		UE_API const FSlateIcon& GetAddNodeButtonIcon(const FName& NodeType) const;
		UE_API const FText& GetAddNodeButtonText(const FName& NodeType) const;
		UE_API const FName& GetAddNodeConnectionType(const FName& NodeType) const;
		UE_API const FName& GetAddNodeConnectionName(const FName& NodeType) const;

		UE_API TSharedPtr<FUICommandInfo>& GetToolCommandForNode(const FName& NodeName);

		/** Return the category the tool has been registered in */
		UE_API const FName& GetToolCategoryForNode(const FName& NodeName) const;

		UE_API UInteractiveToolBuilder* GetToolBuilderForNode(const FName& NodeName);
		UE_API const UInteractiveToolBuilder* GetToolBuilderForNode(const FName& NodeName) const;

		UE_API void UnbindActiveCommands(const TSharedPtr<FUICommandList>& UICommandList) const;
		UE_API void BindCommandsForCurrentTool(const TSharedPtr<FUICommandList>& UICommandList, UInteractiveTool* Tool) const;

	private:

		struct FToolInfo
		{
			// Specified when registering the tool
			TObjectPtr<UInteractiveToolBuilder> ToolBuilder;
			TSharedRef<const IDataflowToolActionCommands> ToolActionCommands;
			FSlateIcon AddNodeButtonIcon;
			FText AddNodeButtonText;
			FName AddNodeConnectionType;
			FName AddNodeConnectionName;

			// Constructed automatically in FDataflowEditorCommandsImpl::RegisterCommands
			TSharedPtr<FUICommandInfo> AddNodeCommand;
			TSharedPtr<FUICommandInfo> ToolCommand;

			/** Tool category for filtering */
			FName ToolCategory;
		};

		TMap<FName, FToolInfo> NodeTypeToToolMap;
	
	};

}

#undef UE_API
