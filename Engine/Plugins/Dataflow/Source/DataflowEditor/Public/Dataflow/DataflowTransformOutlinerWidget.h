// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Dataflow/DataflowSelection.h"
#include "DetailLayoutBuilder.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SComboBox.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "GeometryCollection/ManagedArrayAccessor.h"
#include "Delegates/Delegate.h"
#include "UObject/NameTypes.h"
#include "Styling/StarshipCoreStyle.h"
#include "Widgets/Views/STreeView.h"

//////////////////////////////////////////////////////////////////////////////////////////
/* -- Widget to display Collection/Transform group data in a hierarchy -- */
//////////////////////////////////////////////////////////////////////////////////////////

class FDataflowOutlinerTreeItem;
class FDataflowOutlinerTreeItemBone;
class STransformOutliner;

typedef TArray<TSharedPtr<FDataflowOutlinerTreeItem>> FDataflowOutlinerTreeItemList;
typedef TSharedPtr<FDataflowOutlinerTreeItem> FDataflowOutlinerTreeItemPtr;
typedef TSharedPtr<class FDataflowOutlinerTreeItemBone> FDataflowOutlinerTreeItemBonePtr;

class FDataflowOutlinerTreeItem : public TSharedFromThis<FDataflowOutlinerTreeItem>
{
public:
	virtual ~FDataflowOutlinerTreeItem() = default;
	virtual TSharedRef<ITableRow> MakeTreeRowWidget(const TSharedRef<STableViewBase>& InOwnerTable, bool bIsPinned = false) = 0;
	virtual void GetChildren(FDataflowOutlinerTreeItemList& OutChildren) = 0;

	virtual int32 GetBoneIndex() const { return INDEX_NONE; }

	void GenerateContextMenu(UToolMenu* Menu, STransformOutliner& Outliner);
};

class FDataflowOutlinerTreeItemCollection : public FDataflowOutlinerTreeItem
{
public:
	FDataflowOutlinerTreeItemCollection(TSharedPtr<const FManagedArrayCollection> InCollection, FName InOutputName, TSharedPtr<STreeView<FDataflowOutlinerTreeItemPtr>> InTreeView)
		: Collection(InCollection)
		, OutputName(InOutputName)
		, TreeView(InTreeView)
	{
		RegenerateChildren();
	}

	virtual ~FDataflowOutlinerTreeItemCollection() = default;

	// FDataflowOutlinerTreeItem interface
	virtual TSharedRef<ITableRow> MakeTreeRowWidget(const TSharedRef<STableViewBase>& InOwnerTable, bool bIsPinned = false);
	virtual void GetChildren(FDataflowOutlinerTreeItemList& OutChildren) override;

	FDataflowOutlinerTreeItemPtr GetItemFromBoneIndex(int32 BoneIndex) const;

	void GetChildrenForBone(FDataflowOutlinerTreeItemBone& BoneItem, FDataflowOutlinerTreeItemList& OutChildren) const;
	bool HasChildrenForBone(const FDataflowOutlinerTreeItemBone& BoneItem) const;

	void ExpandAll();
	void RegenerateChildren();
	void RequestTreeRefresh();
	void ExpandByLevel(const int32 InLevel);

	TSharedPtr<const FManagedArrayCollection> GetCollection() { return Collection; }

	bool IsValid() const;

	// Mark item as unused/invalid; helpful because slate defers destroying tree items and can still run callbacks on them until tick
	void Invalidate()
	{
		bInvalidated = true;
	}

private:
	bool FilterBoneIndex(int32 BoneIndex) const;

	TSharedPtr<const FManagedArrayCollection> Collection;
	FName OutputName;

	TSharedPtr<STreeView<FDataflowOutlinerTreeItemPtr>> TreeView;

	// The direct children under this component
	TArray<FDataflowOutlinerTreeItemPtr> ChildItems;
	TMap<int32, FDataflowOutlinerTreeItemPtr> ItemsByBoneIndex;
	int32 RootIndex;

	// track whether the item has been explicitly invalidated
	bool bInvalidated = false;
};

class FDataflowOutlinerTreeItemBone : public FDataflowOutlinerTreeItem
{
public:
	FDataflowOutlinerTreeItemBone(const int32 InBoneIndex, FDataflowOutlinerTreeItemCollection* InParentCollectionItem)
		: BoneIndex(InBoneIndex)
		, ParentCollectionItem(InParentCollectionItem)
		, ItemColor(FSlateColor::UseForeground())
	{}

	// FDataflowOutlinerTreeItem interface
	TSharedRef<ITableRow> MakeTreeRowWidget(const TSharedRef<STableViewBase>& InOwnerTable, bool bIsPinned = false);
	TSharedRef<SWidget> MakeIndexColumnWidget() const;
//	TSharedRef<SWidget> MakeSimulationTypeColumnWidget() const;
	TSharedRef<SWidget> MakeEmptyColumnWidget() const;

	virtual void GetChildren(FDataflowOutlinerTreeItemList& OutChildren) override;
	bool IsValidBone() const;
	virtual int32 GetBoneIndex() const override { return BoneIndex; }
	FSlateColor GetItemColor() const { return ItemColor; }
	bool HasChildren() const;
	TSharedPtr<const FManagedArrayCollection> GetCollection() const;

protected:

private:
	void UpdateItemColorFromCollection();

	const int32 BoneIndex;
	FDataflowOutlinerTreeItemCollection* ParentCollectionItem;
	FSlateColor ItemColor;
};

class STransformOutlinerRow : public SMultiColumnTableRow<FDataflowOutlinerTreeItemBonePtr>
{
protected:
	FDataflowOutlinerTreeItemBonePtr Item;

public:
	SLATE_BEGIN_ARGS(STransformOutlinerRow) {}
	SLATE_END_ARGS()

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

	void Construct(const FArguments& InArgs, TSharedRef<STableViewBase> OwnerTableView, FDataflowOutlinerTreeItemBonePtr InItemToEdit)
	{
		Item = InItemToEdit;
		SMultiColumnTableRow<FDataflowOutlinerTreeItemBonePtr>::Construct(
			FSuperRowType::FArguments()
			.Style(&FStarshipCoreStyle::GetCoreStyle().GetWidgetStyle<FTableRowStyle>("TableView.AlternatingRow"))
			, OwnerTableView);
	}
};

class STransformOutliner : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(STransformOutliner) {}
		SLATE_ARGUMENT(TSharedPtr<SScrollBar>, ExternalVerticalScrollBar)
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs);

	void RegenerateItems();
	void RegenerateHeader();

	TSharedRef<ITableRow> MakeTreeRowWidget(FDataflowOutlinerTreeItemPtr InInfo, const TSharedRef<STableViewBase>& OwnerTable);
	TSharedRef<ITableRow> OnGeneratePinnedRowWidget(FDataflowOutlinerTreeItemPtr InItem, const TSharedRef<STableViewBase>& InOwnerTable, bool bPinned);
	void OnGetChildren(TSharedPtr<FDataflowOutlinerTreeItem> InInfo, TArray< TSharedPtr<FDataflowOutlinerTreeItem> >& OutChildren);
	TSharedPtr<SWidget> OnOpenContextMenu();

	void UpdateGeometryCollection();
	void SetCollection(const TSharedPtr<const FManagedArrayCollection>& InCollection, const FName InOutputName);
	int32 GetBoneSelectionCount() const;

	void ExpandAll();
	void ExpandRecursive(TSharedPtr<FDataflowOutlinerTreeItem> TreeItem, bool bInExpansionState) const;
	void ExpandByLevel(const int32 InLevel);

	int32 DisplayLevel = -1;

	void ContextMenuExpandAll();
	void ContextMenuPreviousLevel();
	void ContextMenuNextLevel();
	
private:
	void OnSelectionChanged(FDataflowOutlinerTreeItemPtr Item, ESelectInfo::Type SelectInfo);

	TSharedPtr<STreeView<FDataflowOutlinerTreeItemPtr>> TreeView;
	TSharedPtr<SHeaderRow> HeaderRowWidget;
	TArray<TSharedPtr<FDataflowOutlinerTreeItemCollection>> RootNodes;
	bool bPerformingSelection;
	TSharedPtr<const FManagedArrayCollection> Collection;
};

