// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Dataflow/DataflowSelection.h"
#include "DetailLayoutBuilder.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SComboBox.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "Delegates/Delegate.h"
#include "UObject/NameTypes.h"
#include "Styling/StarshipCoreStyle.h"
#include "Widgets/Views/STreeView.h"

class FGeometryCollection;
class FGeometryCollectionVerticesTreeItem;
class FGeometryCollectionVerticesTreeItemVertex;
class SVerticesOutliner;

typedef TArray<TSharedPtr<FGeometryCollectionVerticesTreeItem>> FGeometryCollectionVerticesTreeItemList;
typedef TSharedPtr<FGeometryCollectionVerticesTreeItem> FGeometryCollectionVerticesTreeItemPtr;
typedef TSharedPtr<class FGeometryCollectionVerticesTreeItemVertex> FGeometryCollectionVerticesTreeItemVertexPtr;

class FGeometryCollectionVerticesTreeItem : public TSharedFromThis<FGeometryCollectionVerticesTreeItem>
{
public:
	virtual ~FGeometryCollectionVerticesTreeItem() = default;
	virtual TSharedRef<ITableRow> MakeTreeRowWidget(const TSharedRef<STableViewBase>& InOwnerTable, bool bIsPinned = false) = 0;
	virtual void GetChildren(FGeometryCollectionVerticesTreeItemList& OutChildren) = 0;

	virtual int32 GetVertexIndex() const { return INDEX_NONE; }

	void GenerateContextMenu(UToolMenu* Menu, SVerticesOutliner& Outliner);
};

class FGeometryCollectionVerticesTreeItemGeometry : public FGeometryCollectionVerticesTreeItem
{
public:
	FGeometryCollectionVerticesTreeItemGeometry(TSharedPtr<const FManagedArrayCollection> InCollection, int32 InGeoemtryIdx, TSharedPtr<STreeView<FGeometryCollectionVerticesTreeItemPtr>> InTreeView)
		: Collection(InCollection)
		, GeometryIdx(InGeoemtryIdx)
		, TreeView(InTreeView)
	{
		RegenerateChildren();
	}

	virtual ~FGeometryCollectionVerticesTreeItemGeometry() = default;

	// FGeometryCollectionVerticesTreeItem interface
	virtual TSharedRef<ITableRow> MakeTreeRowWidget(const TSharedRef<STableViewBase>& InOwnerTable, bool bIsPinned = false);
	virtual void GetChildren(FGeometryCollectionVerticesTreeItemList& OutChildren) override;

	FGeometryCollectionVerticesTreeItemPtr GetItemFromVertexIndex(int32 InVertexIndex) const;

	void GetChildrenForVertex(FGeometryCollectionVerticesTreeItemVertex& VertexItem, FGeometryCollectionVerticesTreeItemList& OutChildren) const;
	bool HasChildrenForVertex(const FGeometryCollectionVerticesTreeItemVertex& VertexItem) const;

	void ExpandAll();
	void RegenerateChildren();
	void RequestTreeRefresh();

	TSharedPtr<const FManagedArrayCollection> GetCollection() { return Collection; }

	bool IsValid() const;

	// Mark item as unused/invalid; helpful because slate defers destroying tree items and can still run callbacks on them until tick
	void Invalidate()
	{
		bInvalidated = true;
	}

private:
	bool FilterBoneIndex(int32 BoneIndex) const;

	TSharedPtr<const FManagedArrayCollection> Collection;
	int32 GeometryIdx;

	TSharedPtr<STreeView<FGeometryCollectionVerticesTreeItemPtr>> TreeView;

	// The direct children under this component
	TArray<FGeometryCollectionVerticesTreeItemPtr> ChildItems;
	TMap<int32, FGeometryCollectionVerticesTreeItemPtr> ItemsByVertexIndex;
	int32 RootIndex;

	// track whether the item has been explicitly invalidated
	bool bInvalidated = false;
};

class FGeometryCollectionVerticesTreeItemVertex : public FGeometryCollectionVerticesTreeItem
{
public:
	FGeometryCollectionVerticesTreeItemVertex(const int32 InVertexIndex, FGeometryCollectionVerticesTreeItemGeometry* InParentCollectionItem)
		: VertexIndex(InVertexIndex)
		, ParentCollectionItem(InParentCollectionItem)
		, ItemColor(FSlateColor::UseForeground())
	{}

	// FGeometryCollectionVerticesTreeItem interface
	TSharedRef<ITableRow> MakeTreeRowWidget(const TSharedRef<STableViewBase>& InOwnerTable, bool bIsPinned = false);
	TSharedRef<SWidget> MakeIndexColumnWidget() const;
	TSharedRef<SWidget> MakeEmptyColumnWidget() const;

	virtual void GetChildren(FGeometryCollectionVerticesTreeItemList& OutChildren) override;
	bool IsValidVertex() const;
	virtual int32 GetVertexIndex() const override { return VertexIndex; }
	FSlateColor GetItemColor() const { return ItemColor; }
	bool HasChildren() const;
	TSharedPtr<const FManagedArrayCollection> GetCollection() const;

protected:

private:
	void UpdateItemColorFromCollection();

	const int32 VertexIndex;
	FGeometryCollectionVerticesTreeItemGeometry* ParentCollectionItem;
	FSlateColor ItemColor;
};

class SVerticesOutlinerRow : public SMultiColumnTableRow<FGeometryCollectionVerticesTreeItemVertexPtr>
{
protected:
	FGeometryCollectionVerticesTreeItemVertexPtr Item;

public:
	SLATE_BEGIN_ARGS(SVerticesOutlinerRow) {}
	SLATE_END_ARGS()

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

	void Construct(const FArguments& InArgs, TSharedRef<STableViewBase> OwnerTableView, FGeometryCollectionVerticesTreeItemVertexPtr InItemToEdit)
	{
		Item = InItemToEdit;
		SMultiColumnTableRow<FGeometryCollectionVerticesTreeItemVertexPtr>::Construct(
			FSuperRowType::FArguments()
			.Style(&FStarshipCoreStyle::GetCoreStyle().GetWidgetStyle<FTableRowStyle>("TableView.AlternatingRow"))
			, OwnerTableView);
	}
};

class SVerticesOutliner : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SVerticesOutliner) {}
		SLATE_ARGUMENT(TSharedPtr<SScrollBar>, ExternalVerticalScrollBar)
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs);

	void RegenerateItems();
	void RegenerateHeader();

	TSharedRef<ITableRow> MakeTreeRowWidget(FGeometryCollectionVerticesTreeItemPtr InInfo, const TSharedRef<STableViewBase>& OwnerTable);
	TSharedRef<ITableRow> OnGeneratePinnedRowWidget(FGeometryCollectionVerticesTreeItemPtr InItem, const TSharedRef<STableViewBase>& InOwnerTable, bool bPinned);
	void OnGetChildren(TSharedPtr<FGeometryCollectionVerticesTreeItem> InInfo, TArray< TSharedPtr<FGeometryCollectionVerticesTreeItem> >& OutChildren);
	TSharedPtr<SWidget> OnOpenContextMenu();

	void UpdateGeometryCollection();
	void SetCollection(const TSharedPtr<const FManagedArrayCollection>& InCollection);
	int32 GetBoneSelectionCount() const;

	void ExpandAll();
	void ExpandRecursive(TSharedPtr<FGeometryCollectionVerticesTreeItem> TreeItem, bool bInExpansionState) const;

private:
	void OnSelectionChanged(FGeometryCollectionVerticesTreeItemPtr Item, ESelectInfo::Type SelectInfo);

	TSharedPtr<STreeView<FGeometryCollectionVerticesTreeItemPtr>> TreeView;
	TSharedPtr<SHeaderRow> HeaderRowWidget;
	TArray<TSharedPtr<FGeometryCollectionVerticesTreeItemGeometry>> RootNodes;
	bool bPerformingSelection;
	TSharedPtr<const FManagedArrayCollection> Collection;
};

