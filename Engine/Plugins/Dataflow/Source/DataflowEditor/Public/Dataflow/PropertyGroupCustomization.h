// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Dataflow/PropertyListCustomization.h"

#define UE_API DATAFLOWEDITOR_API

namespace UE::Dataflow
{
	class FContext;

	/**
	 * Property group customization to allow selecting of a group from a list of the groups currently held by the node's collection.
	 */
	class FPropertyGroupCustomization : public FPropertyListCustomization
	{
	public:
		static TSharedRef<IPropertyTypeCustomization> MakeInstance();

		FPropertyGroupCustomization()
			: FPropertyListCustomization(FPropertyListCustomization::FOnGetListNames::CreateRaw(this, &FPropertyGroupCustomization::GetListNames))
		{}

	private:

		/**
		 * Turn a string into a valid collection group or attribute name.
		 * The resulting name won't contains spaces and any other special characters as listed in
		 * INVALID_OBJECTNAME_CHARACTERS (currently "',/.:|&!~\n\r\t@#(){}[]=;^%$`).
		 * It will also have all leading underscore removed, as these names are reserved for internal use.
		 * @param InOutString The string to turn into a valid collection name.
		 * @return Whether the InOutString was already a valid collection name.
		 */
		static bool MakeGroupName(FString& InOutString);

		/** List of valid group names for the drop down list. Override this method to filter to a specific set of group names. */
		virtual TArray<FName> GetTargetGroupNames(const FManagedArrayCollection& Collection) const
		{
			return Collection.GroupNames();  // By default returns all of the collection's group names
		}

		UE_API virtual bool MakeValidListName(FString& InOutString, FText& OutErrorMessage) const override;
		UE_API TArray<FName> GetListNames(const FManagedArrayCollection& Collection) const;
	};
}

#undef  UE_API
