// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/DelegateCombinations.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "IPropertyTypeCustomization.h"

class IPropertyHandle;
class ITableRow;
class SComboButton;
class STableViewBase;
class SWidget;
class SDataflowGraphEditor;

namespace UE::Dataflow
{
	class FContext;

	/**
	 * Property list customization to allow selecting of a name from a list derived from the node's collection.
	 */
	class FPropertyListCustomization : public IPropertyTypeCustomization
	{
	public:

		/** Delegate to return the list of valid names for the drop down list. */
		DECLARE_DELEGATE_RetVal_OneParam(TArray<FName>, FOnGetListNames, const FManagedArrayCollection&);

		FPropertyListCustomization() = default;

		explicit FPropertyListCustomization(FOnGetListNames&& InOnGetListNames)
			: IPropertyTypeCustomization()
			, OnGetListNames(MoveTemp(InOnGetListNames))
		{}

	private:

		//~ Begin IPropertyTypeCustomization interface
		DATAFLOWEDITOR_API virtual void CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
		virtual void CustomizeChildren(TSharedRef<IPropertyHandle> /*InPropertyHandle*/, IDetailChildrenBuilder& /*ChildBuilder*/, IPropertyTypeCustomizationUtils& /*CustomizationUtils*/) override {}
		//~ End IPropertyTypeCustomization interface

		/**
		 * Return the FManagedArrayCollection with the specified name from the property held by the top level struct owner of ChildPropertyHandle.
		 * @param Context The current editor Dataflow context.
		 * @param ChildPropertyHandle The handle of a customized property from which to find the owner struct.
		 * @param CollectionPropertyName
		 * @return The collection.
		 */
		static DATAFLOWEDITOR_API const FManagedArrayCollection& GetPropertyCollection(
			const TSharedPtr<UE::Dataflow::FContext>& Context,
			const TSharedPtr<IPropertyHandle>& ChildPropertyHandle,
			const FName CollectionPropertyName);

		/** Override to make a list name valid and generate an error message if name is not valid.*/
		virtual bool MakeValidListName(FString& InOutString, FText& OutErrorMessage) const
		{
			OutErrorMessage = FText::GetEmpty();
			return true;
		}

		/** Name of the collection property. Override this method to specify your own collection property name. */
		virtual FName GetCollectionPropertyName() const
		{
			return DefaultCollectionPropertyName;  // By default returns "Collection"
		}

		FText GetText() const;
		void OnTextCommitted(const FText& NewText, ETextCommit::Type CommitInfo);
		void OnSelectionChanged(TSharedPtr<FText> ItemSelected, ESelectInfo::Type SelectInfo);
		bool OnVerifyTextChanged(const FText& Text, FText& OutErrorMessage);
		TSharedRef<ITableRow> MakeCategoryViewWidget(TSharedPtr<FText> Item, const TSharedRef<STableViewBase>& OwnerTable);
		TSharedRef<SWidget> OnGetMenuContent();
		
		static inline const FName DefaultCollectionPropertyName = TEXT("Collection");

		TWeakPtr<const SDataflowGraphEditor> DataflowGraphEditor;
		TSharedPtr<IPropertyHandle> ChildPropertyHandle;
		TWeakPtr<SComboButton> ComboButton;
		TArray<TSharedPtr<FText>> ListNames;

		FOnGetListNames OnGetListNames;
	};
}
