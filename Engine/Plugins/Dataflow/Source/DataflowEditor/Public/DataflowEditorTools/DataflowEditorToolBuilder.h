// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "UObject/Interface.h"

#include "DataflowEditorToolBuilder.generated.h"

namespace UE::Dataflow {
	class IDataflowConstructionViewMode;
}
struct FToolBuilderState;
class UDataflowContextObject;
class UInteractiveTool;

UINTERFACE(MinimalAPI)
class UDataflowEditorToolBuilder : public UInterface
{
	GENERATED_BODY()
};


class IDataflowEditorToolBuilder
{
	GENERATED_BODY()

public:

	/** Returns all Construction View modes that this tool can operate in. The first element should be the preferred mode to switch to if necessary. */
	virtual void GetSupportedConstructionViewModes(const UDataflowContextObject& ContextObject, TArray<const UE::Dataflow::IDataflowConstructionViewMode*>& Modes) const = 0;

	/** Returns whether or not view can be set to wireframe when this tool is active.. */
	virtual bool CanSetConstructionViewWireframeActive() const { return true; }

	/** Returns true if the tool can keep running when the SceneState changes */
	virtual bool CanSceneStateChange(const UInteractiveTool* ActiveTool, const FToolBuilderState& SceneState) const { return false; }

	/** Respond to SceneState changing */
	virtual void SceneStateChanged(UInteractiveTool* ActiveTool, const FToolBuilderState& SceneState) 
	{ 
		checkf(CanSceneStateChange(ActiveTool, SceneState), TEXT("Current tool cannot handle changing scene state while running"));
	};
};

