// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "UObject/NoExportTypes.h"
#include "BaseTools/BaseBrushTool.h"
#include "BaseTools/MeshSurfacePointMeshEditingTool.h"
#include "Components/DynamicMeshComponent.h"

#include "Sculpting/MeshSculptToolBase.h"

#include "DynamicMesh/DynamicMeshOctree3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"

#include "ToolDataVisualizer.h"
#include "GroupTopology.h"
#include "MeshVertexAttributePaintToolBase.h"

#include "DataflowEditorTools/DataflowEditorToolBuilder.h"


#include "DataflowEditorVertexAttributePaintTool.generated.h"


#define UE_API DATAFLOWEDITOR_API

class UDataflowContextObject;
struct FDataflowVertexAttributeEditableNode;
struct FDataflowVertexAttributeProviderNode;
class UDataflowEditorMode;
class UInteractiveToolManager;

namespace UE::DataflowEditorVertexAttributePaintTool::CVars
{
	extern bool DataflowEditorUseNewWeightMapTool;
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * Tool Builder
 */
UCLASS(MinimalAPI)
class UDataflowEditorVertexAttributePaintToolBuilder : public UMeshSurfacePointMeshEditingToolBuilder, public IDataflowEditorToolBuilder
{
	GENERATED_BODY()
public:
	void SetFallbackToolBuilder(UMeshSurfacePointMeshEditingToolBuilder* InFallbackToolBuilder) { FallbackToolBuilder = InFallbackToolBuilder; }
	UE_API void SetEditorMode(UDataflowEditorMode* InMode);

private:
	UE_API virtual void GetSupportedConstructionViewModes(const UDataflowContextObject& ContextObject, TArray<const UE::Dataflow::IDataflowConstructionViewMode*>& Modes) const override;
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual UMeshSurfacePointTool* CreateNewTool(const FToolBuilderState& SceneState) const override;
	virtual bool CanSetConstructionViewWireframeActive() const { return false; }

	// Per-editor mode lookup keyed by the editor's tool manager. This builder is held by the process-wide FDataflowToolRegistry singleton, 
	// so a single Mode pointer would alias across simultaneously-open Dataflow editors; the map keeps each editor's mode separate.
	TMap<TWeakObjectPtr<const UInteractiveToolManager>, TWeakObjectPtr<UDataflowEditorMode>> ModeForToolManager;

	UPROPERTY()
	TObjectPtr<UMeshSurfacePointMeshEditingToolBuilder> FallbackToolBuilder = nullptr;
};

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

struct FDataflowEditorVertexAttributePaintToolDataAdapter
{
public:
	void Setup(
		const FDynamicMesh3& InMesh,
		UE::Geometry::FDynamicMeshWeightAttribute* InActiveWeightMap,
		FDataflowVertexAttributeEditableNode* InNodeToUpdate,
		TObjectPtr<UDataflowContextObject> InDataflowEditorContextObject
		);
	
	void CommitToNode(TObjectPtr<UDataflowContextObject> InDataflowEditorContextObject, IToolsContextTransactionsAPI* TransactionAPI);

	bool HaveDynamicMeshToWeightConversion() const { return bHaveDynamicMeshToWeightConversion; }
	const TArray<int32>& GetDynamicMeshToWeight() const { return  DynamicMeshToWeight; }
	const TArray<TArray<int32>>& GetWeightToDynamicMesh() const { return WeightToDynamicMesh; }

private:
	// DynamicMesh might be unwelded mesh, but weights are on the welded mesh.
	bool bHaveDynamicMeshToWeightConversion = false;
	TArray<int32> DynamicMeshToWeight;
	TArray<TArray<int32>> WeightToDynamicMesh;

	// Corresponding node to read from and write back to
	FDataflowVertexAttributeEditableNode* NodeToUpdate = nullptr;

	UE::Geometry::FDynamicMeshWeightAttribute* ActiveWeightMap = nullptr;
};

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * Dataflow Editor tool to paint a vertex attribute as vertex colors
 */
UCLASS(MinimalAPI)
class UDataflowEditorVertexAttributePaintTool : public UMeshVertexAttributePaintToolBase
{
	GENERATED_BODY()

public:
	UE_API virtual bool SetupToolMesh(FDynamicMesh3& InOutToolMesh, int32& OutInitialAttributeIndex) override;
	UE_API virtual void CommitToolMesh(FDynamicMesh3& InToolMesh) override;
	
public:

	UE_API virtual void Setup() override;

	UE_API void SetDataflowEditorContextObject(TObjectPtr<UDataflowContextObject> InDataflowEditorContextObject);
	void SetEditorMode(UDataflowEditorMode* InMode) { Mode = InMode; }

	//~ UObject interface
	static UE_API void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);

public:
	UE_API virtual void SetFocusInViewport() const override;


	//
	// Internals
	//

protected:
	UPROPERTY()
	TObjectPtr<UDataflowContextObject> DataflowEditorContextObject = nullptr;

	FDataflowEditorVertexAttributePaintToolDataAdapter DataAdapter;

protected:
	friend class UDataflowEditorVertexAttributePaintToolBuilder;

	UDataflowEditorMode* Mode = nullptr;

};




#undef UE_API

