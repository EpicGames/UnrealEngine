// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataflowRendering/DataflowRenderableTypeSettings.h"
#include "Debug/DebugDrawComponent.h"
#include "Curves/LinearColorRamp.h"
#include "UObject/ObjectPtr.h"

#include "DataflowDebugMeshComponent.generated.h"

UCLASS(MinimalAPI, Hidden)
class UDataflowDebugMeshComponent : public UDebugDrawComponent
{
	GENERATED_BODY()

public:
	using FText3d = FDebugRenderSceneProxy::FText3d;
	using FLine = FDebugRenderSceneProxy::FDebugLine;

	TArray<FText3d> IDs;
	TArray<FLine> Normals;

private:
	virtual FDebugRenderSceneProxy* CreateDebugSceneProxy() override;
	virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;
};
