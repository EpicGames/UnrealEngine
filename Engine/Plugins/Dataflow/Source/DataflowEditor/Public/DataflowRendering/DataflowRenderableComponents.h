// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Templates/UnrealTypeTraits.h"
#include "UObject/UnrealNames.h"
#include "GameFramework/Actor.h"
#include "Components/PrimitiveComponent.h"

namespace UE::Dataflow
{
	/**
	* List of renderable components for a sepcific root actor
	* this class is used by IRenderableType to collect the components to render 
	*/
	struct FRenderableComponents
	{
	public:
		FRenderableComponents(AActor* InRootActor)
			: RootActor(InRootActor)
		{}

		/**
		* Create a new component of a specific type ( needs to inherit from UPrimitiveComponent )
		* and add it to the list 
		* returns  the newly added component so that it can be configured by the caller 
		*/
		template<typename T UE_REQUIRES(TIsDerivedFrom<T, UPrimitiveComponent>::Value)>
		T* AddNewComponent(FName ComponentName = NAME_None, UObject* ComponentParent = nullptr)
		{
			UObject* ParentObject = ComponentParent ? ComponentParent : RootActor;

			const FName ObjectName = MakeUniqueObjectName(ParentObject, T::StaticClass(), FName(ComponentName));
			
			// for now always allocate a new one - in the future we could use a pool 
			T* NewComponent = NewObject<T>(ParentObject, ObjectName);
			if (NewComponent)
			{
				if (USceneComponent* ParentObjectAsComp = Cast<USceneComponent>(ParentObject))
				{
					NewComponent->SetupAttachment(ParentObjectAsComp);
				}
				Components.Add(NewComponent);
			}
			return NewComponent;
		}

		template<typename T UE_REQUIRES(TIsDerivedFrom<T, UObject>::Value)>
		T* MakeNewObject(FName ObjectName = NAME_None)
		{
			return NewObject<T>(RootActor);
		}

		const TArray<UPrimitiveComponent*>& GetComponents() const { return Components; }

	private:
		AActor* RootActor = nullptr;
		TArray<UPrimitiveComponent*> Components;
	};
}
