// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"

namespace UE::Dataflow
{
	struct FRenderableComponents;
	struct FRenderableTypeInstance;
	class IDataflowConstructionViewMode;

	/**
	* this interface defines a way to render a specific data type in Dataflow
	* rendering is achieve by provided a number of components that will be added to the rednering scene
	*/
	struct IRenderableType
	{
	public:
		/** Returns the renderable primary type (ie: TObjectPtr<UDynamicMesh>) */
		virtual FName GetOutputType() const = 0;

		/** Returns the supported render group, useful when rendering a specific type in a particular way (ie : "Surface" may render the geometry related groups ) */
		virtual FName GetRenderGroup() const { return NAME_None; }

		/** returns the class of the settings object if any */
		virtual void GetSettingsClasses(TArray<const UClass*>& OutArray) const { }

		/** return true if the ViewMode is supported for rendering this */
		virtual bool IsViewModeSupported(const IDataflowConstructionViewMode& ViewMode) const = 0;

		/** Whether this type can be rendered in this context */
		virtual bool CanRender(const FRenderableTypeInstance& Instance) const { return true; }

		/** Get the primitive components for the specific render context  */
		virtual void GetPrimitiveComponents(const FRenderableTypeInstance& Instance, FRenderableComponents& OutComponents) const = 0;
	};
}

// Macros to ease the definition of renderable types

#define UE_DATAFLOW_IRENDERABLE_PRIMARY_TYPE(Type, Name) \
	static inline const FName OutputTypeName = TEXT(#Type); \
	virtual FName GetOutputType() const override \
		{ return OutputTypeName; } \
	const Type& Get##Name(const UE::Dataflow::FRenderableTypeInstance& Instance, const Type& Default = Type()) const \
		{ return Instance.GetOutputValue<Type>(Default); }

#define UE_DATAFLOW_IRENDERABLE_RENDER_GROUP(Name) \
	static inline const FName RenderGroupName = TEXT(#Name); \
	virtual FName GetRenderGroup() const override \
		{ return RenderGroupName; } 

#define UE_DATAFLOW_IRENDERABLE_VIEW_MODE(ViewModeClass) \
	virtual bool IsViewModeSupported(const UE::Dataflow::IDataflowConstructionViewMode& InViewMode) const override \
		{ return (InViewMode.GetName() == ViewModeClass::Name); }

#define UE_DATAFLOW_IRENDERABLE_SETTINGS(RenderSettingsClass) \
	virtual void GetSettingsClasses(TArray<const UClass*>& OutArray) const \
		{ OutArray.Add(RenderSettingsClass::StaticClass()); }

