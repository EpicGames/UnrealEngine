// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "UObject/NameTypes.h"

#include "Dataflow/DataflowNodeParameters.h"
#include "Dataflow/DataflowNode.h"

class UPrimitiveComponent;
class AActor;
struct FDataflowSelection;
class UDataflowRenderableTypeSettings;

#define UE_API DATAFLOWEDITOR_API

namespace UE::Dataflow
{
	struct IRenderableType;
	struct FRenderableComponents;
	class IDataflowConstructionViewMode;

	/**
	* This describe a renderable type instance which match to a specific node + output(s)
	* And to be rendered in the context a specific Actor and Dataflow context
	*/
	struct FRenderableTypeInstance final
	{
	public:
		/** find all renderable instances for a specific node and a specific view mode */
		static void GetRenderableInstancesForNode(FContext& Context, const IDataflowConstructionViewMode& ViewMode, const FDataflowNode& Node, bool bEvaluateOutputs, TArray<FRenderableTypeInstance>& OutRenderableInstances);

		/** Whether this instance can render */
		bool CanRender() const;

		/** Returns the display name of this instance */
		FString GetDisplayName() const;

		/** Returns the display name of this instance */
		UE_API FName GetComponentName(const FName& InComponentName) const;

		/** Get the corresponding  */
		void GetPrimitiveComponents(FRenderableComponents& OutComponents) const;

		/** 
		* Get value for the primary output to render 
		* Returns the default value if not found of this fails internally 
		*/
		template <typename T>
		const T& GetOutputValue(const T& Default = T()) const
		{
			return GetOutputValue(OutputName, Default);
		}

		bool HasUptoDateCachedValue(FName CacheName = NAME_None) const
		{
			if (Node)
			{
				FContextCacheKey Key = ComputeCacheKey();
				Key = HashCombine(Key, GetTypeHash(CacheName));
				return EvaluationContext.HasData(Key, Node->GetTimestamp());
			}
			return false;
		}

		template<typename T>
		const T& GetCachedValue(FName CacheName = NAME_None) const
		{
			static const T StaticDefaultValue{};
			FContextCacheKey Key = ComputeCacheKey();
			Key = HashCombine(Key, GetTypeHash(CacheName));
			return EvaluationContext.GetData<T>(Key, nullptr, StaticDefaultValue);
		}

		template<typename T>
		void CacheValue(T&& InValue, FName CacheName = NAME_None) const
		{
			if (Node)
			{
				FContextCacheKey Key = ComputeCacheKey();
				Key = HashCombine(Key, GetTypeHash(CacheName));
				EvaluationContext.SetData(Key, nullptr, Forward<T>(InValue), Node->GetGuid(), /*Node->GetValueHash()*/0, Node->GetTimestamp());
			}
		}

		UE_API const FDataflowNode::FAttributeKey GetVertexAttributeToVisualize() const;

		const void GetSelectionToVisualize(FDataflowSelection& OutSelection) const;

		UE_API TArray<UDataflowRenderableTypeSettings*> GetAllRenderSettings() const;
		UE_API UDataflowRenderableTypeSettings* GetRenderSettings(const UClass* SettingsClass) const;

		template <typename T UE_REQUIRES(std::is_base_of_v<UDataflowRenderableTypeSettings, T>)>
		const T* GetTypedRenderSettings() const
		{
			return Cast<T>(GetRenderSettings(T::StaticClass()));
		}

		template <typename T>
		const T& GetOutputValue(FName Name, const T& Default = T()) const
		{
			if (Node)
			{
				if (const FDataflowOutput* Output = Node->FindOutput(Name))
				{
					if (EvaluationContext.IsThreaded() || !bEvaluateOutputs)
					{
						return Output->ReadValue<T>(EvaluationContext, Default);
					}
					return Output->GetValue<T>(EvaluationContext, Default);
				}
			}
			return Default;
		}
		/** Find the first output of a specific match type and optionally containing a specific metadata string and return its value - non-anytype version */
		template <typename T UE_REQUIRES(!std::is_base_of_v<FDataflowAnyType, T>)>
		const T& GetOutputValueByType(const T& Default, FName Metadata = NAME_None) const
		{
			if (Node)
			{
				for (const FDataflowOutput* Output : Node->GetOutputs())
				{
					if (Output && Output->GetType() == TDataflowPolicyTypeName<T>::GetName())
					{
						if (Metadata.IsNone() || HasMetadata(*Output, Metadata))
						{
							return GetOutputValue<T>(Output->GetName(), Default);
						}
					}
				}
			}
			return Default;
		}

		/** Find the first output of a specific match type and return its value - anytype version */
		template <typename T UE_REQUIRES(std::is_base_of_v<FDataflowAnyType, T>)>
		const typename T::FStorageType& GetOutputValueByType(const typename T::FStorageType& Default, FName Metadata = NAME_None) const
		{
			if (Node)
			{
				for (const FDataflowOutput* Output : Node->GetOutputs())
				{
					if (Output && T::FPolicyType::SupportsTypeStatic(Output->GetType()))
					{
						if (Metadata.IsNone() || HasMetadata(*Output, Metadata))
						{
							return GetOutputValue<typename T::FStorageType>(Output->GetName(), Default);
						}
					}
				}
			}
			return Default;
		}

	private:
		UE_API FContextCacheKey ComputeCacheKey() const;

		UE_API static bool HasMetadata(const FDataflowOutput& Output, FName Metadata);

		FRenderableTypeInstance(FContext& InContext, const TSharedPtr<const FDataflowNode>& InNode, const FName InOutputName, const IRenderableType* InRenderableType, bool bEvaluateOutputsIn)
			: EvaluationContext(InContext)
			, Node(InNode)
			, OutputName(InOutputName)
			, RenderableType(InRenderableType)
			, bEvaluateOutputs(bEvaluateOutputsIn)
		{}

	private:
		FContext& EvaluationContext;
		TSharedPtr<const FDataflowNode> Node;
		const FName OutputName;
		const IRenderableType* RenderableType;
		bool bEvaluateOutputs;
	};
}


#undef UE_API

