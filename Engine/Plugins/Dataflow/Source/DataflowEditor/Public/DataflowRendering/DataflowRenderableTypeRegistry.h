// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "UObject/NameTypes.h"

#define UE_API DATAFLOWEDITOR_API

namespace UE::Dataflow
{
	struct IRenderableType;

	/** this registry contains all Dataflow renderable types instances */
	struct FRenderableTypeRegistry
	{
	public:
		using FRenderableTypes = TArray<const IRenderableType*>;

		/** register a renderable type instance - use UE_DATAFLOW_REGISTER_RENDERABLE_TYPE instead of calling this directly for ease of use */
		UE_API void Register(const IRenderableType* RenderableType);

		/** Get all the available renderable types for a specific primary type */
		UE_API const FRenderableTypes& GetRenderableTypes(FName PrimaryType) const;

		/** get the one and only instance of this registry */
		UE_API static FRenderableTypeRegistry& GetInstance();

	private:
		FRenderableTypeRegistry() = default;

	private:
		TMap<FName, FRenderableTypes> RenderableTypesByPrimaryType;
		TSet<FName> RegisteredAnyTypes;
	};
}

#undef UE_API

// macro to use in a module / plugin to register a specific renderable type class

#define UE_DATAFLOW_REGISTER_RENDERABLE_TYPE(Type) \
	static const Type InstanceOf##Type; \
	UE::Dataflow::FRenderableTypeRegistry::GetInstance().Register(&InstanceOf##Type);
