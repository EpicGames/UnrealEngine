// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"

#include "DataflowRenderableTypeSettings.generated.h"

UCLASS(MinimalAPI)
class UDataflowRenderableTypeSettings : public UObject
{
	GENERATED_BODY()
public:
	DATAFLOWEDITOR_API static void RegisterSection(UClass* Class, FName Name, const FText& DisplayName, TConstArrayView<FName> Categories);
};

// Common types

UCLASS(MinimalAPI, AutoExpandCategories = "UVs")
class UDataflowUVsRenderSettings : public UDataflowRenderableTypeSettings
{
	GENERATED_BODY()

public:
	/** return the overriden channel if enabled or the default passed to this method */
	int32 GetUVChannel(int32 DefaultChannel) const
	{
		return bOverrideUVchannel ? UVChannel : DefaultChannel;
	}

private:
	UPROPERTY(EditAnywhere, Category = "UVs", meta = (InlineEditConditionToggle))
	bool bOverrideUVchannel = false;

	/** UV channel to display if oiverride is enabled */
	UPROPERTY(EditAnywhere, Category = "UVs", meta = (UIMIn = 0, UIMax = 7, ClampMin = 0, ClampMax = 7, EditCondition = bOverrideUVchannel))
	int32 UVChannel = 0;
};
