// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Curves/LinearColorRamp.h"
#include "DynamicMesh/DynamicMesh3.h"

#include "DataflowRenderableTypeUtils.generated.h"

struct FManagedArrayCollection;
struct FMeshDescription;
struct FKConvexElem;
struct FKAggregateGeom;

class UDynamicMeshComponent;
class USkeletalMesh;
class UStaticMesh;
class UMaterial;
class UMaterialInstanceDynamic;
class UDataflowBoxArrayRenderSettings;

#define UE_API DATAFLOWEDITOR_API

UENUM(BlueprintType)
enum class EDataflowColorMethodType : uint8
{
	Constant UMETA(DisplayName = "Constant"),
	Random UMETA(DisplayName = "Random"),
	BySize UMETA(DisplayName = "By Size"),
	BoundingBox UMETA(DisplayName = "BoundinBox")
};

USTRUCT()
struct FDataflowSimpleColorCommonRenderSettings
{
	GENERATED_BODY()

public:
	/** Display surface as wireframe */
	UPROPERTY(EditAnywhere, Category = "Color")
	bool bWireframe = false;

	/** Transparency */
	UPROPERTY(EditAnywhere, Category = "Color", meta = (UIMin = 0, ClampMin = 0, UIMax = 1, ClampMax = 1))
	float Transparency = 0.f;

	/** Constant color */
	UPROPERTY(EditAnywhere, Category = "ConstantColor")
	FLinearColor Color = FLinearColor::White;
};

USTRUCT()
struct FDataflowColorCommonRenderSettings
{
	GENERATED_BODY()

public:
	UE_API FDataflowColorCommonRenderSettings();

	/** Display surface as wireframe */
	UPROPERTY(EditAnywhere, Category = "Color")
	bool bWireframe = false;

	/** Transparency */
	UPROPERTY(EditAnywhere, Category = "Color", meta = (UIMin = 0, ClampMin = 0, UIMax = 1, ClampMax = 1))
	float Transparency = 0.f;

	/** Method for coloring the boxes */
	UPROPERTY(EditAnywhere, Category = "Color")
	EDataflowColorMethodType ColorMethod = EDataflowColorMethodType::Constant;

	/** Constant color */
	UPROPERTY(EditAnywhere, Category = "ConstantColor", meta = (EditCondition = "ColorMethod == EDataflowColorMethodType::Constant"))
	FLinearColor Color = FLinearColor::White;

	/** Seed for the random color */
	UPROPERTY(EditAnywhere, Category = "RandomColor", meta = (UIMin = 0, EditCondition = "ColorMethod == EDataflowColorMethodType::Random"))
	int32 RandomSeed = 0;

	/** Minimum size */
	UPROPERTY(EditAnywhere, Category = "ColorBySize", meta = (UIMin = 0, ClampMin = 0, EditCondition = "ColorMethod == EDataflowColorMethodType::BySize"))
	float SizeMin = 1.f;

	/** Maximum size */
	UPROPERTY(EditAnywhere, Category = "ColorBySize", meta = (UIMin = 0, ClampMin = 0, EditCondition = "ColorMethod == EDataflowColorMethodType::BySize"))
	float SizeMax = 100.f;

	/** Color ramp for colorin by scale */
	UPROPERTY(EditAnywhere, Category = "ColorBySize", meta = (EditCondition = "ColorMethod == EDataflowColorMethodType::BySize"))
	FLinearColorRamp ColorRamp;

	/** The normalized position of the center of the object in the component bounding box is computed.
		Then the X or 1-X, Y or 1-Y, Z or 1-Z of the normalized position is set to the (R,G,B) values. 
		There are 32 different permutation. The index specifies which one gets used. */
	UPROPERTY(EditAnywhere, Category = "ColorByBoundingBox", meta = (UIMin = 0, UIMax = 23, EditCondition = "ColorMethod == EDataflowColorMethodType::BoundingBox"))
	int32 PermutationIndex = 0;
};

namespace UE::Dataflow
{
	struct FRenderableComponents;
	struct FRenderableTypeInstance;

	namespace Rendering
	{
		/** Get the default vertex material for dynamic meshes */
		UE_API UMaterial* GetVertexMaterial();
		
		/** Generate a dynamic mesh from a mesh description UV data */
		TSharedPtr<UE::Geometry::FDynamicMesh3> GenerateUvMesh(const FMeshDescription& MeshDescription, int32 UvChannel);

		/** Generate a dynamic mesh from a static mesh UV data */
		TSharedPtr<UE::Geometry::FDynamicMesh3> GenerateUvMesh(const UStaticMesh& StaticMesh, int32 UvChannel);

		/** Generate a dynamic mesh from a skeletal mesh UV data */
		TSharedPtr<UE::Geometry::FDynamicMesh3> GenerateUvMesh(const USkeletalMesh& SkeletalMesh, int32 UvChannel);

		/** Generate a dynamic mesh from a dynamic mesh UV data */
		TSharedPtr<UE::Geometry::FDynamicMesh3> GenerateUvMesh(const UE::Geometry::FDynamicMesh3& DynamicMesh, int32 UvChannel);

		/** Generate a dynamic mesh from a managed array collection UV data */
		TSharedPtr<UE::Geometry::FDynamicMesh3> GenerateUvMesh(const FManagedArrayCollection& Collection, int32 UvChannel);

		/** Make a dynamic mesh component and add it to the OutComponents */
		UDynamicMeshComponent* AddDynamicMeshComponent(UE::Geometry::FDynamicMesh3&& DynamicMesh, FRenderableComponents& OutComponents);

		/** set UV material on a primitive component */
		void SetUvMaterial(UDynamicMeshComponent& Component, int32 MaterialIndex);

		/** Generate a Uv mesh from a templated source and make a dynamic mesh component out of it and add it to the OutComponents */
		template<typename T>
		void AddUvDynamicMeshComponent(const T& Source, int32 UvIndex, FRenderableComponents& OutComponents)
		{
			if (TSharedPtr<UE::Geometry::FDynamicMesh3> UvMesh = GenerateUvMesh(Source, UvIndex))
			{
				if (UDynamicMeshComponent* Component = AddDynamicMeshComponent(MoveTemp(*UvMesh), OutComponents))
				{
					SetUvMaterial(*Component, 0);
				}
			}
		}

		int32 GetUVChannelFromInstance(const FRenderableTypeInstance& Instance, int32 DefaultValue = 0);

		UE_API FLinearColor GetColor(const int32 InIdx, const FBox& InComponentBoundingBox, const FVector& InCenter, float InSize,
			const FDataflowColorCommonRenderSettings& InSettings);
	}

	namespace RenderGeometry
	{
		/** Build a uniform capsule mesh centered at origin, aligned along +Z. Total height = Length + 2 * Radius. */
		UE_API UE::Geometry::FDynamicMesh3 BuildCapsuleMesh(float Radius, float Length);

		/** Build a tapered capsule mesh centered at origin, aligned along +Z. */
		UE_API UE::Geometry::FDynamicMesh3 BuildTaperedCapsuleMesh(float Radius0, float Radius1, float Length, int32 ArcSteps = 6, int32 CircleSteps = 12);

		/** Build a triangle mesh from a convex element's pre-computed vertex/index data. Returns an empty mesh if data is missing. */
		UE_API UE::Geometry::FDynamicMesh3 BuildConvexMesh(const FKConvexElem& Elem);

		/**
		 * Add UPrimitiveComponent instances for all shape elements in AggGeom to OutComponents.
		 * BoneTransform: component-space transform of the owning bone (pass Identity for standalone geometry).
		 * BoneName: used only for component naming (pass NAME_None for standalone geometry).
		 * InOutShapeCounter: incremented once per shape added; share a single counter across calls for unique names.
		 */
		UE_API void AddAggGeomComponents(
			const FKAggregateGeom& AggGeom,
			const FTransform& BoneTransform,
			FName BoneName,
			FRenderableComponents& OutComponents,
			UObject* ComponentParent,
			UMaterialInstanceDynamic* MaterialInstance,
			int32& InOutShapeCounter);
	}
}

#undef UE_API
