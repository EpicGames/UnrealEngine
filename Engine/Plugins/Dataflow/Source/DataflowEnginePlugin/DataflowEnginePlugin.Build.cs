// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class DataflowEnginePlugin : ModuleRules
	{
		public DataflowEnginePlugin(ReadOnlyTargetRules Target) : base(Target)
		{
			bTreatAsEngineModule = true;
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"GeometryFramework"
				}
			);
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					// NOTE: UVEditorTools is a separate module so that it doesn't rely on the editor.
					// So, do not add editor dependencies here.
	
					"Engine",
					"RenderCore",
					"RHI",
					"DataflowCore",
					"DataflowEngine",
					"Chaos",
					"GeometryCore",
					"DynamicMesh"
				}
			);
		}
	}
}
