// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowActor.h"

#include "CoreMinimal.h"
#include "Dataflow/DataflowComponent.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowActor)

DEFINE_LOG_CATEGORY_STATIC(ADataflowLogging, Log, All);

ADataflowActor::ADataflowActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	//UE_LOGF(ADataflowLogging, Verbose, "ADataflowActor::ADataflowActor()");
	DataflowComponent = CreateDefaultSubobject<UDataflowComponent>(TEXT("DataflowComponent0"));
	RootComponent = DataflowComponent;
	PrimaryActorTick.bCanEverTick = true;
}

#if WITH_EDITOR
bool ADataflowActor::GetReferencedContentObjects(TArray<UObject*>& Objects) const
{
	return Super::GetReferencedContentObjects(Objects);
}
#endif

