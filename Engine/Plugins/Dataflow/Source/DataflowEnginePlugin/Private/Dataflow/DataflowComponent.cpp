// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowComponent.h"
#include "DataflowEngineSceneProxy.h"

#include "Dataflow/DataflowEdNode.h"
#include "GeometryCollection/Facades/CollectionRenderingFacade.h"
#include "GeometryCollection/Facades/CollectionBoundsFacade.h"
#include "GeometryCollection/GeometryCollection.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowComponent)

DEFINE_LOG_CATEGORY_STATIC(LogDataflowRenderComponentInternal, Log, All);

UDataflowComponent::UDataflowComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	PrimaryComponentTick.bCanEverTick = true;
	bTickInEditor = true;
}

UDataflowComponent::~UDataflowComponent()
{
}

void UDataflowComponent::Invalidate()
{
	bUpdateRender = true;
	bUpdateSelection = true;
	bBoundsNeedsUpdate = true;
	SelectionState.Vertices.Empty();
	RenderCollection = FManagedArrayCollection();
}


void UDataflowComponent::ResetRenderTargets() 
{ 
	RenderTargets.Reset();
	Invalidate();
}

void UDataflowComponent::AddRenderTarget(const UDataflowEdNode* InTarget) 
{ 
	RenderTargets.AddUnique(InTarget); 
	Invalidate();
}


void UDataflowComponent::UpdateLocalBounds()
{
	if (bBoundsNeedsUpdate)
	{
		GeometryCollection::Facades::FBoundsFacade BoundsFacade(RenderCollection);
		BoundsFacade.DefineSchema();

		BoundsFacade.UpdateBoundingBox();
		BoundingBox = BoundsFacade.GetBoundingBoxInCollectionSpace();

		bBoundsNeedsUpdate = false;

		UpdateBounds();
	}
}

FBoxSphereBounds UDataflowComponent::CalcBounds(const FTransform& LocalToWorldIn) const
{
	return BoundingBox.TransformBy(GetComponentTransform());
}

void UDataflowComponent::SetRenderingCollection(FManagedArrayCollection&& InCollection)
{
	Invalidate();
	RenderCollection = InCollection;
}

const FManagedArrayCollection& UDataflowComponent::GetRenderingCollection() const
{
	return RenderCollection;
}

FManagedArrayCollection& UDataflowComponent::ModifyRenderingCollection()
{
	return RenderCollection;
}

// Deprecated in 5.7
FMaterialRelevance UDataflowComponent::GetMaterialRelevance(ERHIFeatureLevel::Type InFeatureLevel) const
{
	return GetMaterialRelevance(GetFeatureLevelShaderPlatform_Checked(InFeatureLevel));
}

FMaterialRelevance UDataflowComponent::GetMaterialRelevance(EShaderPlatform InShaderPlatform) const
{
	// Combine the material relevance for all materials.
	FMaterialRelevance Result;
	for (int32 ElementIndex = 0; ElementIndex < GetNumMaterials(); ElementIndex++)
	{
		UMaterialInterface const* MaterialInterface = GetMaterial(ElementIndex);
		if (!MaterialInterface)
		{
			MaterialInterface = UMaterial::GetDefaultMaterial(MD_Surface);
		}
		Result |= MaterialInterface->GetRelevance_Concurrent(InShaderPlatform);
	}

	//if (OverlayMaterial != nullptr)
	//{
	//	Result |= OverlayMaterial->GetRelevance_Concurrent(InFeatureLevel);
	//}

	return Result;
}

FPrimitiveSceneProxy* UDataflowComponent::CreateSceneProxy()
{
	//this->bHasPerInstanceHitProxies = true; // just makes OnClick return null in the hit proxy
	return GeometryCollection::Facades::FRenderingFacade(RenderCollection).CanRenderSurface() ? new FDataflowEngineSceneProxy(this) : nullptr;
}

UMaterialInterface* UDataflowComponent::GetMaterial(int32 Index) const
{
	return Dataflow->Material ? Dataflow->Material : GetDefaultMaterial();
}

UMaterialInterface* UDataflowComponent::GetDefaultMaterial() const
{
	const TCHAR* Path = TEXT("/Engine/EditorMaterials/Dataflow/DataflowVertexMaterial.DataflowVertexMaterial");
	return LoadObject<UMaterialInterface>(nullptr, Path, nullptr, LOAD_None, nullptr);
}
