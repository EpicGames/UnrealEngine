// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowComponentSelectionState.h"

#include "Dataflow/DataflowComponent.h"
#include "GeometryCollection/Facades/CollectionRenderingFacade.h"


void FDataflowSelectionState::UpdateSelection(UDataflowComponent* DataflowComponent)
{
	using namespace GeometryCollection::Facades;
	FRenderingFacade Facade(DataflowComponent->ModifyRenderingCollection());
	if (Facade.IsValid())
	{
		TManagedArray<int32>& ObjectSelectionArray = Facade.ModifySelectionState();
		TManagedArray<int32>& VertexSelectionArray = Facade.ModifyVertexSelection();

		if (Mode == EMode::DSS_Dataflow_Object)
		{
			if (Nodes.Num())
			{
				const TManagedArray<FString>& GeomNameArray = Facade.GetGeometryName();

				TArray<bool> bVisited;
				bVisited.Init(false, Nodes.Num());

				for (int i = 0; i < ObjectSelectionArray.Num(); i++)
				{
					ObjectID ID(GeomNameArray[i], i);
					ObjectSelectionArray[i] = false;
					int32 IndexOf = Nodes.IndexOfByKey(ID);
					if (IndexOf != INDEX_NONE)
					{
						ObjectSelectionArray[i] = true;
						bVisited[IndexOf] = true;
					}
				}

				// remove unknown selections
				for (int32 Ndx = bVisited.Num() - 1; 0 <= Ndx; Ndx--)
				{
					if (!bVisited[Ndx])
					{
						Nodes.RemoveAt(Ndx);
					}
				}
			}
			else
			{
				ObjectSelectionArray.Fill(0);
			}
		}
		else if (Mode == EMode::DSS_Dataflow_Vertex)
		{
			VertexSelectionArray.Fill(0);

			int32 NumVertices = VertexSelectionArray.Num();
			for (int32 Index : Vertices)
			{
				if (Index < NumVertices)
				{
					VertexSelectionArray[Index] = 1;
				}
			}
		}
		else
		{
			VertexSelectionArray.Fill(0);
			ObjectSelectionArray.Fill(0);

		}
	}
}
