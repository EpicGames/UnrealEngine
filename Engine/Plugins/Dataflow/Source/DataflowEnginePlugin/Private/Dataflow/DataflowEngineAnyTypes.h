// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Dataflow/DataflowAnyType.h"
#include "Dataflow/DataflowTypePolicy.h"
#include "UObject/ObjectPtr.h"

#include "DataflowEngineAnyTypes.generated.h"

class UDynamicMesh;

// todo : move this to general type policy file ?
template<typename T>
struct FDataflowTypedArrayTypePolicy : 
public TDataflowMultiTypePolicy<T, TArray<T>>
{
};

/**
* Dynamic mesh array TArray with compatibility to plug one single element
*/
USTRUCT()
struct FDataflowDynamicMeshArray : public FDataflowAnyType
{
	using FPolicyType = FDataflowTypedArrayTypePolicy<TObjectPtr<UDynamicMesh>>;
	using FStorageType = TArray<TObjectPtr<UDynamicMesh>>;

	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category=Value)
	TArray<TObjectPtr<UDynamicMesh>> Value;
};

UE_DATAFLOW_POLICY_DECLARE_TYPENAME(TObjectPtr<UDynamicMesh>)

template<>
struct FDataflowConverter<TArray<TObjectPtr<UDynamicMesh>>>
{
	template <typename TFromType>
	static void From(const TFromType& From, TArray<TObjectPtr<UDynamicMesh>>& To)
	{
		if constexpr (std::is_same_v<TFromType, TObjectPtr<UDynamicMesh>>)
		{
			To.Init(From, 1);
		}
		else if constexpr (std::is_same_v<TFromType, TArray<TObjectPtr<UDynamicMesh>>>)
		{
			To = From;
		}
	}

	template <typename TToType>
	static void To(const TArray<TObjectPtr<UDynamicMesh>>& From, TToType& To)
	{
		if constexpr (std::is_same_v<TToType, TObjectPtr<UDynamicMesh>>)
		{
			To = (From.Num()) ? From[0] : TObjectPtr<UDynamicMesh>();
		}
		else if constexpr (std::is_same_v<TToType, TArray<TObjectPtr<UDynamicMesh>>>)
		{
			To = From;
		}

		
	}
};

namespace UE::Dataflow
{
	void RegisterEngineAnyTypes();
}
