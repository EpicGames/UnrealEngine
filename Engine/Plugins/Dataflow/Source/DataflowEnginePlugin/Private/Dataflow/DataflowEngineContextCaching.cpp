// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowEngineContextCaching.h"

#include "Dataflow/DataflowEnginePlugin.h"
#include "Dataflow/DataflowNodeParameters.h"
#include "Dataflow/DataflowContextCachingFactory.h"
#include "GeometryCollection/ManagedArrayCollection.h"

namespace UE::Dataflow
{
	void ContextCachingCallbacks()
	{
		using namespace UE::Dataflow;

		/**
		* Dataflow ContextCaching (FManagedArrayCollection) 
		*
		*		@param Type : FManagedArrayCollection::StaticType()
		* 
		*/
		FContextCachingFactory::GetInstance()->RegisterSerializeFunction(FManagedArrayCollection::StaticType(),
			[](FArchive& Ar, FContextCacheElementBase* Element)
			{
				if (Ar.IsSaving())
				{
					check(Element!=nullptr); // read from a explicit type and return a null element
					
					const FManagedArrayCollection EmptyCollection;
					// cache always return const data , because they are immutable, however the serialize method is not const and require a const_cast
					FManagedArrayCollection& Collection = const_cast<FManagedArrayCollection&>(FContextCachingFactory::GetTypedElement<FManagedArrayCollection>(Element, EmptyCollection));
					Chaos::FChaosArchive ChaosAr(Ar);
					Collection.Serialize(ChaosAr);

					return (FContextCacheElementBase*)nullptr;
				}
				else if(Ar.IsLoading())
				{
					check(Element == nullptr); // write into a explicit type and return a new element
					
					FManagedArrayCollection Collection;
					Chaos::FChaosArchive ChaosAr(Ar);
					Collection.Serialize(ChaosAr);

					return FContextCachingFactory::NewTypedElement<FManagedArrayCollection>(MoveTemp(Collection));
				}
				return (FContextCacheElementBase*)nullptr;
			});

	}

}
