// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"


namespace UE::Dataflow
{
	/**
	 * Rendering callbacks for the Dataflow Rendering Factory
	 */
	void ContextCachingCallbacks();

}