// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowEnginePlugin.h"

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "Dataflow/DataflowEngineContextCaching.h"
#include "Dataflow/DataflowEngineAnyTypes.h"
#include "GeometryCollection/GeometryCollectionUtility.h"
#include "GeometryCollection/Facades/CollectionRenderingFacade.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"


FColor IDataflowEnginePlugin::VertexColor = FLinearColor(0.0,0.0,0.0).ToRGBE();
FColor IDataflowEnginePlugin::SelectionPrimaryColor = FLinearColor(0.8f, 0.4f, 0.0f).ToRGBE();
FColor IDataflowEnginePlugin::SelectionLockedPrimaryColor = FLinearColor(0.8f, 0.4f, 0.2f).ToRGBE();

class FDataflowEnginePlugin : public IDataflowEnginePlugin
{
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};

IMPLEMENT_MODULE( FDataflowEnginePlugin, DataflowEnginePlugin)



void FDataflowEnginePlugin::StartupModule()
{
	UE::Dataflow::ContextCachingCallbacks();
	UE::Dataflow::RegisterEngineAnyTypes();
	FModuleManager::Get().LoadModule("DataflowEngine");
	FModuleManager::Get().LoadModule("DataflowSimulation");
}


void FDataflowEnginePlugin::ShutdownModule()
{
	
}



