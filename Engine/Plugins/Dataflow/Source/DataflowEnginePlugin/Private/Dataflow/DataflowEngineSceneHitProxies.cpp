// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowEngineSceneHitProxies.h"
#include "GenericPlatform/ICursor.h"

IMPLEMENT_HIT_PROXY(HDataflowDefault, HActor);
IMPLEMENT_HIT_PROXY(HDataflowNode, HActor);
IMPLEMENT_HIT_PROXY(HDataflowVertex, HActor);

HDataflowDefault::HDataflowDefault(AActor* InActor, const UPrimitiveComponent* InPrimitiveComponent)
	: HActor(InActor, InPrimitiveComponent) {}

EMouseCursor::Type HDataflowDefault::GetMouseCursor()
{
	return EMouseCursor::Default;
}

HDataflowNode::HDataflowNode(AActor* InActor, const UPrimitiveComponent* InPrimitiveComponent, FString InNodeName, int32 InGeometryIndex)
	: HActor(InActor, InPrimitiveComponent)
{
	NodeName = InNodeName;
	GeometryIndex = InGeometryIndex;
	SectionIndex = GeometryIndex;
}

EMouseCursor::Type HDataflowNode::GetMouseCursor()
{
	return EMouseCursor::Default;
}

HDataflowVertex::HDataflowVertex(AActor* InActor, const UPrimitiveComponent* InPrimitiveComponent, int32 InVertexIndex)
	: HActor(InActor, InPrimitiveComponent)
{
	SectionIndex = InVertexIndex;
}

EMouseCursor::Type HDataflowVertex::GetMouseCursor()
{
	return EMouseCursor::Crosshairs;
}

