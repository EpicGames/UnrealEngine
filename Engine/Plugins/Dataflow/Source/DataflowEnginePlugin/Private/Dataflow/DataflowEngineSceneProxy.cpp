// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataflowEngineSceneProxy.h"

#include "Async/ParallelFor.h"
#include "Dataflow/DataflowComponent.h"
#include "Dataflow/DataflowEnginePlugin.h"
#include "Dataflow/DataflowEngineSceneHitProxies.h"
#include "DynamicMeshBuilder.h"
#include "Engine/CollisionProfile.h"
#include "GeometryCollection/Facades/CollectionRenderingFacade.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "PrimitiveSceneProxy.h"
#include "SceneInterface.h"
#include "StaticMeshResources.h"

const TArray<FVector3f> BoxVertexArray = {
	{-1.0, -1.0, -1.0}, {1.0, -1.0, -1.0}, {1.0, -1.0, 1.0}, {-1.0, -1.0, 1.0},
	{-1.0, 1.0, -1.0}, {1.0, 1.0, -1.0}, {1.0, 1.0, 1.0}, {-1.0, 1.0, 1.0},
};

const TArray<FIntVector> BoxIndicesArray = {
	{1, 5, 4}, {2, 6, 5}, {3, 7, 6}, {0, 4, 7}, {2, 1, 0}, {5, 6, 7},
	{7, 4, 5}, {0, 3, 2}, {7, 3, 0}, {6, 2, 3}, {5, 1, 2}, {4, 0, 1}
};


FDataflowEngineSceneProxy::FDataflowEngineSceneProxy(UDataflowComponent* Component)
	: FPrimitiveSceneProxy(Component)
	, DataflowComponent(Component)
	, ConstantData(Component->GetRenderingCollection().NewCopy())
	, RenderMaterial(UMaterial::GetDefaultMaterial(MD_Surface))
	, MaterialRelevance(Component->GetMaterialRelevance(GetScene().GetShaderPlatform()))
	, VertexFactory(GetScene().GetFeatureLevel(), "FDataflowEngineMeshVertexFactory")
	, BoxVertexFactory(GetScene().GetFeatureLevel(), "FDataflowEngineBoxInstanceVertexFactory")
{
	RenderMaterial = GetRenderMaterial();
}

FDataflowEngineSceneProxy::~FDataflowEngineSceneProxy() 
{}

UMaterialInterface* FDataflowEngineSceneProxy::GetRenderMaterial() const
{
	UMaterialInterface* RetRenderMaterial = DataflowComponent->GetMaterial(0);
	if (RetRenderMaterial == nullptr)
	{
		RetRenderMaterial = UMaterial::GetDefaultMaterial(MD_Surface);
	}
	return RetRenderMaterial;
}

void FDataflowEngineSceneProxy::CreateRenderThreadResources(FRHICommandListBase& RHICmdList)
{
	check(ConstantData);
	check(RenderMaterial);

#if WITH_EDITOR
	SetUsedMaterialForVerification({RenderMaterial});
#endif

	CreateInstancedVertexRenderThreadResources(RHICmdList);
	CreateMeshRenderThreadResources(RHICmdList);
}


void FDataflowEngineSceneProxy::DestroyRenderThreadResources()
{
	DestroyInstancedVertexRenderThreadResources();
	DestroyMeshRenderThreadResources();

	delete ConstantData;
	ConstantData = nullptr;
}


#if WITH_EDITOR
HHitProxy* FDataflowEngineSceneProxy::CreateHitProxies(UPrimitiveComponent* Component, TArray<TRefCountPtr<HHitProxy> >& OutHitProxies)
{
	check(!IsInRenderingThread());

	GeometryCollection::Facades::FRenderingFacade Facade(*ConstantData);
	if (Facade.NumTriangles() > 0)
	{
		OutHitProxies.Add(new HDataflowDefault(Component->GetOwner(), Component));
		LocalHitProxies.Add(OutHitProxies.Last());

		const FDataflowSelectionState& State = DataflowComponent->GetSelectionState();
		if (State.Mode == FDataflowSelectionState::EMode::DSS_Dataflow_Object)
		{
			const TManagedArray<FString>& NodeName = Facade.GetGeometryName();
			TManagedArray<int32>& GeometryHitProxyIndex = Facade.ModifyGeometryHitProxyIndex();

			int32 NumGeom = Facade.NumGeometry();
			for (int i = 0; i < NumGeom; i++)
			{
				OutHitProxies.Add(new HDataflowNode(Component->GetOwner(), Component, NodeName[i], i));
				LocalHitProxies.Add(OutHitProxies.Last());
				GeometryHitProxyIndex[i] = LocalHitProxies.Num() - 1;
			}
		}
		else if (State.Mode == FDataflowSelectionState::EMode::DSS_Dataflow_Vertex)
		{
			TManagedArray<int32>& VertexHitProxyIndex = Facade.ModifyVertexHitProxyIndex();

			int32 NumVerts = Facade.NumVertices();
			for (int i = 0; i < NumVerts; i++)
			{
				OutHitProxies.Add(new HDataflowVertex(Component->GetOwner(), Component, i));
				LocalHitProxies.Add(OutHitProxies.Last());
				VertexHitProxyIndex[i] = LocalHitProxies.Num() - 1;
			}
		}
	}
	return FPrimitiveSceneProxy::CreateHitProxies(Component, OutHitProxies);
}

#endif // WITH_EDITOR

void FDataflowEngineSceneProxy::GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap,
													   FMeshElementCollector& Collector) const
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_OverlaySceneProxy_GetDynamicMeshElements);

	check(RenderMaterial);

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		if (VisibilityMap & (1 << ViewIndex))
		{
			GetMeshDynamicMeshElements(ViewIndex, Collector);
			GetDynamicInstancedVertexMeshElements(ViewIndex, Collector);
		}
	}
}

//
// Mesh Selection Rendering
//

void FDataflowEngineSceneProxy::CreateMeshRenderThreadResources(FRHICommandListBase& RHICmdList)
{
	GeometryCollection::Facades::FRenderingFacade Facade(*ConstantData);
	const FDataflowSelectionState& State = DataflowComponent->GetSelectionState();
	check(Facade.CanRenderSurface());

	const int32 NumTriangleVertices = Facade.NumTriangles();
	const int32 NumTriangleIndices = Facade.NumTriangles();
	const int32 TotalNumVertices = NumTriangleVertices * 3;
	const int32 TotalNumIndices = NumTriangleIndices * 3;
	constexpr int32 NumTextureCoordinates = 1;

	VertexBuffers.PositionVertexBuffer.Init(TotalNumVertices);
	VertexBuffers.StaticMeshVertexBuffer.Init(TotalNumVertices, NumTextureCoordinates);
	VertexBuffers.ColorVertexBuffer.Init(TotalNumVertices);
	IndexBuffer.Indices.SetNumUninitialized(TotalNumIndices);

	// Initialize
	// Triangles are represented as two tris, all of whose vertices are
	// coincident. The material then offsets them according to the signs of the
	// vertex normals in a camera facing orientation. Size of the point is given
	// by U0.
	if (Facade.NumTriangles() > 0)
	{
		int32 NumGeom = Facade.NumGeometry();
		for (int i = 0; i < NumGeom; i++)
		{
			if (Facade.GetIndicesCount()[i] != 0)
			{
				MeshBatchDatas.Emplace();
				FDataflowTriangleSetMeshBatchData& MeshBatchData = MeshBatchDatas.Last();

				MeshBatchData.MinVertexIndex = Facade.GetIndicesStart()[i] * 3;
				MeshBatchData.MaxVertexIndex = MeshBatchData.MinVertexIndex + Facade.GetIndicesCount()[i] * 3 - 1;
				MeshBatchData.FirstTriangleIndex = Facade.GetIndicesStart()[i] * 3;
				MeshBatchData.NumTriangles = Facade.GetIndicesCount()[i];
				MeshBatchData.GeomIndex = i;
				MeshBatchData.MeshTransform = Facade.GetGeometryTransform()[i];
			}
		}


		const TManagedArray<FIntVector>& Indices = Facade.GetIndices();
		const TManagedArray<FVector3f>& Vertex = Facade.GetVertices();
		const TManagedArray<int32>& SelectionArray = Facade.GetSelectionState();
		const TManagedArray<int32>& GeomIndex = Facade.GetVertexToGeometryIndex();
		const TManagedArray<FLinearColor>& VertexColor = Facade.GetVertexColor();

		// The color stored in the vertices actually gets interpreted as a linear
		// color by the material, whereas it is more convenient for the user of the
		// TriangleSet to specify colors as sRGB. So we actually have to convert it
		// back to linear. The ToFColor(false) call just scales back into 0-255
		// space.
		ParallelFor(Facade.NumTriangles(), [&](int32 i)
		{
			const int32 VertexBufferIndex = 3 * i;
			const int32 IndexBufferIndex = 3 * i;

			const auto& P1 = Vertex[Indices[i][0]];
			const auto& P2 = Vertex[Indices[i][1]];
			const auto& P3 = Vertex[Indices[i][2]];

			VertexBuffers.PositionVertexBuffer.VertexPosition(VertexBufferIndex + 0) = P1;
			VertexBuffers.PositionVertexBuffer.VertexPosition(VertexBufferIndex + 1) = P2;
			VertexBuffers.PositionVertexBuffer.VertexPosition(VertexBufferIndex + 2) = P3;

			FVector3f Tangent1 = (P2 - P1).GetSafeNormal();
			FVector3f Tangent2 = (P3 - P2).GetSafeNormal();
			FVector3f Normal = (Tangent2 ^ Tangent1).GetSafeNormal();

			VertexBuffers.StaticMeshVertexBuffer.SetVertexTangents(VertexBufferIndex + 0, FVector3f(1, 0, 0), FVector3f(0, 1, 0), Normal);
			VertexBuffers.StaticMeshVertexBuffer.SetVertexTangents(VertexBufferIndex + 1, FVector3f(1, 0, 0), FVector3f(0, 1, 0), Normal);
			VertexBuffers.StaticMeshVertexBuffer.SetVertexTangents(VertexBufferIndex + 2, FVector3f(1, 0, 0), FVector3f(0, 1, 0), Normal);

			VertexBuffers.StaticMeshVertexBuffer.SetVertexUV(VertexBufferIndex + 0, 0, FVector2f(0, 0));
			VertexBuffers.StaticMeshVertexBuffer.SetVertexUV(VertexBufferIndex + 1, 0, FVector2f(0, 0));
			VertexBuffers.StaticMeshVertexBuffer.SetVertexUV(VertexBufferIndex + 2, 0, FVector2f(0, 0));

			FColor SelectionColor = (State.Mode == FDataflowSelectionState::EMode::DSS_Dataflow_Object) ? IDataflowEnginePlugin::SelectionPrimaryColor : IDataflowEnginePlugin::SelectionLockedPrimaryColor;
			VertexBuffers.ColorVertexBuffer.VertexColor(VertexBufferIndex + 0) = SelectionArray[GeomIndex[Indices[i][0]]] ? SelectionColor : VertexColor[Indices[i][0]].ToFColor(true);
			VertexBuffers.ColorVertexBuffer.VertexColor(VertexBufferIndex + 1) = SelectionArray[GeomIndex[Indices[i][1]]] ? SelectionColor : VertexColor[Indices[i][1]].ToFColor(true);
			VertexBuffers.ColorVertexBuffer.VertexColor(VertexBufferIndex + 2) = SelectionArray[GeomIndex[Indices[i][2]]] ? SelectionColor : VertexColor[Indices[i][2]].ToFColor(true);

			IndexBuffer.Indices[IndexBufferIndex + 0] = VertexBufferIndex + 0;
			IndexBuffer.Indices[IndexBufferIndex + 1] = VertexBufferIndex + 1;
			IndexBuffer.Indices[IndexBufferIndex + 2] = VertexBufferIndex + 2;
		});
	}
	
	VertexBuffers.PositionVertexBuffer.InitResource(RHICmdList);
	VertexBuffers.StaticMeshVertexBuffer.InitResource(RHICmdList);
	VertexBuffers.ColorVertexBuffer.InitResource(RHICmdList);

	FLocalVertexFactory::FDataType Data;
	VertexBuffers.PositionVertexBuffer.BindPositionVertexBuffer(&VertexFactory, Data);
	VertexBuffers.StaticMeshVertexBuffer.BindTangentVertexBuffer(&VertexFactory, Data);
	VertexBuffers.StaticMeshVertexBuffer.BindTexCoordVertexBuffer(&VertexFactory, Data);
	VertexBuffers.ColorVertexBuffer.BindColorVertexBuffer(&VertexFactory, Data);
	VertexFactory.SetData(RHICmdList, Data);

	VertexFactory.InitResource(RHICmdList);
	IndexBuffer.InitResource(RHICmdList);
}



void FDataflowEngineSceneProxy::DestroyMeshRenderThreadResources()
{
	check(IsInRenderingThread());
	VertexBuffers.PositionVertexBuffer.ReleaseResource();
	VertexBuffers.StaticMeshVertexBuffer.ReleaseResource();
	VertexBuffers.ColorVertexBuffer.ReleaseResource();
	IndexBuffer.ReleaseResource();
	VertexFactory.ReleaseResource();

}

void FDataflowEngineSceneProxy::GetMeshDynamicMeshElements(int32 ViewIndex, FMeshElementCollector& Collector) const
{
	const GeometryCollection::Facades::FRenderingFacade Facade(*ConstantData);
	const TManagedArray<int32>& GeometryHitProxyIndex = Facade.GetGeometryHitProxyIndex();
	const FDataflowSelectionState& State = DataflowComponent->GetSelectionState();

	for (const FDataflowTriangleSetMeshBatchData& MeshBatchData : MeshBatchDatas)
	{
		const FMatrix BatchTransform = GetLocalToWorld() * MeshBatchData.MeshTransform.ToMatrixWithScale();

		FDynamicPrimitiveUniformBuffer& DynamicPrimitiveUniformBuffer = Collector.AllocateOneFrameResource<FDynamicPrimitiveUniformBuffer>();
		DynamicPrimitiveUniformBuffer.Set(Collector.GetRHICommandList(), BatchTransform, BatchTransform, GetBounds(), GetLocalBounds(), true, false, AlwaysHasVelocity());

		FMeshBatch& Mesh = Collector.AllocateMesh();
		Mesh.bWireframe = false;
		Mesh.VertexFactory = &VertexFactory;
		Mesh.MaterialRenderProxy = RenderMaterial->GetRenderProxy();
		Mesh.ReverseCulling = IsLocalToWorldDeterminantNegative();
		Mesh.Type = PT_TriangleList;
		Mesh.DepthPriorityGroup = SDPG_World;
		Mesh.bCanApplyViewModeOverrides = true;

		FMeshBatchElement& BatchElement = Mesh.Elements[0];
		BatchElement.IndexBuffer = &IndexBuffer;
		BatchElement.PrimitiveUniformBufferResource = &DynamicPrimitiveUniformBuffer.UniformBuffer;
		BatchElement.FirstIndex = MeshBatchData.FirstTriangleIndex;
		BatchElement.NumPrimitives = MeshBatchData.NumTriangles;
		BatchElement.MinVertexIndex = MeshBatchData.MinVertexIndex;
		BatchElement.MaxVertexIndex = MeshBatchData.MaxVertexIndex;

#if WITH_EDITOR
		if (State.Mode == FDataflowSelectionState::EMode::DSS_Dataflow_Object)
		{
			Mesh.BatchHitProxyId = LocalHitProxies[GeometryHitProxyIndex[MeshBatchData.GeomIndex]]->Id;
		}
		else
		{
			Mesh.BatchHitProxyId = LocalHitProxies[0]->Id;
		}
#endif
		Collector.AddMesh(ViewIndex, Mesh);
	}
}


//
// Vertex Selection Rendering
//

void FDataflowEngineSceneProxy::CreateInstancedVertexRenderThreadResources(FRHICommandListBase& RHICmdList)
{
	const FDataflowSelectionState& State = DataflowComponent->GetSelectionState();
	if (State.Mode == FDataflowSelectionState::EMode::DSS_Dataflow_Vertex)
	{
		const bool bCanUseGPUScene = UseGPUScene(GetScene().GetShaderPlatform());
		if (bCanUseGPUScene)
		{
			GeometryCollection::Facades::FRenderingFacade Facade(*ConstantData);
			const TManagedArray<FVector3f>& Vertex = Facade.GetVertices();
			const TManagedArray<int32>& VertexSelection = Facade.GetVertexSelection();
			const TManagedArray<int32>& GeometrySelection = Facade.GetSelectionState();
			const TManagedArray<int32>& VertexToGeometryMap = Facade.GetVertexToGeometryIndex();

			NumRenderedVerts = Facade.NumVerticesOnSelectedGeometry();
			const int32 NumElements = BoxIndicesArray.Num() * 3;
			const int32 TotalNumElements = NumElements * NumRenderedVerts;
			constexpr int32 NumTextureCoordinates = 1;

			if (NumRenderedVerts)
			{
				BoxVertexBuffers.PositionVertexBuffer.Init(TotalNumElements);
				BoxVertexBuffers.StaticMeshVertexBuffer.Init(TotalNumElements, NumTextureCoordinates);
				BoxVertexBuffers.ColorVertexBuffer.Init(TotalNumElements);
				BoxIndexBuffer.Indices.SetNumUninitialized(TotalNumElements);

				VertexBatchDatas.SetNum(NumRenderedVerts);
				for (int32 VertIndex = 0, Vdx = 0; VertIndex < Facade.NumVertices(); VertIndex++)
				{
					if (GeometrySelection[VertexToGeometryMap[VertIndex]])
					{
						VertexBatchDatas[Vdx].FirstElementIndex = Vdx * NumElements;
						VertexBatchDatas[Vdx].NumElements = BoxIndicesArray.Num();
						VertexBatchDatas[Vdx].MinVertexIndex = Vdx * NumElements;
						VertexBatchDatas[Vdx].MaxVertexIndex = Vdx * NumElements + NumElements - 1;
						VertexBatchDatas[Vdx].VertexIndex = VertIndex;

						for (int32 Idx = 0; Idx < BoxIndicesArray.Num(); Idx++)
						{
							const int32 BufferIndex = (Vdx * NumElements) + Idx * 3;

							float Scale = 2.0;
							const auto& P1 = BoxVertexArray[BoxIndicesArray[Idx][0]] * Scale + Vertex[VertIndex];
							const auto& P2 = BoxVertexArray[BoxIndicesArray[Idx][1]] * Scale + Vertex[VertIndex];
							const auto& P3 = BoxVertexArray[BoxIndicesArray[Idx][2]] * Scale + Vertex[VertIndex];

							BoxVertexBuffers.PositionVertexBuffer.VertexPosition(BufferIndex + 0) = P1;
							BoxVertexBuffers.PositionVertexBuffer.VertexPosition(BufferIndex + 1) = P2;
							BoxVertexBuffers.PositionVertexBuffer.VertexPosition(BufferIndex + 2) = P3;

							FVector3f Tangent1 = (P2 - P1).GetSafeNormal();
							FVector3f Tangent2 = (P3 - P2).GetSafeNormal();
							FVector3f Normal = (Tangent2 ^ Tangent1).GetSafeNormal();

							BoxVertexBuffers.StaticMeshVertexBuffer.SetVertexTangents(BufferIndex + 0, FVector3f(1, 0, 0), FVector3f(0, 1, 0), Normal);
							BoxVertexBuffers.StaticMeshVertexBuffer.SetVertexTangents(BufferIndex + 1, FVector3f(1, 0, 0), FVector3f(0, 1, 0), Normal);
							BoxVertexBuffers.StaticMeshVertexBuffer.SetVertexTangents(BufferIndex + 2, FVector3f(1, 0, 0), FVector3f(0, 1, 0), Normal);

							BoxVertexBuffers.StaticMeshVertexBuffer.SetVertexUV(BufferIndex + 0, 0, FVector2f(0, 0));
							BoxVertexBuffers.StaticMeshVertexBuffer.SetVertexUV(BufferIndex + 1, 0, FVector2f(0, 0));
							BoxVertexBuffers.StaticMeshVertexBuffer.SetVertexUV(BufferIndex + 2, 0, FVector2f(0, 0));

							FColor FaceColor = VertexSelection[VertIndex] ? IDataflowEnginePlugin::SelectionPrimaryColor : IDataflowEnginePlugin::VertexColor;
							BoxVertexBuffers.ColorVertexBuffer.VertexColor(BufferIndex + 0) = FaceColor;
							BoxVertexBuffers.ColorVertexBuffer.VertexColor(BufferIndex + 1) = FaceColor;
							BoxVertexBuffers.ColorVertexBuffer.VertexColor(BufferIndex + 2) = FaceColor;

							BoxIndexBuffer.Indices[BufferIndex + 0] = BufferIndex + 0;
							BoxIndexBuffer.Indices[BufferIndex + 1] = BufferIndex + 1;
							BoxIndexBuffer.Indices[BufferIndex + 2] = BufferIndex + 2;
						}
						Vdx++;
					}
				}

				BoxVertexBuffers.PositionVertexBuffer.InitResource(RHICmdList);
				BoxVertexBuffers.StaticMeshVertexBuffer.InitResource(RHICmdList);
				BoxVertexBuffers.ColorVertexBuffer.InitResource(RHICmdList);

				FLocalVertexFactory::FDataType Data;
				BoxVertexBuffers.PositionVertexBuffer.BindPositionVertexBuffer(&VertexFactory, Data);
				BoxVertexBuffers.StaticMeshVertexBuffer.BindTangentVertexBuffer(&VertexFactory, Data);
				BoxVertexBuffers.StaticMeshVertexBuffer.BindTexCoordVertexBuffer(&VertexFactory, Data);
				BoxVertexBuffers.ColorVertexBuffer.BindColorVertexBuffer(&VertexFactory, Data);
				BoxVertexFactory.SetData(RHICmdList, Data);

				BoxVertexFactory.InitResource(RHICmdList);
				BoxIndexBuffer.InitResource(RHICmdList);
			}
		}
	}
}

void FDataflowEngineSceneProxy::DestroyInstancedVertexRenderThreadResources()
{
	if (NumRenderedVerts)
	{
		BoxVertexBuffers.PositionVertexBuffer.ReleaseResource();
		BoxVertexBuffers.StaticMeshVertexBuffer.ReleaseResource();
		BoxVertexBuffers.ColorVertexBuffer.ReleaseResource();
		BoxIndexBuffer.ReleaseResource();
		BoxVertexFactory.ReleaseResource();
	}
}

void FDataflowEngineSceneProxy::GetDynamicInstancedVertexMeshElements(int32 ViewIndex, FMeshElementCollector& Collector) const
{
	if (NumRenderedVerts)
	{
		if (DataflowComponent->GetSelectionState().Mode == FDataflowSelectionState::EMode::DSS_Dataflow_Vertex)
		{
			GeometryCollection::Facades::FRenderingFacade Facade(*ConstantData);
			const TManagedArray<FVector3f>& Vertex = Facade.GetVertices();
			const TManagedArray<int32>& VertexHitProxyIndex = Facade.GetVertexHitProxyIndex();
			const int32 NumVertices = Facade.NumVertices();
			const int32 NumElements = BoxIndicesArray.Num() * 3;

			for (const FDataflowVertexBatchData& VertexBatchData : VertexBatchDatas)
			{
				FDynamicPrimitiveUniformBuffer& DynamicPrimitiveUniformBuffer = Collector.AllocateOneFrameResource<FDynamicPrimitiveUniformBuffer>();
				DynamicPrimitiveUniformBuffer.Set(Collector.GetRHICommandList(), GetLocalToWorld(), GetLocalToWorld(), GetBounds(), GetLocalBounds(), true, false, AlwaysHasVelocity());

				FMeshBatch& Mesh = Collector.AllocateMesh();
				Mesh.CastShadow = false;
				Mesh.bUseAsOccluder = false;
				Mesh.VertexFactory = &BoxVertexFactory;
				Mesh.MaterialRenderProxy = RenderMaterial->GetRenderProxy();
				Mesh.ReverseCulling = IsLocalToWorldDeterminantNegative();
				Mesh.Type = PT_TriangleList;
				Mesh.DepthPriorityGroup = SDPG_World;
				Mesh.bCanApplyViewModeOverrides = true;

				FMeshBatchElement& BatchElement = Mesh.Elements[0];
				BatchElement.IndexBuffer = &BoxIndexBuffer;
				BatchElement.PrimitiveUniformBufferResource = &DynamicPrimitiveUniformBuffer.UniformBuffer;
				BatchElement.FirstIndex = VertexBatchData.FirstElementIndex;
				BatchElement.NumPrimitives = VertexBatchData.NumElements;
				BatchElement.MinVertexIndex = VertexBatchData.MinVertexIndex;
				BatchElement.MaxVertexIndex = VertexBatchData.MaxVertexIndex;

#if WITH_EDITOR
				Mesh.BatchHitProxyId = LocalHitProxies[VertexHitProxyIndex[VertexBatchData.VertexIndex]]->Id;
#endif
				Collector.AddMesh(ViewIndex, Mesh);
			}
		}
	}
}




bool FDataflowEngineSceneProxy::CanBeOccluded() const
{
	return false;
}

FPrimitiveViewRelevance FDataflowEngineSceneProxy::GetViewRelevance(const FSceneView* View) const
{
	FPrimitiveViewRelevance Result;
	Result.bDrawRelevance = IsShown(View);
	Result.bShadowRelevance = IsShadowCast(View);
	Result.bDynamicRelevance = true;
	Result.bRenderInMainPass = ShouldRenderInMainPass();
	Result.bUsesLightingChannels =GetLightingChannelMask() != GetDefaultLightingChannelMask();
	Result.bRenderCustomDepth = false;
	Result.bTranslucentSelfShadow = bCastVolumetricTranslucentShadow;
	MaterialRelevance.SetPrimitiveViewRelevance(Result);
	Result.bVelocityRelevance = false;
	return Result;
}

SIZE_T FDataflowEngineSceneProxy::GetTypeHash() const
{
	static SIZE_T UniquePointer;
	return reinterpret_cast<SIZE_T>(&UniquePointer);
}
