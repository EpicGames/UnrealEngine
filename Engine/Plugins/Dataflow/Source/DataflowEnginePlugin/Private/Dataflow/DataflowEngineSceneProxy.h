// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Async/ParallelFor.h"
#include "Dataflow/DataflowEngineSceneHitProxies.h"
#include "DynamicMeshBuilder.h"
#include "Engine/CollisionProfile.h"
#include "Materials/Material.h"
#include "PrimitiveSceneProxy.h"
#include "StaticMeshResources.h"

class UDataflowComponent;
struct FManagedArrayCollection;

struct FDataflowTriangleSetMeshBatchData
{
	int32 FirstTriangleIndex = INDEX_NONE;
	int32 NumTriangles = INDEX_NONE;
	int32 MinVertexIndex = INDEX_NONE;
	int32 MaxVertexIndex = INDEX_NONE;
	int32 GeomIndex = INDEX_NONE;
	FTransform MeshTransform = FTransform::Identity;
};

struct FDataflowVertexBatchData
{
	int32 FirstElementIndex = INDEX_NONE;
	int32 NumElements = INDEX_NONE;
	int32 MinVertexIndex = INDEX_NONE;
	int32 MaxVertexIndex = INDEX_NONE;
	int32 VertexIndex = INDEX_NONE;
};

class FDataflowEngineSceneProxy final : public FPrimitiveSceneProxy
{
public:

	FDataflowEngineSceneProxy(UDataflowComponent* Component);
	virtual ~FDataflowEngineSceneProxy();

	//~ FPrimitiveSceneProxy
	virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override;
	virtual bool CanBeOccluded() const override;
	virtual uint32 GetMemoryFootprint() const override { return sizeof(*this) + GetAllocatedSize(); }
	virtual SIZE_T GetTypeHash() const override;
	virtual void CreateRenderThreadResources(FRHICommandListBase& RHICmdList) override;
	virtual void DestroyRenderThreadResources() override;
	virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override;
#if WITH_EDITOR
	virtual HHitProxy* CreateHitProxies(UPrimitiveComponent* Component, TArray<TRefCountPtr<HHitProxy> >& OutHitProxies) override;
#endif
	//~ End FPrimitiveSceneProxy

private:

	UDataflowComponent* DataflowComponent = nullptr;
	FManagedArrayCollection* ConstantData = nullptr;
	UMaterialInterface* RenderMaterial = nullptr;
#if WITH_EDITOR
	TArray<HHitProxy*> LocalHitProxies;
#endif
	UMaterialInterface* GetRenderMaterial() const;


	//
	// Surface Rendering
	//

	TArray<FDataflowTriangleSetMeshBatchData> MeshBatchDatas;
	FMaterialRelevance MaterialRelevance;
	FLocalVertexFactory VertexFactory;
	FStaticMeshVertexBuffers VertexBuffers;
	FDynamicMeshIndexBuffer32 IndexBuffer;

	/** Create rendering resources for surface selection data. */
	void CreateMeshRenderThreadResources(FRHICommandListBase& RHICmdList);
	/** Destroy rendering resources for surface selection data. */
	void DestroyMeshRenderThreadResources();
	/** Build MeshElements for rendering surface selection data. */
	void GetMeshDynamicMeshElements(int32 ViewIndex, FMeshElementCollector& Collector) const;


	//
	// Vertex Rendering
	//

	int32 NumRenderedVerts = 0;
	TArray<FDataflowVertexBatchData> VertexBatchDatas;
	FLocalVertexFactory BoxVertexFactory;
	FStaticMeshVertexBuffers BoxVertexBuffers;
	FDynamicMeshIndexBuffer32 BoxIndexBuffer;

	/** Create rendering resources for vertex selection data. */
	void CreateInstancedVertexRenderThreadResources(FRHICommandListBase& RHICmdList);
	/** Destroy rendering resources for vertex selection data. */
	void DestroyInstancedVertexRenderThreadResources();
	/** Build MeshElements for rendering vertex selection data. */
	void GetDynamicInstancedVertexMeshElements(int32 ViewIndex, FMeshElementCollector& Collector) const;

};



