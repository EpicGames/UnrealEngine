// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GameFramework/Actor.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "Dataflow/DataflowComponent.h"

#include "DataflowActor.generated.h"

#define UE_API DATAFLOWENGINEPLUGIN_API


UCLASS(MinimalAPI)
class ADataflowActor : public AActor
{
	GENERATED_UCLASS_BODY()

public:

	/* DataflowComponent */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = Destruction, meta = (ExposeFunctionCategories = "Components|Dataflow", AllowPrivateAccess = "true"))
	TObjectPtr<UDataflowComponent> DataflowComponent;
	UDataflowComponent* GetDataflowComponent() const { return DataflowComponent; }


#if WITH_EDITOR
	UE_API virtual bool GetReferencedContentObjects(TArray<UObject*>& Objects) const override;
#endif
};

#undef UE_API
