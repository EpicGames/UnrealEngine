// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "GeometryCollection/ManagedArrayCollection.h"
#include "Components/MeshComponent.h"
#include "Dataflow/DataflowComponentSelectionState.h"
#include "Dataflow/DataflowObject.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "DataflowComponent.generated.h"

#define UE_API DATAFLOWENGINEPLUGIN_API

namespace UE::Dataflow { class IDataflowConstructionViewMode; }

/**
*	UDataflowComponent
*/
UCLASS(MinimalAPI, meta = (BlueprintSpawnableComponent))
class UDataflowComponent : public UPrimitiveComponent
{
	GENERATED_UCLASS_BODY()

public:

	UE_API virtual ~UDataflowComponent();

	UE_API void Invalidate();
	UE_API void UpdateLocalBounds();


	//~ USceneComponent Interface.
	UE_API virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;

	//~ UPrimitiveComponent Interface.
	UE_API virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	UE_API virtual UMaterialInterface* GetMaterial(int32 Index) const override;

	/* Rendering Support*/
	UE_DEPRECATED(5.7, "Please use GetMaterialRelevance with EShaderPlatform argument and not ERHIFeatureLevel::Type")
	UE_API virtual FMaterialRelevance GetMaterialRelevance(ERHIFeatureLevel::Type InFeatureLevel) const;
	UE_API virtual FMaterialRelevance GetMaterialRelevance(EShaderPlatform InShaderPlatform) const;
	UE_API UMaterialInterface* GetDefaultMaterial() const;

	/** Render Targets*/
	UE_API void ResetRenderTargets(); 
	UE_API void AddRenderTarget(const UDataflowEdNode* InTarget); 
	const TArray<const UDataflowEdNode*>& GetRenderTargets() const {return RenderTargets;}

	/** Context */
	void SetContext(TSharedPtr<UE::Dataflow::FContext> InContext) { Context = InContext; }

	/** RenderCollection */
	UE_API void SetRenderingCollection(FManagedArrayCollection&& InCollection);
	UE_API const FManagedArrayCollection& GetRenderingCollection() const;
	      UE_API FManagedArrayCollection& ModifyRenderingCollection();

	/** Dataflow */
	void SetDataflow(const UDataflow* InDataflow) { Dataflow = InDataflow; }
	const UDataflow* GetDataflow() const { return Dataflow; }

	/* Selection */
	const FDataflowSelectionState& GetSelectionState() const { return SelectionState; }
	void SetSelectionState(const FDataflowSelectionState& InState) 
	{
		bUpdateSelection = true;
		SelectionState = InState; 
	}

	/* View mode */
	// NOTE: Currently UDataflowComponent is not used in the Dataflow Editor. Instead the FDataflowConstructionScene converts the FRenderingFacade to a UDynamicMeshComponent.
	// If we do start using UDataflowComponent we will need to update the current View Mode as it's changed using this function.
	void SetViewMode(const UE::Dataflow::IDataflowConstructionViewMode* InViewMode)
	{
		ViewMode = InViewMode;
	}

private:
	TSharedPtr<UE::Dataflow::FContext> Context;
	TArray<const UDataflowEdNode*> RenderTargets;
	TObjectPtr< const UDataflow> Dataflow;
	FManagedArrayCollection RenderCollection;

	bool bUpdateRender = true;
	bool bUpdateSelection = true;
	bool bBoundsNeedsUpdate = true;
	FBoxSphereBounds BoundingBox = FBoxSphereBounds(ForceInitToZero);
	FDataflowSelectionState SelectionState = FDataflowSelectionState(FDataflowSelectionState::EMode::DSS_Dataflow_None);
	const UE::Dataflow::IDataflowConstructionViewMode* ViewMode;
};

#undef UE_API
