// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#define UE_API DATAFLOWENGINEPLUGIN_API

class UDataflowComponent;


class FDataflowSelectionState
{
public:
	struct ObjectID
	{
		ObjectID(FString InName, int32 InID) : Name(InName), ID(InID) {}

		bool operator==(const ObjectID& A) const {
			return A.Name.Equals(Name);
		}

		FString Name;
		int32 ID;
	};

	enum EMode
	{
		DSS_Dataflow_None,
		DSS_Dataflow_Object,
		DSS_Dataflow_Face,
		DSS_Dataflow_Vertex,
		DSS_Dataflow_Max
	};

	FDataflowSelectionState(EMode InMode) : Mode(InMode) {}
	FDataflowSelectionState(const UDataflowComponent* DataflowComponent)
	{
		check(DataflowComponent);
	}

	UE_API void UpdateSelection(UDataflowComponent* DataflowComponent);

	bool IsEmpty() const
	{
		return Nodes.Num() == 0 
			&& Vertices.Num() == 0;
	}

	bool operator==(const FDataflowSelectionState& A) const {
		return A.Nodes == Nodes && A.Vertices == Vertices;
	}
	bool operator!=(const FDataflowSelectionState& A) const {
		return !this->operator==(A);
	}

	EMode Mode;
	TArray<ObjectID> Nodes;
	TArray<int32> Vertices;
};

#undef UE_API
