// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "DataflowConnectionTypes.generated.h"


USTRUCT(BlueprintType)
struct FCollectionAttributeKey
{
	GENERATED_USTRUCT_BODY()
public:
	FCollectionAttributeKey() : Attribute(""), Group("") {}
	FCollectionAttributeKey(FString InAttribute, FString InGroup)
		: Attribute(InAttribute), Group(InGroup) {}
	TPair<FName, FName> GetNamedKey() { return { FName(Attribute), FName(Group) }; }

	UPROPERTY(EditAnywhere, Category = "Collection Key")
	FString Attribute;

	UPROPERTY(EditAnywhere, Category = "Collection Key")
	FString Group;
};
