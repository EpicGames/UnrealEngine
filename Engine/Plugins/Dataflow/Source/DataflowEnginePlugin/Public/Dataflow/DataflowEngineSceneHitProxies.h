// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/UObjectGlobals.h"
#include "Components/PrimitiveComponent.h"

#include "EngineUtils.h"

class UDataflowComponent;
class UPrimitiveComponent;

/** HitProxy with for dataflow actor.
 */
struct HDataflowDefault : public HActor
{
	DECLARE_HIT_PROXY(DATAFLOWENGINEPLUGIN_API)

	HDataflowDefault(AActor* InActor, const UPrimitiveComponent* InPrimitiveComponent);

	virtual EMouseCursor::Type GetMouseCursor() override;
};


/** HitProxy with for dataflow actor.
 */
struct HDataflowNode : public HActor
{
	DECLARE_HIT_PROXY(DATAFLOWENGINEPLUGIN_API)

	int32 GeometryIndex = INDEX_NONE;
	FString NodeName = FString("");

	HDataflowNode(AActor* InActor, const UPrimitiveComponent* InPrimitiveComponent, FString InNodeName, int32 InGeometryIndex);

	virtual EMouseCursor::Type GetMouseCursor() override;
};


/** HitProxy with for dataflow actor.
 */
struct HDataflowVertex : public HActor
{
	DECLARE_HIT_PROXY(DATAFLOWENGINEPLUGIN_API)

	HDataflowVertex(AActor* InActor, const UPrimitiveComponent* InPrimitiveComponent, int32 InVertexIndex);

	virtual EMouseCursor::Type GetMouseCursor() override;
};