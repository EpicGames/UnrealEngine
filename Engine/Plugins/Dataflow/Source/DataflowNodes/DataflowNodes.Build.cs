// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class DataflowNodes : ModuleRules
	{
		public DataflowNodes(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"ChaosCore",
					"Chaos",
					"DataflowCore",
					"DataflowEngine",
					"DataflowEnginePlugin",
					"Engine",
					"MeshConversion",
					"MeshDescription",
					"SkeletalMeshDescription",
					"GeometryCore"
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"DynamicMesh",
					"GeometryFramework",
					"ImageCore",
					"InputCore",
					"ModelingComponents",
					"Slate",
					"SlateCore",
					"StaticMeshDescription",
				}
			);

			if (Target.Type == TargetType.Editor)
			{
				PrivateDependencyModuleNames.AddRange(
					new string[]
					{
						"SkeletalMeshUtilitiesCommon",
						"PropertyEditor",
					}
				);
			}
		}
	}
}
