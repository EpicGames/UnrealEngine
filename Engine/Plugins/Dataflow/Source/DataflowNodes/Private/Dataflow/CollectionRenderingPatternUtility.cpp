// Copyright Epic Games, Inc. All Rights Reserved. 

#include "Dataflow/CollectionRenderingPatternUtility.h"
#include "DynamicMesh/MeshNormals.h"
#include "DynamicMesh/NonManifoldMappingSupport.h"
#include "Engine/SkeletalMesh.h"
#include "GeometryCollection/Facades/CollectionRenderingFacade.h"
#include "SkeletalMeshAttributes.h"
#include "ToDynamicMesh.h"

using namespace UE::Geometry;

namespace UE::Dataflow
{
	namespace Conversion
	{
		// Convert a rendering facade to a dynamic mesh
		void RenderingFacadeToDynamicMesh(const GeometryCollection::Facades::FRenderingFacade& Facade, int32 InMeshIndex, FDynamicMesh3& DynamicMesh, const bool bBuildRemapping)
		{
			if (Facade.CanRenderSurface())
			{
				int32 StartTriangles = 0;
				int32 StartVertices = 0;
				int32 StartMaterials = 0;
				int32 NumTriangles = Facade.NumTriangles();
				int32 NumVertices = Facade.NumVertices();
				int32 NumMaterials = Facade.NumMaterials();

				if (InMeshIndex != INDEX_NONE)
				{
					if (ensure(0 <= InMeshIndex && InMeshIndex < Facade.NumGeometry()))
					{
						StartTriangles = Facade.GetIndicesStart()[InMeshIndex];
						StartVertices = Facade.GetVertexStart()[InMeshIndex];
						StartMaterials = Facade.GetMaterialStart()[InMeshIndex];
						NumTriangles = Facade.GetIndicesCount()[InMeshIndex];
						NumVertices = Facade.GetVertexCount()[InMeshIndex];
						NumMaterials = Facade.GetMaterialCount()[InMeshIndex];
					}
				}

				TArray<int32> Remapping;
				const TManagedArray<FIntVector>& Indices = Facade.GetIndices();
				const TManagedArray<FVector3f>& Positions = Facade.GetVertices();
				const TManagedArray<FVector3f>& Normals = Facade.GetNormals();
				const TManagedArray<FLinearColor>& Colors = Facade.GetVertexColor();
				const TManagedArray<TArray<FVector2f>>& UVs = Facade.GetVertexUV();
				const TManagedArray<int32>& FacadeMaterialID = Facade.GetMaterialID();

				const int32 MeshStartVtxID = DynamicMesh.VertexCount();

				int32 LastVertexIndex = StartVertices + NumVertices;
				for (int32 VertexIndex = StartVertices; VertexIndex < LastVertexIndex; ++VertexIndex)
				{
					DynamicMesh.AppendVertex(FVertexInfo(FVector3d(Positions[VertexIndex]), Normals[VertexIndex],
						FVector3f(Colors[VertexIndex].R, Colors[VertexIndex].G, Colors[VertexIndex].B)));
					Remapping.Add(VertexIndex);
				}
				int32 LastTriangleIndex = StartTriangles + NumTriangles;
				for (int32 TriangleIndex = StartTriangles; TriangleIndex < LastTriangleIndex; ++TriangleIndex)
				{
					DynamicMesh.AppendTriangle(FIndex3i(
						Indices[TriangleIndex].X - StartVertices,
						Indices[TriangleIndex].Y - StartVertices,
						Indices[TriangleIndex].Z - StartVertices)
					);
				}
				DynamicMesh.EnableAttributes();

				// Build Remmaping indices back into the colleciton. 
				if (bBuildRemapping && (Remapping.Num() < Facade.NumVertices()))
				{
					UE::Geometry::FNonManifoldMappingSupport::AttachNonManifoldVertexMappingData(Remapping, DynamicMesh);
				}

				FDynamicMeshNormalOverlay* const NormalOverlay = DynamicMesh.Attributes()->PrimaryNormals();
				NormalOverlay->CreatePerVertex(0.f);
				auto SetOverlayNormalsFromVertexNormals = [&](int TriangleID)
				{
					const FIndex3i Tri = DynamicMesh.GetTriangle(TriangleID);
					const FIndex3i NormalElementTri = NormalOverlay->GetTriangle(TriangleID);
					for (int TriVertIndex = 0; TriVertIndex < 3; ++TriVertIndex)
					{
						FVector3f Normal(Normals[Remapping[Tri[TriVertIndex]]]);
						NormalOverlay->SetElement(NormalElementTri[TriVertIndex], Normal);
					}
				};
				for (const int TriangleID : DynamicMesh.TriangleIndicesItr())
				{
					SetOverlayNormalsFromVertexNormals(TriangleID);
				}

				DynamicMesh.Attributes()->EnablePrimaryColors();
				DynamicMesh.Attributes()->PrimaryColors()->CreatePerVertex(0.f);
				DynamicMesh.EnableVertexColors(FVector3f::Zero());
				FDynamicMeshColorOverlay* const ColorOverlay = DynamicMesh.Attributes()->PrimaryColors();
				auto SetColorsFromWeights = [&](int TriangleID)
				{
					const FIndex3i Tri = DynamicMesh.GetTriangle(TriangleID);
					const FIndex3i ColorElementTri = ColorOverlay->GetTriangle(TriangleID);
					for (int TriVertIndex = 0; TriVertIndex < 3; ++TriVertIndex)
					{
						FVector4f Color(Colors[Remapping[Tri[TriVertIndex]]]); Color.W = 1.0f;
						ColorOverlay->SetElement(ColorElementTri[TriVertIndex], Color);
					}
				};
				for (const int TriangleID : DynamicMesh.TriangleIndicesItr())
				{
					SetColorsFromWeights(TriangleID);
				}

				DynamicMesh.Attributes()->EnableMaterialID();
				FDynamicMeshMaterialAttribute* const MaterialIDAttrib = DynamicMesh.Attributes()->GetMaterialID();
				for (const int TriangleID : DynamicMesh.TriangleIndicesItr())
				{
					MaterialIDAttrib->SetValue(TriangleID, FacadeMaterialID[TriangleID + StartTriangles] - StartMaterials);
				}

				const int32 NumUVLayers = Remapping.Num() > 0 ? UVs[Remapping[0]].Num() : 0;

				if (NumUVLayers > 0)
				{
					for (int32 VertexIndex = StartVertices; VertexIndex < LastVertexIndex; ++VertexIndex)
					{
						const TArray<FVector2f>& VertexUVs = UVs[VertexIndex];
						check(NumUVLayers == VertexUVs.Num());
					}

					DynamicMesh.Attributes()->SetNumUVLayers(NumUVLayers);

					for (int32 UVLayerIndex = 0; UVLayerIndex < NumUVLayers; ++UVLayerIndex)
					{
						FDynamicMeshUVOverlay* const UVLayer = DynamicMesh.Attributes()->GetUVLayer(UVLayerIndex);
						UVLayer->CreatePerVertex(0.f);

						for (const int TriangleID : DynamicMesh.TriangleIndicesItr())
						{
							const FIndex3i Tri = DynamicMesh.GetTriangle(TriangleID);
							const FIndex3i UVElementTri = UVLayer->GetTriangle(TriangleID);

							for (int TriVertIndex = 0; TriVertIndex < 3; ++TriVertIndex)
							{
								const FVector2f UV(UVs[Remapping[Tri[TriVertIndex]]][UVLayerIndex]);
								UVLayer->SetElement(UVElementTri[TriVertIndex], UV);
							}
						}
					}
				}

				const TManagedArray<TArray<float>>* BoneWeights = Facade.FindBoneWeights();
				const TManagedArray<TArray<int32>>* BoneIndices = Facade.FindBoneIndices();
				if (BoneWeights && BoneIndices)
				{
					const FName DefaultProfileName("Default");
					FDynamicMeshVertexSkinWeightsAttribute* Attribute = DynamicMesh.Attributes()->GetSkinWeightsAttribute(DefaultProfileName);
					if (Attribute == nullptr)
					{
						Attribute = new FDynamicMeshVertexSkinWeightsAttribute(&DynamicMesh);
						DynamicMesh.Attributes()->AttachSkinWeightsAttribute(DefaultProfileName, Attribute);
					}
					for (int32 VertexIndex = StartVertices; VertexIndex < LastVertexIndex; ++VertexIndex)
					{
						const TArray<float>& VtxBoneWeights = (*BoneWeights)[VertexIndex];
						const TArray<int32>& VtxBoneIndices = (*BoneIndices)[VertexIndex];

						UE::AnimationCore::FBoneWeights Weights;
						const int32 NumWeights = FMath::Min(VtxBoneWeights.Num(), VtxBoneIndices.Num());
						for (int32 WeightIndex = 0; WeightIndex < NumWeights; ++WeightIndex)
						{
							Weights.SetBoneWeight(VtxBoneIndices[WeightIndex], VtxBoneWeights[WeightIndex]);
						}
						const int32 VtxID = MeshStartVtxID + (VertexIndex - StartVertices);
						Attribute->SetValue(VtxID, Weights);
					}
				}
			}
		}

		// Convert a dynamic mesh to a rendering facade
		void DynamicMeshToRenderingFacade(const FDynamicMesh3& DynamicMesh, GeometryCollection::Facades::FRenderingFacade& Facade)
		{
			if (Facade.CanRenderSurface())
			{
				const int32 NumTriangles = Facade.NumTriangles();
				const int32 NumVertices = Facade.NumVertices();

				// We can only override vertices attributes (position, normals, colors)
				if ((NumTriangles == DynamicMesh.TriangleCount()) && (NumVertices == DynamicMesh.VertexCount()))
				{
					TManagedArray<FVector3f>& Positions = Facade.ModifyVertices();
					TManagedArray<FVector3f>& Normals = Facade.ModifyNormals();
					TManagedArray<FLinearColor>& Colors = Facade.ModifyVertexColor();

					for (int32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
					{
						Positions[VertexIndex] = FVector3f(DynamicMesh.GetVertex(VertexIndex));
						Normals[VertexIndex] = DynamicMesh.GetVertexNormal(VertexIndex);
						Colors[VertexIndex] = DynamicMesh.GetVertexColor(VertexIndex);
					}
				}
			}
		}
	}

}	// namespace UE::Dataflow