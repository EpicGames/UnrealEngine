// Copyright Epic Games, Inc. All Rights Reserved.#include "DataflowToolNodeSnapshotCustomization.h"

#include "Dataflow/Customization/DataflowToolNodeSnapshotCustomization.h"

#include "Dataflow/DataflowToolNode.h"

#if WITH_EDITOR

#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Images/SImage.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "DataflowToolNodeSnapshotCustomization"

namespace FDataflowToolNodeSnapshotProperties::Private
{
	static const FName NamePropertyName = "Name";
	static const FName DescPropertyName = "Description";
	static const FName DatePropertyName = "Date";	
	static const FName LockedPropertyName = "bLocked";
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<IPropertyTypeCustomization> FDataflowToolNodeSnapshotCustomization::MakeInstance()
{
	return MakeShareable(new FDataflowToolNodeSnapshotCustomization());
}

// CustomizeHeader just delegates to BuildWidget
void FDataflowToolNodeSnapshotCustomization::CustomizeHeader(
	TSharedRef<IPropertyHandle> PropertyHandle,
	FDetailWidgetRow& HeaderRow,
	IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	HeaderRow
		.WholeRowContent()
		[
			BuildWidget(PropertyHandle)
		];
}

TSharedRef<SWidget> FDataflowToolNodeSnapshotCustomization::BuildWidget(
	TSharedRef<IPropertyHandle> SnapshotHandle)
{
	return SNew(SHorizontalBox)
		.ToolTipText_Lambda([SnapshotHandle]()
			{
				FString Desc;
				if (TSharedPtr<IPropertyHandle> DescHandle = SnapshotHandle->GetChildHandle(FDataflowToolNodeSnapshotProperties::Private::DescPropertyName))
				{
					DescHandle->GetValue(Desc);
				}
				return FText::FromString(Desc);
			})

		// Lock toggle
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(2.f, 0.f, 6.f, 0.f)
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "NoBorder")
			.ContentPadding(2.f)
			.ToolTipText_Lambda([SnapshotHandle]()
				{
					bool bLocked = false;
					if (TSharedPtr<IPropertyHandle> LockedHandle = SnapshotHandle->GetChildHandle(FDataflowToolNodeSnapshotProperties::Private::LockedPropertyName))
					{
						LockedHandle->GetValue(bLocked);
					}
					return bLocked
						? LOCTEXT("UnlockAction", "Unlock — may be discarded during cleanup")
						: LOCTEXT("LockAction", "Lock — preserved during cleanup");
				})
			.OnClicked_Lambda([SnapshotHandle]() -> FReply
				{
					if (TSharedPtr<IPropertyHandle> LockedHandle = SnapshotHandle->GetChildHandle(FDataflowToolNodeSnapshotProperties::Private::LockedPropertyName))
					{
						bool bLocked = false;
						LockedHandle->GetValue(bLocked);
						LockedHandle->SetValue(!bLocked);
					}
					return FReply::Handled();
				})
			[
				SNew(SImage)
				.DesiredSizeOverride(FVector2D(16.f, 16.f))
				.Image_Lambda([SnapshotHandle]() -> const FSlateBrush*
					{
						bool bLocked = false;
						if (TSharedPtr<IPropertyHandle> LockedHandle = SnapshotHandle->GetChildHandle(FDataflowToolNodeSnapshotProperties::Private::LockedPropertyName))
						{
							LockedHandle->GetValue(bLocked);
						}
						return bLocked
							? FAppStyle::GetBrush("PropertyWindow.Locked")
							: FAppStyle::GetBrush("PropertyWindow.Unlocked");
					})
			]
		]

	// Editable name 
	+ SHorizontalBox::Slot()
	.FillWidth(0.50f)
	.VAlign(VAlign_Center)
	.Padding(0.f, 0.f, 8.f, 0.f)
	[
		SNew(SEditableTextBox)
		.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
		.Text_Lambda([SnapshotHandle]() -> FText
			{
				FString Name;
				if (TSharedPtr<IPropertyHandle> NameHandle = SnapshotHandle->GetChildHandle(FDataflowToolNodeSnapshotProperties::Private::NamePropertyName))
				{
					NameHandle->GetValue(Name);
				}
				return FText::FromString(Name);
			})
		.OnTextCommitted_Lambda([SnapshotHandle](const FText& NewText, ETextCommit::Type)
			{
				if (TSharedPtr<IPropertyHandle> NameHandle = SnapshotHandle->GetChildHandle(FDataflowToolNodeSnapshotProperties::Private::NamePropertyName))
				{
					NameHandle->SetValue(NewText.ToString());
				}
			})
	]

	// ead-only description
	+ SHorizontalBox::Slot()
		.FillWidth(0.50f)
		.VAlign(VAlign_Center)
		.Padding(0.f, 0.f, 8.f, 0.f)
		[
			SNew(STextBlock)
				.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
				.Text_Lambda([SnapshotHandle]() -> FText
					{
						FString Desc;
						if (TSharedPtr<IPropertyHandle> DescHandle = SnapshotHandle->GetChildHandle(FDataflowToolNodeSnapshotProperties::Private::DescPropertyName))
						{
							DescHandle->GetValue(Desc);
						}
						return FText::FromString(Desc);
					})
		]

	// Date (read-only)
	+ SHorizontalBox::Slot()
	.AutoWidth()
	.VAlign(VAlign_Center)
	.Padding(0.f, 0.f, 4.f, 0.f)
	[
		SNew(STextBlock)
		.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
		.ColorAndOpacity(FSlateColor::UseSubduedForeground())
		.Text_Lambda([SnapshotHandle]() -> FText
			{
				if (TSharedPtr<IPropertyHandle> DateHandle = SnapshotHandle->GetChildHandle(FDataflowToolNodeSnapshotProperties::Private::DatePropertyName))
				{
					TArray<void*> RawData;
					DateHandle->AccessRawData(RawData);
					if (!RawData.IsEmpty() && RawData[0])
					{
						return FText::FromString(static_cast<FDateTime*>(RawData[0])->ToFormattedString(TEXT("%a, %b %e, %Y at %H:%M")));
					}
				}
				return FText::GetEmpty();
			})
	];
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////


TSharedRef<IPropertyTypeCustomization> FDataflowToolNodeSnapshotSetCustomization::MakeInstance()
{
	return MakeShareable(new FDataflowToolNodeSnapshotSetCustomization());
}

void FDataflowToolNodeSnapshotSetCustomization::CustomizeHeader(
	TSharedRef<IPropertyHandle> PropertyHandle,
	FDetailWidgetRow& HeaderRow,
	IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	// PropertyHandle is the owning struct — find the snapshots array inside it
	SnapshotSetHandle = PropertyHandle;
	ArrayHandle = PropertyHandle->GetChildHandle(FName("Snapshots"));
	ActiveIndexHandle = PropertyHandle->GetChildHandle(FName("ActiveSnapshot"));

	if (ArrayHandle.IsValid())
	{
		// Rebuild list whenever the array changes
		ArrayHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateLambda([this]()
			{
				RefreshView();
			}));

		ArrayHandle->AsArray()->SetOnNumElementsChanged(
			FSimpleDelegate::CreateLambda([this]()
				{
					RefreshView();
				}));
	}

	RefreshListItems();

	HeaderRow
		.NameContent()
		[
			SNew(STextBlock)
				.Text(PropertyHandle->GetPropertyDisplayName())
				.Font(IDetailLayoutBuilder::GetDetailFontBold())
		]
		.ValueContent()
		[
			SNew(STextBlock)
				.Text(this, &FDataflowToolNodeSnapshotSetCustomization::GetHeaderText)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.ColorAndOpacity(FSlateColor::UseSubduedForeground())
		];
}

void FDataflowToolNodeSnapshotSetCustomization::CustomizeChildren(
	TSharedRef<IPropertyHandle> PropertyHandle,
	IDetailChildrenBuilder& ChildBuilder,
	IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	if (ArrayHandle.IsValid())
	{
		ChildBuilder.AddCustomRow(FText::GetEmpty())
		.WholeRowContent()
		[
			SNew(SBox)
			.MaxDesiredHeight(300.f)
			[
				SAssignNew(ListView, SListView<FSnapshotItemPtr>)
				.ListItemsSource(&ListItems)
				.SelectionMode(ESelectionMode::Single)
				.OnGenerateRow(this, &FDataflowToolNodeSnapshotSetCustomization::OnGenerateRow)
				.OnSelectionChanged(this, &FDataflowToolNodeSnapshotSetCustomization::OnSelectionChanged)
				.OnContextMenuOpening(this, &FDataflowToolNodeSnapshotSetCustomization::OnContextMenuOpening)
				.HeaderRow
				(
					SNew(SHeaderRow)
					+ SHeaderRow::Column("Snapshot")
					.DefaultLabel(LOCTEXT("SnapshotHeader", "Snapshot"))
					.FillWidth(1.f)
				)
			]
		];
	}
}

int32 FDataflowToolNodeSnapshotSetCustomization::GetSnapshotIndex(const FSnapshotItemPtr& Item) const
{
	const int32 SelectedItemIndex = ListItems.Find(Item);
	if (SelectedItemIndex != INDEX_NONE)
	{
		return (ListItems.Num() - 1 - SelectedItemIndex); // list view is in inverse order
	}
	return INDEX_NONE;
}

bool FDataflowToolNodeSnapshotSetCustomization::IsActiveSnapshot(const FSnapshotItemPtr& Item) const
{
	const int32 SnapshotIndex = GetSnapshotIndex(Item);

	int32 ActiveSnapshotIndex = INDEX_NONE;
	if (ActiveIndexHandle)
	{
		ActiveIndexHandle->GetValue(ActiveSnapshotIndex);
	}
	return (ActiveSnapshotIndex == SnapshotIndex);
}

TSharedPtr<SWidget> FDataflowToolNodeSnapshotSetCustomization::OnContextMenuOpening()
{
	TArray<FSnapshotItemPtr> SelectedItems = ListView->GetSelectedItems();
	if (SelectedItems.Num() == 0)
	{
		return nullptr;
	}

	FSnapshotItemPtr Item = SelectedItems[0];
	const bool bIsActiveSnapshot = IsActiveSnapshot(Item);

	FMenuBuilder MenuBuilder(true, nullptr);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("ActivateSnapshot", "Activate snapshot"),
		LOCTEXT("ActivateSnapshotTooltip", "Make this snaphot the active one"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([this, Item]()
				{
					const int32 SnapshotIndex = GetSnapshotIndex(Item);
					if (SnapshotIndex != INDEX_NONE)
					{
						void* StructData = nullptr;
						if (SnapshotSetHandle->GetValueData(StructData))
						{
							if (FDataflowToolNodeSnapshotSet* SnapshotSet = reinterpret_cast<FDataflowToolNodeSnapshotSet*>(StructData))
							{
								SnapshotSet->SetActiveSnapshot(SnapshotIndex, /*bNotify*/true);
								RefreshView();
							}
						}
					}
				}),
			FCanExecuteAction::CreateLambda([bIsActiveSnapshot]()
				{
					return !bIsActiveSnapshot;
				}))
	);

	MenuBuilder.AddMenuSeparator();

	MenuBuilder.AddMenuEntry(
		LOCTEXT("DeleteSnapshot", "Delete snapshot"),
		LOCTEXT("DeleteSnapshotTooltip", "Delete this snapshot"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([this, Item]()
				{
					const int32 SnapshotIndex = GetSnapshotIndex(Item);
					if (SnapshotIndex != INDEX_NONE)
					{
						void* StructData = nullptr;
						if (SnapshotSetHandle->GetValueData(StructData))
						{
							if (FDataflowToolNodeSnapshotSet* SnapshotSet = reinterpret_cast<FDataflowToolNodeSnapshotSet*>(StructData))
							{
								SnapshotSet->RemoveSnapshot(SnapshotIndex, /*bNotify*/true);
								RefreshView();
							}
						}
					}
				}))
	);

	return MenuBuilder.MakeWidget();
}

void FDataflowToolNodeSnapshotSetCustomization::RefreshView()
{
	RefreshListItems();
	if (ListView.IsValid())
	{
		ListView->RequestListRefresh();
	}
}

void FDataflowToolNodeSnapshotSetCustomization::RefreshListItems()
{
	ListItems.Reset();
	if (ArrayHandle.IsValid())
	{
		uint32 NumElements = 0;
		ArrayHandle->AsArray()->GetNumElements(NumElements);

		// Add in reverse order as we want the more recent on top
		if (NumElements > 0)
		{
			for (int32 Index = (NumElements - 1); Index >= 0; --Index)
			{
				ListItems.Add(ArrayHandle->AsArray()->GetElement(Index));
			}
		}
	}
}

TSharedRef<ITableRow> FDataflowToolNodeSnapshotSetCustomization::OnGenerateRow(
	FSnapshotItemPtr Item,
	const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(STableRow<FSnapshotItemPtr>, OwnerTable)
	[
		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.MinWidth(16.f)
		.Padding(0.f, 0.f, 4.f, 0.f)
		[
			SNew(SImage)
			.DesiredSizeOverride(FVector2D(16.f, 16.f))
			.Image_Lambda([this, Item]() -> const FSlateBrush*
				{
					const bool bIsActiveSnapshot = IsActiveSnapshot(Item);
					return bIsActiveSnapshot ? FAppStyle::GetBrush("Symbols.Check") : nullptr;
				})
			.ColorAndOpacity(FColor::Green)
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.FillWidth(1.0)
		.Padding(0.f, 0.f, 4.f, 0.f)
		[
			FDataflowToolNodeSnapshotCustomization::BuildWidget(Item.ToSharedRef())
		]
	];
}

void FDataflowToolNodeSnapshotSetCustomization::OnSelectionChanged(
	FSnapshotItemPtr Item,
	ESelectInfo::Type SelectInfo)
{
	SelectedItem = Item;
}

FText FDataflowToolNodeSnapshotSetCustomization::GetHeaderText() const
{
	return FText::Format(
		LOCTEXT("SnapshotCount", "{0} snapshot(s)"),
		FText::AsNumber(ListItems.Num()));
}

#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE