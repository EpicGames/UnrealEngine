// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "IPropertyTypeCustomization.h"
#include "PropertyHandle.h"
#include "Widgets/Views/SListView.h"

class FDataflowToolNodeSnapshotCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	// Reusable — call this anywhere you have a handle to a snapshot element
	static TSharedRef<SWidget> BuildWidget(TSharedRef<IPropertyHandle> SnapshotHandle);

	virtual void CustomizeHeader(
		TSharedRef<IPropertyHandle> PropertyHandle,
		FDetailWidgetRow& HeaderRow,
		IPropertyTypeCustomizationUtils& CustomizationUtils) override;

	virtual void CustomizeChildren(
		TSharedRef<IPropertyHandle> PropertyHandle,
		IDetailChildrenBuilder& ChildBuilder,
		IPropertyTypeCustomizationUtils& CustomizationUtils) override
	{
	}
};

class FDataflowToolNodeSnapshotSetCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	virtual void CustomizeHeader(
		TSharedRef<IPropertyHandle> PropertyHandle,
		FDetailWidgetRow& HeaderRow,
		IPropertyTypeCustomizationUtils& CustomizationUtils) override;

	virtual void CustomizeChildren(
		TSharedRef<IPropertyHandle> PropertyHandle,
		IDetailChildrenBuilder& ChildBuilder,
		IPropertyTypeCustomizationUtils& CustomizationUtils) override;

private:
	using FSnapshotItemPtr = TSharedPtr<IPropertyHandle>;

	TSharedPtr<IPropertyHandle>				SnapshotSetHandle;
	TSharedPtr<IPropertyHandle>				ArrayHandle;
	TSharedPtr<IPropertyHandle>				ActiveIndexHandle;
	TArray<FSnapshotItemPtr>				ListItems;
	TSharedPtr<SListView<FSnapshotItemPtr>> ListView;
	FSnapshotItemPtr						SelectedItem;

	TSharedPtr<SWidget> OnContextMenuOpening();
	void  RefreshView();
	void RefreshListItems();
	int32 GetSnapshotIndex(const FSnapshotItemPtr& Item) const;
	bool IsActiveSnapshot(const FSnapshotItemPtr& Item) const;

	TSharedRef<ITableRow> OnGenerateRow(
		FSnapshotItemPtr Item,
		const TSharedRef<STableViewBase>& OwnerTable);

	void OnSelectionChanged(
		FSnapshotItemPtr Item,
		ESelectInfo::Type SelectInfo);

	FText GetHeaderText() const;
};

#endif // WITH_EDITOR