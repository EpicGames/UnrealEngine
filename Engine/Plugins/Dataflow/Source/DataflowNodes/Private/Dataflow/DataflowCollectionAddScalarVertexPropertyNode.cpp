// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowCollectionAddScalarVertexPropertyNode.h"

#include "Dataflow/DataflowInputOutput.h"
#include "Dataflow/DataflowTools.h"
#include "InteractiveToolChange.h"
#include "Misc/LazySingleton.h"

#include "Dataflow/DataflowCollectionAttributeUtils.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowCollectionAddScalarVertexPropertyNode)

#define LOCTEXT_NAMESPACE "DataflowCollectionAddScalarVertexProperty"

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
//
// FDataflowCollectionAddScalarVertexPropertyNode
//

FDataflowCollectionAddScalarVertexPropertyNode::FDataflowCollectionAddScalarVertexPropertyNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowVertexAttributeEditableNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&AttributeKey);
	RegisterOutputConnection(&Collection, &Collection);
	RegisterOutputConnection(&AttributeKey, &AttributeKey);
}

TArray<UE::Dataflow::FRenderingParameter> FDataflowCollectionAddScalarVertexPropertyNode::GetRenderParametersImpl() const
{
	return FDataflowAddScalarVertexPropertyCallbackRegistry::Get().GetRenderingParameters(TargetGroup.Name);
}

void FDataflowCollectionAddScalarVertexPropertyNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace UE::Dataflow;

	// Get the pin value if plugged
	FCollectionAttributeKey Key = GetWeightAttributeKey(Context);

	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		// Evaluate in collection
		FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);

		if (!InCollection.HasGroup(FName(*Key.Group)))
		{
			const FText ErrorMessage = FText::Format(LOCTEXT("MissingVertexGroup", "Vertex Group [%s] could not be found in the collection, check the collection or select a different vertex group"), FText::FromString(Key.Group));
			Context.Error(ErrorMessage, this, Out);
		}
		else
		{
			if (!Key.Attribute.IsEmpty())
			{
				const FName InName(Key.Attribute);
				const FName InGroup(Key.Group);
				TManagedArray<float>& ScalarWeights = InCollection.AddAttribute<float>(InName, InGroup);

				TArray<float> StoredWeights;
				StoredWeights.SetNumZeroed(ScalarWeights.Num());
				GetWeightsFromActiveSnapshot(InCollection, InGroup, StoredWeights);

				if (StoredWeights.Num() > 0 && StoredWeights.Num() != ScalarWeights.Num())
				{
					FDataflowTools::LogAndToastWarning(*this,
						LOCTEXT("VertexCountMismatchHeadline", "Vertex count mismatch."),
						FText::Format(LOCTEXT("VertexCountMismatchDetails", "Vertex weights in the node: {0}\n Vertices in group \"{1}\" in the Collection: {2}"),
							StoredWeights.Num(),
							FText::FromName(InGroup),
							ScalarWeights.Num()));
				}

				TConstArrayView<float> InputWeights = MakeArrayView(ScalarWeights.GetConstArray());
				ComputeFinalWeights(InputWeights, StoredWeights, MakeArrayView(ScalarWeights.GetData(), ScalarWeights.Num()));
			}
			else
			{
				const FText ErrorMessage = FText(LOCTEXT("EmptyAttributeName", "Attribute name is empty, collection will remain unchanged"));
				Context.Error(ErrorMessage, this, Out);
			}
		}

		SetValue(Context, MoveTemp(InCollection), &Collection);
	}
	else if (Out->IsA<FCollectionAttributeKey>(&AttributeKey))
	{
		SetValue(Context, MoveTemp(Key), &AttributeKey);
	}
}

// DEPRECATED 5.8
bool FDataflowCollectionAddScalarVertexPropertyNode::FillAttributeWeights( 
	const TSharedPtr<const FManagedArrayCollection> SelectedCollection, const FCollectionAttributeKey& InAttributeKey, TArray<float>& OutAttributeValues) const
{
	if (!InAttributeKey.Attribute.IsEmpty())
	{
		const FName InName(InAttributeKey.Attribute);
		const FName InGroup(InAttributeKey.Group);
		
		if(const TManagedArray<float>* AttributeArray = SelectedCollection->FindAttributeTyped<float>(InName, InGroup))
		{
			OutAttributeValues = AttributeArray->GetConstArray();
			return true;
		}
	}
	OutAttributeValues.Init(0.0f, OutAttributeValues.Num());
	return false;
}

FCollectionAttributeKey FDataflowCollectionAddScalarVertexPropertyNode::GetWeightAttributeKey(UE::Dataflow::FContext& Context) const
{
	// Get the pin value if plugged
	FCollectionAttributeKey Key = GetValue(Context, &AttributeKey, AttributeKey);

	// If nothing set used the local value
	if(Key.Attribute.IsEmpty() && Key.Group.IsEmpty())
	{
		Key.Group = TargetGroup.Name.ToString();
		Key.Attribute = Name;
	}
	return Key;
}

void FDataflowCollectionAddScalarVertexPropertyNode::GetSupportedViewModes(UE::Dataflow::FContext& Context, TArray<FName>& OutViewModeNames) const
{
	for (UE::Dataflow::FRenderingParameter& RenderParameter : FDataflowCollectionAddScalarVertexPropertyNode::GetRenderParametersImpl())
	{
		OutViewModeNames.AddUnique(RenderParameter.ViewMode);
	}
	// matches UE::Dataflow::FDataflowConstruction3DViewMode::Name - but we cannot include the module to get it directly 
	//static const FName DataflowConstruction3DViewModeName = TEXT("3DView");
	//static const FName Cloth3DSimView = TEXT("Cloth3DSimView");
	//static const FName ClothRenderView = TEXT("ClothRenderView");
	//OutViewModeNames.Add(DataflowConstruction3DViewModeName);
	//OutViewModeNames.Add(Cloth3DSimView);
	//OutViewModeNames.Add(ClothRenderView);
}

void FDataflowCollectionAddScalarVertexPropertyNode::GetVertexAttributeValues(UE::Dataflow::FContext& Context, TArray<float>& OutValues) const
{
	const FCollectionAttributeKey InKey = GetWeightAttributeKey(Context);
	const FName InAttribName(InKey.Attribute);
	const FName InGroupName(InKey.Group);
	const FManagedArrayCollection& InCollection = GetValue(Context, &Collection, Collection);

	UE::Dataflow::FDataflowCollectionFacade Facade(InCollection, InGroupName);
	const int32 NumVertices = Facade.GetVertexPositions().Num();
	OutValues.SetNumZeroed(NumVertices);

	TArray<float> StoredWeights;
	StoredWeights.SetNumZeroed(NumVertices);
	GetWeightsFromActiveSnapshot(InCollection, InGroupName, StoredWeights);

	if (const TManagedArray<float>* WeightsAttribute = InCollection.FindAttributeTyped<float>(InAttribName, InGroupName))
	{
		ComputeFinalWeights(WeightsAttribute->GetConstArray(), StoredWeights, OutValues);
	}
	else
	{
		TArray<float> ZeroSetupWeights;
		ZeroSetupWeights.Init(0.0f, NumVertices);
		ComputeFinalWeights(ZeroSetupWeights, StoredWeights, OutValues);
	}
	
}

void FDataflowCollectionAddScalarVertexPropertyNode::SetVertexAttributeValues(UE::Dataflow::FContext& Context, const TArray<float>& InValues, const TArray<int32>& InWeightIndices)
{
	const FCollectionAttributeKey InKey = GetWeightAttributeKey(Context);
	const FName InAttribName(InKey.Attribute);
	const FName InGroupName(InKey.Group);
	const FManagedArrayCollection& InCollection = GetValue(Context, &Collection, Collection);

	UE::Dataflow::FDataflowCollectionFacade Facade(InCollection, InGroupName);
	const int32 NumVertices = Facade.GetVertexPositions().Num();

	TArray<float> WeightsToStore;
	WeightsToStore.SetNumZeroed(NumVertices);
	// Seed with the weights already stored on the active snapshot so vertices outside the current paint set (such as other components painted into the same collection) are preserved instead of being zeroed.
	GetWeightsFromActiveSnapshot(InCollection, InGroupName, WeightsToStore);

	const TManagedArray<float>* AttributeArray = InCollection.FindAttributeTyped<float>(InAttribName, InGroupName);
	if (AttributeArray && AttributeArray->Num() == NumVertices)
	{
		ComputeWeightsToStore(AttributeArray->GetConstArray(), InValues, InWeightIndices, WeightsToStore);
	}
	else
	{
		TArray<float> ZeroSetupWeights;
		ZeroSetupWeights.Init(0.0f, NumVertices);
		ComputeWeightsToStore(ZeroSetupWeights, InValues, InWeightIndices, WeightsToStore);
	}

	FDataflowToolNodeSnapshot& NewSnapShot = AddSnapshot();
	NewSnapShot.Description = FString::Printf(TEXT("%s - %d vertices"), *InKey.Group, NumVertices);
	StoreWeightsInSnapshot(InCollection, WeightsToStore, InGroupName, NewSnapShot);

	Invalidate();
}


void FDataflowCollectionAddScalarVertexPropertyNode::GetExtraVertexMapping(UE::Dataflow::FContext& Context, FName SelectedViewMode, TArray<int32>& OutMappingToWeight, TArray<TArray<int32>>& OutMappingFromWeight) const
{
	OutMappingToWeight.Reset();
	OutMappingFromWeight.Reset();

	const FCollectionAttributeKey InKey = GetWeightAttributeKey(Context);
	const FName InAttribName(InKey.Attribute);
	const FName InGroupName(InKey.Group);
	FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);

	UE::Dataflow::FDataflowCollectionFacade Facade(InCollection, InGroupName);
	TConstArrayView<int32> Mapping2DTo3D = Facade.Get2Dto3DMapping();
	if (Mapping2DTo3D.IsEmpty())
	{
		return;
	}

	OutMappingToWeight = Mapping2DTo3D;

	// recompute the reverse mapping 
	OutMappingFromWeight.SetNum(Facade.GetVertexPositions().Num());
	for (int32 Index2D = 0; Index2D < Mapping2DTo3D.Num(); ++Index2D)
	{
		const int32 Index3D = Mapping2DTo3D[Index2D];
		if (OutMappingFromWeight.IsValidIndex(Index3D))
		{
			OutMappingFromWeight[Index3D].Add(Index2D);
		}
	}
}

// DEPRECATED 5.8
void FDataflowCollectionAddScalarVertexPropertyNode::ReportVertexWeights(const TArray<float>& SetupWeights, const TArray<float>& FinalWeights, const TArray<int32>& WeightIndices)
{
	VertexWeights.Init(0.0f, SetupWeights.Num());
	ComputeWeightsToStore(SetupWeights, FinalWeights, WeightIndices, VertexWeights);
}

void FDataflowCollectionAddScalarVertexPropertyNode::ComputeWeightsToStore(TConstArrayView<float> InputWeights, TConstArrayView<float> FinalWeights, TConstArrayView<int32> WeightIndices, TArrayView<float> WeightsToStore)
{
	check((WeightIndices.Num() == FinalWeights.Num()) || (WeightIndices.IsEmpty() && (InputWeights.Num() == FinalWeights.Num())));
	check(WeightsToStore.Num() == InputWeights.Num());

	// The caller seeds WeightsToStore with the previously stored values, so vertices outside WeightIndices are left untouched here.
	for (int32 WeightIndex = 0, NumWeights = FinalWeights.Num(); WeightIndex <NumWeights; ++WeightIndex)
	{
		const int32 VertexIndex = WeightIndices.IsEmpty() ? WeightIndex : WeightIndices[WeightIndex];
		if (WeightsToStore.IsValidIndex(VertexIndex))
		{
			switch (OverrideType)
			{
			case EDataflowWeightMapOverrideType::ReplaceAll:
				WeightsToStore[VertexIndex] = FinalWeights[WeightIndex];
				break;
			case EDataflowWeightMapOverrideType::ReplaceChanged:
				WeightsToStore[VertexIndex] = (InputWeights[VertexIndex] == FinalWeights[WeightIndex]) ? ReplaceChangedPassthroughValue : FinalWeights[WeightIndex];
				break;
			case EDataflowWeightMapOverrideType::AddDifference:
				WeightsToStore[VertexIndex] = FinalWeights[WeightIndex] - InputWeights[VertexIndex];
				break;
			default: unimplemented();
			}
		}
	}
}

// DEPRECATED 5.8
void FDataflowCollectionAddScalarVertexPropertyNode::ExtractVertexWeights(const TArray<float>& SetupWeights, TArrayView<float> FinalWeights) const
{
	ComputeFinalWeights(SetupWeights, VertexWeights, FinalWeights);
}

void FDataflowCollectionAddScalarVertexPropertyNode::ComputeFinalWeights(TConstArrayView<float> InputWeights, TConstArrayView<float> StoredWeights, TArrayView<float> FinalWeights) const
{
	check(InputWeights.Num() == FinalWeights.Num());

	int32 NumWeights = FMath::Min(StoredWeights.Num(), InputWeights.Num());
	if(NumWeights != 0)
	{ 
		for (int32 WeightIndex = 0; WeightIndex < NumWeights; ++WeightIndex)
		{
			switch (OverrideType)
			{
				case EDataflowWeightMapOverrideType::ReplaceAll:
					FinalWeights[WeightIndex] = FMath::Clamp(StoredWeights[WeightIndex], 0.f, 1.f);
					break;
				case EDataflowWeightMapOverrideType::ReplaceChanged:
					FinalWeights[WeightIndex] = FMath::Clamp( (StoredWeights[WeightIndex] == ReplaceChangedPassthroughValue) ? InputWeights[WeightIndex] : StoredWeights[WeightIndex], 0.0f, 1.0f);
					break;
				case EDataflowWeightMapOverrideType::AddDifference:
					FinalWeights[WeightIndex] = FMath::Clamp(InputWeights[WeightIndex] + StoredWeights[WeightIndex], 0.f, 1.f);
					break;
				default: unimplemented();
			}
		}
	}
	else
	{
		NumWeights = InputWeights.Num();
		for (int32 WeightIndex = 0; WeightIndex < NumWeights; ++WeightIndex)
		{
			FinalWeights[WeightIndex] = InputWeights[WeightIndex];
		}
	}
}

namespace DataflowCollectionAddScalarVertexPropertyNode::Private
{
	static FName VertexGroupName = TEXT("Vertices");
	static FName VertexPositionsAttributeName = TEXT("Positions");
	static FName VertexWeightsAttributeName = TEXT("Weights");

	static FName FaceGroupName = TEXT("Faces");
	static FName FaceIndicesAttributeName = TEXT("Indices");

	template<typename T>
	void CopyData(TConstArrayView<T> From, TManagedArray<T>& To)
	{
		if (ensure(From.Num() == To.Num()))
		{
			for (int32 Index = 0; Index < From.Num(); ++Index)
			{
				To[Index] = From[Index];
			}
		}
	};
}

void FDataflowCollectionAddScalarVertexPropertyNode::GetWeightsFromActiveSnapshot(const FManagedArrayCollection& InCollection, FName TargetGroupName, TArrayView<float> OutValues) const
{
	if (const FDataflowToolNodeSnapshot* ActiveSnapshot = GetActiveSnapshot())
	{
		GetWeightsFromSnapshot(InCollection, *ActiveSnapshot, TargetGroupName, OutValues);
	}
	else
	{
		// This keeps backward compatibility to load what's on the node before we add snapshots
		// if topology as changed since, for now let's simply do our best 
		const int32 NumValuesToCopy = FMath::Min(VertexWeights.Num(), OutValues.Num());
		for (int32 Index = 0; Index < NumValuesToCopy; ++Index)
		{
			OutValues[Index] = VertexWeights[Index];
		}
	}
}

void FDataflowCollectionAddScalarVertexPropertyNode::GetWeightsFromSnapshot(const FManagedArrayCollection& InCollection, const FDataflowToolNodeSnapshot& Snapshot, FName TargetGroupName, TArrayView<float> OutValues) const
{
	namespace NodePrivate = DataflowCollectionAddScalarVertexPropertyNode::Private;
	
	using namespace UE::Dataflow;

	const TManagedArray<float>* VtxWeights = Snapshot.Data.FindAttribute<float>(NodePrivate::VertexWeightsAttributeName, NodePrivate::VertexGroupName);
	if (VtxWeights)
	{
		FDataflowCollectionFacade Facade(InCollection, TargetGroupName);
		TConstArrayView<FVector3f> CollectionVtxPositions = Facade.GetVertexPositions();
		TConstArrayView<FIntVector> CollectionTriangles = Facade.GetTriangleIndices();

		const int32 NumVertices = CollectionVtxPositions.Num();
		const int32 NumTriangles = CollectionTriangles.Num();

		bool bMatchOriginalTopology = true;
		bMatchOriginalTopology &= (VtxWeights->Num() == OutValues.Num());
		bMatchOriginalTopology &= (Snapshot.Data.NumElements(NodePrivate::VertexGroupName) == NumVertices);
		bMatchOriginalTopology &= (Snapshot.Data.NumElements(NodePrivate::FaceGroupName) == NumTriangles);
		// todo : add more checks

		const TManagedArray<FVector3f>* VtxPositions = Snapshot.Data.FindAttribute<FVector3f>(NodePrivate::VertexPositionsAttributeName, NodePrivate::VertexGroupName);
		const TManagedArray<FIntVector>* FaceIndices = Snapshot.Data.FindAttribute<FIntVector>(NodePrivate::FaceIndicesAttributeName, NodePrivate::FaceGroupName);
		const bool bIsValidSourceMesh = (VtxPositions && FaceIndices);

		const bool bIsValidTargetMesh = (NumVertices > 0 && NumTriangles > 0);

		if (!bMatchOriginalTopology)
		{
			if (bIsValidSourceMesh && bIsValidTargetMesh)
			{
				// TODO(Dataflow) need to transfer the weights from one topo to another 
				// in this automatic mode we only transfer on 3D model because the patterns may have changed and be in different place
				// we also only store the original 3D Mesh 
			}
			else
			{
				// no transfer since there's no geometry to transfer to or from
			}
		}
		else
		{
			// We simply read the weights as they are 
			check(VtxWeights->Num() == OutValues.Num());
			for (int32 Index = 0; Index < VtxWeights->Num(); ++Index)
			{
				OutValues[Index] = (*VtxWeights)[Index];
			}
		}
	}
}

void FDataflowCollectionAddScalarVertexPropertyNode::StoreWeightsInSnapshot(const FManagedArrayCollection& InCollection, TConstArrayView<float> InValues, FName TargetGroupName, FDataflowToolNodeSnapshot& OutSnapshot) const
{
	namespace NodePrivate = DataflowCollectionAddScalarVertexPropertyNode::Private;
	using namespace UE::Dataflow;

	OutSnapshot.Data.Reset();

	FDataflowCollectionFacade Facade(InCollection, TargetGroupName);
	TConstArrayView<FVector3f> CollectionVtxPositions = Facade.GetVertexPositions();
	TConstArrayView<FIntVector> CollectionTriangles = Facade.GetTriangleIndices();

	if (InValues.Num() && InValues.Num() == CollectionVtxPositions.Num())
	{
		OutSnapshot.Data.AddGroup(NodePrivate::VertexGroupName);
		OutSnapshot.Data.Resize(CollectionVtxPositions.Num(), NodePrivate::VertexGroupName);

		OutSnapshot.Data.AddGroup(NodePrivate::FaceGroupName);
		OutSnapshot.Data.Resize(CollectionTriangles.Num(), NodePrivate::FaceGroupName);

		TManagedArray<float>& VtxWeights = OutSnapshot.Data.AddAttribute<float>(NodePrivate::VertexWeightsAttributeName, NodePrivate::VertexGroupName);
		NodePrivate::CopyData(MakeArrayView(InValues), VtxWeights);

		TManagedArray<FVector3f>& VtxPositions = OutSnapshot.Data.AddAttribute<FVector3f>(NodePrivate::VertexPositionsAttributeName, NodePrivate::VertexGroupName);
		NodePrivate::CopyData(CollectionVtxPositions, VtxPositions);

		TManagedArray<FIntVector>& FaceIndices = OutSnapshot.Data.AddAttribute<FIntVector>(NodePrivate::FaceIndicesAttributeName, NodePrivate::FaceGroupName);
		NodePrivate::CopyData(CollectionTriangles, FaceIndices);
	}
}

FDataflowNode::FAttributeKey FDataflowCollectionAddScalarVertexPropertyNode::GetVertexAttributeToVisualize(UE::Dataflow::FContext& Context, const FName OutputName, const FName RenderGroup) const
{
	const FCollectionAttributeKey CollectionAttributeKey = GetWeightAttributeKey(Context);

	return FAttributeKey
	{
		.AttributeName = FName(CollectionAttributeKey.Attribute),
		.GroupName = FName(CollectionAttributeKey.Group),
	};
}

#undef LOCTEXT_NAMESPACE
