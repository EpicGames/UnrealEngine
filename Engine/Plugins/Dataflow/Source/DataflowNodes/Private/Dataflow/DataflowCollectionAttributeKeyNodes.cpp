// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowCollectionAttributeKeyNodes.h"
#include "GeometryCollection/GeometryCollection.h"

#include "Dataflow/DataflowCore.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowCollectionAttributeKeyNodes)

namespace UE::Dataflow
{
	void DataflowCollectionAttributeKeyNodes()
	{
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMakeAttributeKeyDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FBreakAttributeKeyDataflowNode);
	}
}


void FMakeAttributeKeyDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	FString AttributeName = GetValue<FString>(Context, &AttributeIn);
	FString GroupName = GetValue<FString>(Context, &GroupIn);
	SetValue(Context, FCollectionAttributeKey(GroupName, AttributeName), &AttributeKeyOut);
}

void FBreakAttributeKeyDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	FCollectionAttributeKey CollectionKey = GetValue<FCollectionAttributeKey>(Context, &AttributeKeyIn);
	SetValue(Context, CollectionKey.Attribute, &AttributeOut);
	SetValue(Context, CollectionKey.Group, &GroupOut);
}