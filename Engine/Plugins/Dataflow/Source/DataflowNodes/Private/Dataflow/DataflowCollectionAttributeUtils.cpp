// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowCollectionAttributeUtils.h"

#include "Dataflow/DataflowNode.h"
#include "GeometryCollection/GeometryCollection.h"


#define LOCTEXT_NAMESPACE "DataflowCollectionAttributeUtils"

namespace UE::Dataflow
{
	FDataflowCollectionFacade::FDataflowCollectionFacade(const FManagedArrayCollection& InCollection, FName TargetGroup)
		: Collection(InCollection)
	{
		TargetGroupInfo = FDataflowAddScalarVertexPropertyCallbackRegistry::Get().GetTargetGroupInfo(TargetGroup);
	}
	
	TConstArrayView<FVector3f> FDataflowCollectionFacade::GetVertexPositions() const
	{
		if (TargetGroupInfo.TargetGroup.IsNone())
		{
			return {};
		}

		const FDataflowNode::FAttributeKey Key = TargetGroupInfo.PositionAttributeKey;

		if (const TManagedArray<FVector3f>* VertexAttribute = Collection.FindAttributeTyped<FVector3f>(Key.AttributeName, Key.GroupName))
		{
			return VertexAttribute->GetConstArray();
		}
		return {};
	}

	TConstArrayView<FIntVector> FDataflowCollectionFacade::GetTriangleIndices() const
	{
		if (TargetGroupInfo.TargetGroup.IsNone())
		{
			return {};
		}

		const FDataflowNode::FAttributeKey Key = TargetGroupInfo.IndicesAttributeKey;

		if (const TManagedArray<FIntVector>* IndicesAttribute = Collection.FindAttributeTyped<FIntVector>(Key.AttributeName, Key.GroupName))
		{
			return IndicesAttribute->GetConstArray();
		}
		return {};
	}

	TConstArrayView<int32> FDataflowCollectionFacade::Get2Dto3DMapping() const
	{
		if (TargetGroupInfo.TargetGroup.IsNone())
		{
			return {};
		}

		const FDataflowNode::FAttributeKey Key = TargetGroupInfo.MappingFrom2DTo3DAttributeKey;
		if (const TManagedArray<int32>* IndicesAttribute = Collection.FindAttributeTyped<int32>(Key.AttributeName, Key.GroupName))
		{
			return IndicesAttribute->GetConstArray();
		}
		return {};
	}
}

#undef LOCTEXT_NAMESPACE 

