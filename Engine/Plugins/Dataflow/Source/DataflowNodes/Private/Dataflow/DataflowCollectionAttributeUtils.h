// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include <GeometryCollection/ManagedArrayCollection.h>
#include <Math/MathFwd.h>
#include <UObject/NameTypes.h>
#include "Dataflow/DataflowCollectionAddScalarVertexPropertyNode.h"

namespace UE::Dataflow
{
	// General collection facade to help nodes to deal with both Cloth and geometry collections
	struct FDataflowCollectionFacade
	{
	public:
		FDataflowCollectionFacade(const FManagedArrayCollection& Collection, FName TargetGroup);
		
		TConstArrayView<FVector3f> GetVertexPositions() const;
		TConstArrayView<FIntVector> GetTriangleIndices() const;
		TConstArrayView<int32> Get2Dto3DMapping() const;

	private:
		using FTargetGroupInfo = FDataflowAddScalarVertexPropertyCallbackRegistry::FTargetGroupInfo;

		const FManagedArrayCollection& Collection;
		FTargetGroupInfo TargetGroupInfo;
	};

}
