// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowCollectionEditSkeletonBonesNode.h"
#include "Dataflow/DataflowInputOutput.h"
#include "Dataflow/DataflowDebugDrawObject.h"
#include "DataflowCollectionSkeletalMeshUtils.h"
#include "GeometryCollection/Facades/CollectionRenderingFacade.h"
#include "Rendering/SkeletalMeshModel.h"
#include "Engine/SkeletalMesh.h"
#include "Animation/Skeleton.h"
#include "Components/SkeletalMeshComponent.h"

#if WITH_EDITOR
#include "StaticToSkeletalMeshConverter.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowCollectionEditSkeletonBonesNode)

#define LOCTEXT_NAMESPACE "DataflowCollectionEditSkeleton"

FDataflowCollectionEditSkeletonBonesNode::FDataflowCollectionEditSkeletonBonesNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowPrimitiveNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&Skeleton);
	RegisterOutputConnection(&Collection, &Collection);
	RegisterOutputConnection(&Skeleton, &Skeleton);
}

TArray<UE::Dataflow::FRenderingParameter> FDataflowCollectionEditSkeletonBonesNode::GetRenderParametersImpl() const
{
	return FDataflowAddScalarVertexPropertyCallbackRegistry::Get().GetRenderingParameters(VertexGroup.Name);
}

void FDataflowCollectionEditSkeletonBonesNode::AddPrimitiveComponents(UE::Dataflow::FContext& Context, const TSharedPtr<const FManagedArrayCollection> RenderCollection, TObjectPtr<UObject> NodeOwner, 
	TObjectPtr<AActor> RootActor, TArray<TObjectPtr<UPrimitiveComponent>>& PrimitiveComponents)  
{
	if(RootActor && RenderCollection)
	{
		if(TObjectPtr<USkeleton> TransientSkeleton = UpdateToolSkeleton(Context))
		{
			GeometryCollection::Facades::FRenderingFacade RenderingFacade(*RenderCollection);
			const int32 NumGeometry = RenderingFacade.IsValid() ? RenderingFacade.NumGeometry() : 0;
		
			const bool bNeedsConstruction = (SkeletalMeshes.Num() != NumGeometry) || !bValidSkeletalMeshes;
		
			if(SkeletalMeshes.Num() != NumGeometry)
			{
				SkeletalMeshes.Reset();
				for(int32 GeometryIndex = 0; GeometryIndex < NumGeometry; ++GeometryIndex)
				{
					FString SkeletalMeshName = FString("SK_DataflowSkeletalMesh_") + FString::FromInt(GeometryIndex);
					SkeletalMeshName = MakeUniqueObjectName(NodeOwner, USkeletalMesh::StaticClass(), *SkeletalMeshName, EUniqueObjectNameOptions::GloballyUnique).ToString();
					SkeletalMeshes.Add(NewObject<USkeletalMesh>(NodeOwner, FName(*SkeletalMeshName), RF_Transient));
				}
			}
			if(bNeedsConstruction)
			{
				bValidSkeletalMeshes = UE::Dataflow::Private::BuildSkeletalMeshes(SkeletalMeshes, RenderCollection, TransientSkeleton->GetReferenceSkeleton());
				if(!bValidSkeletalMeshes)
				{
					SkeletalMeshes.Reset();
				}
			}
			if(!SkeletalMeshes.IsEmpty())
			{
				for(int32 GeometryIndex = 0; GeometryIndex < NumGeometry; ++GeometryIndex)
				{
					FName SkeletalMeshName(FString("Dataflow_SkeletalMesh") + FString::FromInt(GeometryIndex));
					USkeletalMeshComponent* SkeletalMeshComponent = NewObject<USkeletalMeshComponent>(RootActor, SkeletalMeshName);
					SkeletalMeshes[GeometryIndex]->SetSkeleton(TransientSkeleton);
					SkeletalMeshComponent->SetSkeletalMesh(SkeletalMeshes[GeometryIndex]);
					PrimitiveComponents.Add(SkeletalMeshComponent);
				}
			}
		}
	}
}

TObjectPtr<USkeleton> FDataflowCollectionEditSkeletonBonesNode::UpdateToolSkeleton(UE::Dataflow::FContext& Context)
{
	if (ToolSkeleton == nullptr)
	{
		TObjectPtr<USkeleton> InputSkeleton = GetValue(Context, &Skeleton, Skeleton);
		ToolSkeleton = DuplicateObject<USkeleton>(InputSkeleton, nullptr);
	}
	return ToolSkeleton;
}

void FDataflowCollectionEditSkeletonBonesNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace UE::Dataflow;
	
	if (Out->IsA(&Collection))
	{
		SafeForwardInput(Context, &Collection, &Collection);
	}
	else if (Out->IsA(&Skeleton))
	{
		if(ToolSkeleton)
		{ 
			SetValue(Context, ToolSkeleton, &Skeleton);
		}
		else
		{
			SafeForwardInput(Context, &Skeleton, &Skeleton);
		}
	}
}

void FDataflowCollectionEditSkeletonBonesNode::OnInvalidate()
{
	Super::OnInvalidate();
	
	bValidSkeletalMeshes = false;
}

#if WITH_EDITOR

void FDataflowCollectionEditSkeletonBonesNode::DebugDraw(UE::Dataflow::FContext& Context,
		IDataflowDebugDrawInterface& DataflowRenderingInterface,
		const FDebugDrawParameters& DebugDrawParameters) const
{
	if(TObjectPtr<USkeleton> InputSkeleton = GetValue(Context, &Skeleton, Skeleton))
	{
		TRefCountPtr<FDataflowDebugDrawSkeletonObject> SkeletonObject = MakeDebugDrawObject<FDataflowDebugDrawSkeletonObject>(
			DataflowRenderingInterface.ModifyDataflowElements(), InputSkeleton->GetReferenceSkeleton(), false);
		
		DataflowRenderingInterface.DrawObject(TRefCountPtr<IDataflowDebugDrawObject>(SkeletonObject));
		
		SkeletonObject->OnBoneSelectionChanged.AddLambda([this](const TArray<FName>& BoneNames)
		{
			OnBoneSelectionChanged.Broadcast(BoneNames);
		});
	}
}

bool FDataflowCollectionEditSkeletonBonesNode::CanDebugDrawViewMode(const FName& ViewModeName) const
{
	return true;
}

#endif

#undef LOCTEXT_NAMESPACE
