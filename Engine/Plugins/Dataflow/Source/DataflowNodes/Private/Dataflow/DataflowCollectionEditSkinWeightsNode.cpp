// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowCollectionEditSkinWeightsNode.h"
#include "Dataflow/DataflowInputOutput.h"
#include "Dataflow/DataflowDebugDrawObject.h"
#include "Dataflow/DataflowSkeletalMeshNodes.h"
#include "Dataflow/CollectionRenderingPatternUtility.h"
#include "DataflowCollectionSkeletalMeshUtils.h"
#include "DynamicMeshToMeshDescription.h"
#include "DynamicMesh/DynamicBoneAttribute.h"
#include "MeshConversionOptions.h"
#include "SkeletalMeshAttributes.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "GeometryCollection/Facades/CollectionRenderingFacade.h"
#include "Rendering/SkeletalMeshModel.h"
#include "Engine/SkeletalMesh.h"
#include "Animation/Skeleton.h"
#include "Components/DynamicMeshComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "InteractiveToolChange.h"
#include "DynamicMesh/DynamicVertexSkinWeightsAttribute.h"
#include "DynamicMesh/NonManifoldMappingSupport.h"

#if WITH_EDITOR
#include "StaticToSkeletalMeshConverter.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowCollectionEditSkinWeightsNode)

#define LOCTEXT_NAMESPACE "DataflowCollectionEditSkinWeights"


namespace UE::Dataflow::Private
{
	bool bEditSkinWeightsNodeGeneratesDynamicMesh = true;
	FAutoConsoleVariableRef CVarEditSkinWeightsNodeGeneratesDynamicMesh(TEXT("p.Dataflow.EditSkinWeightsNodeGeneratesDynamicMesh"), bEditSkinWeightsNodeGeneratesDynamicMesh, TEXT("When on the edit skin weight node will generate dynamic mesh component instead of the mode expensive skeletal mesh ones"));


template<typename ScalarType, typename VectorType, int32 NumComponents>
bool SetAttributeValues(FManagedArrayCollection& SelectedCollection,
	const FCollectionAttributeKey& AttributeKey, const TArray<TArray<ScalarType>>& AttributeValues, const ScalarType& DefaultValue, const bool bRenormalizeValues)
{
	if (!AttributeValues.IsEmpty() && !AttributeKey.Attribute.IsEmpty() && !AttributeKey.Group.IsEmpty())
	{
		const FName AttributeName(AttributeKey.Attribute);
		const FName AttributeGroup(AttributeKey.Group);
		
		if(TManagedArray<TArray<ScalarType>>* AttributeArray = SelectedCollection.FindAttributeTyped<TArray<ScalarType>>(AttributeName, AttributeGroup))
		{
			if(AttributeArray->Num() == AttributeValues.Num())
			{
				for(int32 VertexIndex = 0, NumVertices = AttributeArray->Num(); VertexIndex < NumVertices; ++VertexIndex)
				{
					AttributeArray->GetData()[VertexIndex] = AttributeValues[VertexIndex];
				}
			}
			return true;
		}
		else if(TManagedArray<VectorType>* AttributeVector = SelectedCollection.FindAttributeTyped<VectorType>(AttributeName, AttributeGroup))
		{
			if(AttributeVector->Num() == AttributeValues.Num())
			{
				for(int32 VertexIndex = 0, NumVertices = AttributeVector->Num(); VertexIndex < NumVertices; ++VertexIndex)
				{
					VectorType& ElementVector = AttributeVector->GetData()[VertexIndex];
					const int32 NumValidComponents = FMath::Min(NumComponents, AttributeValues[VertexIndex].Num());
					
					float TotalValue = 0.0f;
					for(int32 ComponentIndex = 0; ComponentIndex < NumValidComponents; ++ComponentIndex)
					{
						ElementVector[ComponentIndex] = AttributeValues[VertexIndex][ComponentIndex];
						TotalValue += ElementVector[ComponentIndex];
					}

					for(int32 ComponentIndex = 0; ComponentIndex < NumComponents; ++ComponentIndex)
					{
						ElementVector[ComponentIndex] = (ComponentIndex >= NumValidComponents) ? DefaultValue : (bRenormalizeValues && TotalValue != 0.0f)
							?  ElementVector[ComponentIndex] / TotalValue : ElementVector[ComponentIndex];
					}
				}
			}
			return true;
		}
	}
	return false;
}

template<typename ScalarType, typename VectorType, int32 NumComponents>
bool FillAttributeValues(const FManagedArrayCollection& SelectedCollection,
	const FCollectionAttributeKey& AttributeKey, TArray<TArray<ScalarType>>& AttributeValues)
{
	if (!AttributeKey.Attribute.IsEmpty() && !AttributeKey.Group.IsEmpty())
	{
		const FName AttributeName(AttributeKey.Attribute);
		const FName AttributeGroup(AttributeKey.Group);

		if(const TManagedArray<TArray<ScalarType>>* AttributeArray = SelectedCollection.FindAttributeTyped<TArray<ScalarType>>(AttributeName, AttributeGroup))
		{
			AttributeValues = AttributeArray->GetConstArray();
		}
		else if(const TManagedArray<VectorType>* AttributeVector = SelectedCollection.FindAttributeTyped<VectorType>(AttributeName, AttributeGroup))
		{
			AttributeValues.SetNum(AttributeVector->Num());
			for(int32 VertexIndex = 0, NumVertices = AttributeVector->Num(); VertexIndex < NumVertices; ++VertexIndex)
			{
				const VectorType& ElementVector = AttributeVector->GetConstArray()[VertexIndex];
				AttributeValues[VertexIndex].SetNum(NumComponents);
				
				for(int32 ComponentIndex = 0; ComponentIndex < NumComponents; ++ComponentIndex)
				{
					AttributeValues[VertexIndex][ComponentIndex] = ElementVector[ComponentIndex];
				}
			}
		}
		else
		{	
			AttributeValues.SetNum(SelectedCollection.NumElements(*AttributeKey.Group));
		}
		return true;
	}
	return false;
}

template<typename ScalarType, typename VectorType, int32 NumComponents>
bool GetAttributeValues(FManagedArrayCollection& SelectedCollection,
	const FCollectionAttributeKey& AttributeKey, TArray<TArray<ScalarType>>& AttributeValues, const bool bVectorValues)
{
	if (!AttributeKey.Attribute.IsEmpty() && !AttributeKey.Group.IsEmpty())
	{
		const FName AttributeName(AttributeKey.Attribute);
		const FName AttributeGroup(AttributeKey.Group);

		if(SelectedCollection.FindAttributeTyped<TArray<ScalarType>>(AttributeName, AttributeGroup) == nullptr &&
			SelectedCollection.FindAttributeTyped<VectorType>(AttributeName, AttributeGroup) == nullptr)
		{
			if(bVectorValues)
			{
				SelectedCollection.AddAttribute<VectorType>(AttributeName, AttributeGroup);
			}
			else
			{
				SelectedCollection.AddAttribute<TArray<ScalarType>>(AttributeName, AttributeGroup);
			}
		}
	}
	return FillAttributeValues<ScalarType,VectorType, NumComponents>(SelectedCollection, AttributeKey, AttributeValues);
}

void CorrectSkinWeights(TArray<TArray<int32>>& BoneIndices, TArray<TArray<float>>& BoneWeights)
{
	check(BoneIndices.Num() == BoneWeights.Num());

	TArray<int32> ValidIndices;
	TArray<float> ValidWeights;
	
	for(int32 VertexIndex = 0, NumVertices = BoneWeights.Num(); VertexIndex < NumVertices; ++VertexIndex)
	{
		if(BoneWeights[VertexIndex].Num() == BoneIndices[VertexIndex].Num())
		{
			ValidIndices.Reset(BoneIndices[VertexIndex].Num());
			ValidWeights.Reset(BoneWeights[VertexIndex].Num());
			
			for(int32 WeightIndex = 0, NumWeights = BoneWeights[VertexIndex].Num(); WeightIndex < NumWeights; ++WeightIndex)
			{
				if(BoneIndices[VertexIndex][WeightIndex] != INDEX_NONE)
				{
					ValidIndices.Add(BoneIndices[VertexIndex][WeightIndex]);
					ValidWeights.Add(BoneWeights[VertexIndex][WeightIndex]);
				}
			}
			BoneIndices[VertexIndex] = ValidIndices;
			BoneWeights[VertexIndex] = ValidWeights;
		}
	}
}

}

void FDataflowVertexSkinWeightData::FromArrays(int32 VertexIndex, const TArray<int32>& InIndices, const TArray<float>& InWeights)
{
	if (ensure(VertexIndex < ArrayNum))
	{
		ensure(InIndices.Num() == InWeights.Num());
		const int32 Offset = VertexIndex * MaxNumInflences;
		for (int32 Index = 0; Index < MaxNumInflences; ++Index)
		{
			Bones[Offset + Index] = (InIndices.IsValidIndex(Index)) ? InIndices[Index] : INDEX_NONE;
			Weights[Offset + Index] = (InWeights.IsValidIndex(Index)) ? InWeights[Index] : 0.0f;
		}
	}
}

void FDataflowVertexSkinWeightData::ToArrays(int32 VertexIndex, TArray<int32>& OutIndices, TArray<float>& OutWeights) const
{
	if (ensure(VertexIndex < ArrayNum))
	{
		const int32 Offset = VertexIndex * MaxNumInflences;
		for (int32 Index = 0; Index < MaxNumInflences; ++Index)
		{
			if (Bones[Offset + Index] != INDEX_NONE)
			{
				OutIndices.Add(Bones[Offset + Index]);
				OutWeights.Add(Weights[Offset + Index]);
			}
		}
	}
}

int32 FDataflowVertexSkinWeightData::Num() const
{
	return ArrayNum;
}

void FDataflowVertexSkinWeightData::SetNum(int32 VertexNum)
{
	if (ArrayNum != VertexNum)
	{
		Bones.Init(INDEX_NONE, VertexNum * MaxNumInflences);
		Weights.Init(0.0f, VertexNum * MaxNumInflences);
		ArrayNum = VertexNum;
	}
	
}

void FDataflowVertexSkinWeightData::Reset()
{
	Bones.Reset();
	Weights.Reset();
	ArrayNum = 0;
}

bool FDataflowVertexSkinWeightData::Serialize(FArchive& Ar)
{
	Ar << ArrayNum;
	Bones.BulkSerialize(Ar);
	Weights.BulkSerialize(Ar);
	return true;
}

FArchive& operator<<(FArchive& Ar, FDataflowVertexSkinWeightData& Data)
{
	Data.Serialize(Ar);
	return Ar;
}


//
// FDataflowCollectionEditSkinWeightsNode
//

FDataflowCollectionEditSkinWeightsNode::FDataflowCollectionEditSkinWeightsNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowPrimitiveNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&BoneIndicesKey);
	RegisterInputConnection(&BoneWeightsKey);
	RegisterInputConnection(&SkeletalMesh);
	RegisterOutputConnection(&Collection, &Collection);
	RegisterOutputConnection(&BoneIndicesKey, &BoneIndicesKey);
	RegisterOutputConnection(&BoneWeightsKey, &BoneWeightsKey);
}

void FDataflowCollectionEditSkinWeightsNode::PostSerialize(const FArchive& Ar)
{
	FDataflowPrimitiveNode::PostSerialize(Ar);

	if (Ar.IsLoading())
	{
		if (SkinWeights_DEPRECATED.Num() > 0)
		{
			VertexSkinWeights.SetNum(SkinWeights_DEPRECATED.Num());
			for (int32 Index = 0; Index < SkinWeights_DEPRECATED.Num(); ++Index)
			{
				const FDataflowSkinWeightData& OldSkinWeights = SkinWeights_DEPRECATED[Index];
				VertexSkinWeights.FromArrays(Index, OldSkinWeights.BoneIndices, OldSkinWeights.BoneWeights);
			}
		}
		SkinWeights_DEPRECATED.Empty();
	}
}

TArray<UE::Dataflow::FRenderingParameter> FDataflowCollectionEditSkinWeightsNode::GetRenderParametersImpl() const
{
	return FDataflowAddScalarVertexPropertyCallbackRegistry::Get().GetRenderingParameters(VertexGroup.Name);
}

void FDataflowCollectionEditSkinWeightsNode::AddPrimitiveComponents(UE::Dataflow::FContext& Context, const TSharedPtr<const FManagedArrayCollection> RenderCollection, TObjectPtr<UObject> NodeOwner,
	TObjectPtr<AActor> RootActor, TArray<TObjectPtr<UPrimitiveComponent>>& PrimitiveComponents)  
{
	if(RootActor && RenderCollection)
	{
		GeometryCollection::Facades::FRenderingFacade RenderingFacade(*RenderCollection);
		const int32 NumGeometry = RenderingFacade.IsValid() ? RenderingFacade.NumGeometry() : 0;
		
		const bool bHasSkinWeightsDataInRenderCollection = ((RenderingFacade.FindBoneIndices() != nullptr) && (RenderingFacade.FindBoneWeights() != nullptr));
		if (UE::Dataflow::Private::bEditSkinWeightsNodeGeneratesDynamicMesh && bHasSkinWeightsDataInRenderCollection)
		{
			const bool bNeedsConstruction = (DynamicMeshes.Num() != NumGeometry);
			if (bNeedsConstruction)
			{
				TObjectPtr<USkeletalMesh> InSkeletalMesh = GetValue(Context, &SkeletalMesh, SkeletalMesh);

				DynamicMeshes.SetNum(NumGeometry);
				for (int32 MeshIndex = 0; MeshIndex < NumGeometry; ++MeshIndex)
				{
					DynamicMeshes[MeshIndex] = NewObject<UDynamicMesh>(RootActor);
				}

				ParallelFor(DynamicMeshes.Num(),
					[this, &RenderingFacade, &InSkeletalMesh](int32 GeometryIndex)
					{
						TObjectPtr<UDynamicMesh> DynamicMeshObject = DynamicMeshes[GeometryIndex];
						check(DynamicMeshObject && DynamicMeshObject->GetMeshPtr());

						FDynamicMesh3& DynamicMesh = DynamicMeshObject->GetMeshRef();
						UE::Dataflow::Conversion::RenderingFacadeToDynamicMesh(RenderingFacade, GeometryIndex, DynamicMesh);

						if (InSkeletalMesh)
						{
							const FReferenceSkeleton& RefSkeleton = InSkeletalMesh->GetRefSkeleton();
							const int32 NumBones = RefSkeleton.GetRawBoneNum();
							if (NumBones)
							{
								using namespace UE::Geometry; 
								if (FDynamicMeshAttributeSet* Attributes = DynamicMesh.Attributes())
								{
									Attributes->EnableBones(NumBones);
									FDynamicMeshBoneNameAttribute* BoneNameAttrib = Attributes->GetBoneNames();
									FDynamicMeshBoneParentIndexAttribute* BoneParentIndices = Attributes->GetBoneParentIndices();
									FDynamicMeshBonePoseAttribute* BonePoses = Attributes->GetBonePoses();
									FDynamicMeshBoneColorAttribute* BoneColors = Attributes->GetBoneColors();

									for (int32 BoneIdx = 0; BoneIdx < NumBones; ++BoneIdx)
									{
										BoneNameAttrib->SetValue(BoneIdx, RefSkeleton.GetRawRefBoneInfo()[BoneIdx].Name);
										BoneParentIndices->SetValue(BoneIdx, RefSkeleton.GetRawRefBoneInfo()[BoneIdx].ParentIndex);
										BonePoses->SetValue(BoneIdx, RefSkeleton.GetRawRefBonePose()[BoneIdx]);
										BoneColors->SetValue(BoneIdx, FVector4f::One());
									}

									if(VertexSkinWeights.Num() > 0)
									{ 
										FDynamicMeshVertexSkinWeightsAttribute* SkinWeights = Attributes->GetSkinWeightsAttribute(FSkeletalMeshAttributesShared::DefaultSkinWeightProfileName);

										TArray<int32> BoneIndices;
										TArray<float> BoneWeights;

										// Per-geometry vertex offset into the collection-global VertexSkinWeights array.
										const int32 GeometryVertexOffset = RenderingFacade.GetVertexStart()[GeometryIndex];

										FNonManifoldMappingSupport NonManifoldMappingSupport(DynamicMesh);
										for (int32 MeshVertexIndex : DynamicMesh.VertexIndicesItr())
										{
											BoneIndices.Reset();
											BoneWeights.Reset();

											// Translate to the collection-global index used to key VertexSkinWeights.
											const int32 GlobalVertexIndex = GeometryVertexOffset + NonManifoldMappingSupport.GetOriginalNonManifoldVertexID(MeshVertexIndex);
											VertexSkinWeights.ToArrays(GlobalVertexIndex, BoneIndices, BoneWeights);
											check(BoneIndices.Num() == BoneWeights.Num());

											UE::AnimationCore::FBoneWeights WeightsData;
											const int32 NumWeights = BoneIndices.Num();
											for (int32 WeightIndex = 0; WeightIndex < NumWeights; ++WeightIndex)
											{
												WeightsData.SetBoneWeight(BoneIndices[WeightIndex], BoneWeights[WeightIndex]);
											}
											SkinWeights->SetValue(MeshVertexIndex, WeightsData);
										}
									}
								}
							}
						}
					});
			}
			if (!DynamicMeshes.IsEmpty())
			{
				PrimitiveComponents.Reserve(PrimitiveComponents.Num() + NumGeometry);
				UMaterial* DefaultTwoSidedMaterial = Cast<UMaterial>(StaticLoadObject(UMaterial::StaticClass(), NULL, TEXT("/Engine/EditorMaterials/Dataflow/DataflowTwoSidedVertexMaterial")));
				for (int32 GeometryIndex = 0; GeometryIndex < NumGeometry; ++GeometryIndex)
				{
					const FName DynamicMeshName(FString::Printf(TEXT("[%d]_DynamicMesh_(%s)"), GeometryIndex, *FGuid::NewGuid().ToString()));
					UDynamicMeshComponent* DynamicMeshComponent = NewObject<UDynamicMeshComponent>(RootActor, DynamicMeshName);
					DynamicMeshComponent->SetDynamicMesh(DynamicMeshes[GeometryIndex]);
					DynamicMeshComponent->SetOverrideRenderMaterial(DefaultTwoSidedMaterial);
					PrimitiveComponents.Add(DynamicMeshComponent);
				}
			}

			return;
		}

		const bool bNeedsConstruction = (SkeletalMeshes.Num() != NumGeometry) || !bValidSkeletalMeshes;
		
		if(SkeletalMeshes.Num() != NumGeometry)
		{
			SkeletalMeshes.Reset();
			for(int32 GeometryIndex = 0; GeometryIndex < NumGeometry; ++GeometryIndex)
			{
				const FString GeometryName = RenderingFacade.GetGeometryName()[GeometryIndex];
				FString SkeletalMeshName = GeometryName.IsEmpty() ? FString("SK_DataflowSkeletalMesh_") + FString::FromInt(GeometryIndex) : GeometryName;
				SkeletalMeshName = MakeUniqueObjectName(NodeOwner, USkeletalMesh::StaticClass(), *SkeletalMeshName, EUniqueObjectNameOptions::GloballyUnique).ToString();
				SkeletalMeshes.Add(NewObject<USkeletalMesh>(NodeOwner, FName(*SkeletalMeshName), RF_Transient));
			}
		}

		TObjectPtr<USkeletalMesh> InSkeletalMesh = GetValue(Context, &SkeletalMesh, SkeletalMesh);
		if(bNeedsConstruction && InSkeletalMesh)
		{
			bValidSkeletalMeshes = UE::Dataflow::Private::BuildSkeletalMeshes(SkeletalMeshes, RenderCollection, InSkeletalMesh->GetRefSkeleton());
			if(!bValidSkeletalMeshes)
			{
				SkeletalMeshes.Reset();
			}
		}
		if(!SkeletalMeshes.IsEmpty())
		{
			for(int32 GeometryIndex = 0; GeometryIndex < NumGeometry; ++GeometryIndex)
			{
				FName SkeletalMeshName(FString("Dataflow_SkeletalMesh") + FString::FromInt(GeometryIndex));
				USkeletalMeshComponent* SkeletalMeshComponent = NewObject<USkeletalMeshComponent>(RootActor, SkeletalMeshName);
				SkeletalMeshComponent->SetSkeletalMesh(SkeletalMeshes[GeometryIndex]);
				UMaterial* DefaultTwoSidedMaterial = Cast<UMaterial>(StaticLoadObject(UMaterial::StaticClass(), NULL, TEXT("/Engine/EditorMaterials/Dataflow/DataflowTwoSidedVertexMaterial")));
				SkeletalMeshComponent->SetMaterial(0, DefaultTwoSidedMaterial);
				PrimitiveComponents.Add(SkeletalMeshComponent);
			}
		}
	}
}

void FDataflowCollectionEditSkinWeightsNode::OnPropertyChanged(UE::Dataflow::FContext& Context, const FPropertyChangedEvent& InPropertyChangedEvent)
{
	if (InPropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(FDataflowCollectionEditSkinWeightsNode, SkeletalMesh))
	{
		VertexSkinWeights.Reset();
	}
}

void FDataflowCollectionEditSkinWeightsNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace UE::Dataflow;
	
	// Get the pin value if plugged
	FCollectionAttributeKey BoneIndicesKeyValue = GetBoneIndicesKey(Context);
	FCollectionAttributeKey BoneWeightsKeyValue = GetBoneWeightsKey(Context);

	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		// Evaluate in collection
		FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);

		if (!BoneIndicesKeyValue.Attribute.IsEmpty() && !BoneWeightsKeyValue.Attribute.IsEmpty() )
		{
			TArray<TArray<float>> SetupWeights, FinalWeights;
			TArray<TArray<int32>> SetupIndices, FinalIndices;
			
			if(GetAttributeWeights(InCollection, BoneIndicesKeyValue, BoneWeightsKeyValue, SetupIndices, SetupWeights, bCompressSkinWeights))
			{
				FinalIndices.SetNum(SetupIndices.Num());
				FinalWeights.SetNum(SetupWeights.Num());
				
				ExtractVertexWeights(Context, SetupIndices, SetupWeights, TArrayView<TArray<int32>>(FinalIndices.GetData(), FinalIndices.Num()),
				TArrayView<TArray<float>>(FinalWeights.GetData(), FinalWeights.Num()));

				SetAttributeWeights(InCollection, BoneIndicesKeyValue, BoneWeightsKeyValue, FinalIndices, FinalWeights);
			}
			else
			{
				Context.Error(LOCTEXT("EmptyAttributes", "Invalid attributes names, enter valid attribute name or connect valid attribute keys to inputs"), this, Out);
			}
		}
		else
		{
			if (BoneIndicesKeyValue.Attribute.IsEmpty())
			{
				Context.Error(LOCTEXT("EmptyBoneIndicesAttributes", "Bone indices attribute name is empty, enter a valid attribute name or connect valid attribute key to inputs"), this, Out);
			}

			if (BoneWeightsKeyValue.Attribute.IsEmpty())
			{
				Context.Error(LOCTEXT("EmptyBoneWeightsAttributes", "Bone weights attribute name is empty, enter a valid attribute name or connect valid attribute key to inputs"), this, Out);
			}
		}

		SetValue(Context, MoveTemp(InCollection), &Collection);
	}
	else if (Out->IsA<FCollectionAttributeKey>(&BoneIndicesKey))
	{
		SetValue(Context, MoveTemp(BoneIndicesKeyValue), &BoneIndicesKey);
	}
	else if (Out->IsA<FCollectionAttributeKey>(&BoneWeightsKey))
	{
		SetValue(Context, MoveTemp(BoneWeightsKeyValue), &BoneWeightsKey);
	}
}

int32 FDataflowCollectionEditSkinWeightsNode::ComputeInputRefSkeletonHashFromInput(UE::Dataflow::FContext& Context) const
{
	const TObjectPtr<USkeletalMesh> InSkeletalMesh = GetValue(Context, &SkeletalMesh, SkeletalMesh);
	if (InSkeletalMesh)
	{
		return UE::Dataflow::Private::ComputeHasFromRefSkeleton(InSkeletalMesh->GetRefSkeleton());
	}
	return 0;
}

void FDataflowCollectionEditSkinWeightsNode::ReportVertexWeights(UE::Dataflow::FContext& Context, const TArray<TArray<int32>>& SetupIndices, const TArray<TArray<float>>& SetupWeights, const TArray<TArray<int32>>& FinalIndices, const TArray<TArray<float>>& FinalWeights)
{
	check(SetupWeights.Num() == FinalWeights.Num());
	check(SetupWeights.Num() == SetupIndices.Num());
	check(FinalWeights.Num() == FinalIndices.Num());
	
	// update ref skeletal hash for the current stored skinweights
	SkinWeightsRefSkeletonHash = ComputeInputRefSkeletonHashFromInput(Context);

	// update the weights themselves
	VertexSkinWeights.SetNum(FinalWeights.Num());
	
	for (int32 VertexIndex = 0, NumVertices = FinalWeights.Num(); VertexIndex < NumVertices; ++VertexIndex)
	{
		VertexSkinWeights.FromArrays(VertexIndex, FinalIndices[VertexIndex], FinalWeights[VertexIndex]);
	}
}

void FDataflowCollectionEditSkinWeightsNode::ExtractVertexWeights(UE::Dataflow::FContext& Context, const TArray<TArray<int32>>& SetupIndices, const TArray<TArray<float>>& SetupWeights,
	TArrayView<TArray<int32>> FinalIndices, TArrayView<TArray<float>> FinalWeights) const
{
	check(SetupWeights.Num() == FinalWeights.Num());
	check(SetupWeights.Num() == SetupIndices.Num());
	check(FinalWeights.Num() == FinalIndices.Num());

	// we only returned the stored weight if they match the input refskeleton, other wise, returned the original weights
	const int32 RefSkeletonHashFromInput = ComputeInputRefSkeletonHashFromInput(Context);

	if(VertexSkinWeights.Num() == FinalWeights.Num() && RefSkeletonHashFromInput == SkinWeightsRefSkeletonHash)
	{
		for (int32 VertexIndex = 0, NumVertices = FinalWeights.Num(); VertexIndex < NumVertices; ++VertexIndex)
		{
			VertexSkinWeights.ToArrays(VertexIndex, FinalIndices[VertexIndex], FinalWeights[VertexIndex]);
			if (FinalIndices[VertexIndex].IsEmpty())
			{
				FinalWeights[VertexIndex] = SetupWeights[VertexIndex];
				FinalIndices[VertexIndex] = SetupIndices[VertexIndex];
			}
		}
	}
	else
	{
		for (int32 VertexIndex = 0, NumVertices = FinalWeights.Num(); VertexIndex < NumVertices; ++VertexIndex)
		{
			FinalWeights[VertexIndex] = SetupWeights[VertexIndex];
			FinalIndices[VertexIndex] = SetupIndices[VertexIndex];
		}
	}
}

bool FDataflowCollectionEditSkinWeightsNode::SetAttributeWeights(FManagedArrayCollection& SelectedCollection,
	const FCollectionAttributeKey& InBoneIndicesKey, const FCollectionAttributeKey& InBoneWeightsKey,
	const TArray<TArray<int32>>& AttributeIndices, const TArray<TArray<float>>& AttributeWeights)
{
	return UE::Dataflow::Private::SetAttributeValues<int32, FIntVector4, 4>(SelectedCollection, InBoneIndicesKey, AttributeIndices,  INDEX_NONE, false) && 
		   UE::Dataflow::Private::SetAttributeValues<float, FVector4f, 4>(SelectedCollection, InBoneWeightsKey, AttributeWeights, 0.0f, true);
}

bool FDataflowCollectionEditSkinWeightsNode::GetAttributeWeights(FManagedArrayCollection& SelectedCollection,
	const FCollectionAttributeKey& InBoneIndicesKey, const FCollectionAttributeKey& InBoneWeightsKey,
	TArray<TArray<int32>>& AttributeIndices, TArray<TArray<float>>& AttributeWeights, const bool bCanCompressSkinWeights)
{
	const bool bValidAttributes =
		UE::Dataflow::Private::GetAttributeValues<int32, FIntVector4, 4>(SelectedCollection, InBoneIndicesKey, AttributeIndices, bCanCompressSkinWeights) && 
		UE::Dataflow::Private::GetAttributeValues<float, FVector4f, 4>(SelectedCollection, InBoneWeightsKey, AttributeWeights, bCanCompressSkinWeights);

	UE::Dataflow::Private::CorrectSkinWeights(AttributeIndices, AttributeWeights);

	return bValidAttributes;
}

bool FDataflowCollectionEditSkinWeightsNode::FillAttributeWeights(const FManagedArrayCollection& SelectedCollection,
	const FCollectionAttributeKey& InBoneIndicesKey, const FCollectionAttributeKey& InBoneWeightsKey,
	TArray<TArray<int32>>& AttributeIndices, TArray<TArray<float>>& AttributeWeights)
{
	const bool bValidAttributes =
		UE::Dataflow::Private::FillAttributeValues<int32, FIntVector4, 4>(SelectedCollection, InBoneIndicesKey, AttributeIndices) &&
		UE::Dataflow::Private::FillAttributeValues<float, FVector4f, 4>(SelectedCollection, InBoneWeightsKey, AttributeWeights);

	UE::Dataflow::Private::CorrectSkinWeights(AttributeIndices, AttributeWeights);

	return bValidAttributes;
}

FCollectionAttributeKey FDataflowCollectionEditSkinWeightsNode::GetBoneIndicesKey(UE::Dataflow::FContext& Context) const
{
	// Get the pin value if plugged
	FCollectionAttributeKey Key = GetValue(Context, &BoneIndicesKey, BoneIndicesKey);

	// If nothing set used the local value
	if(Key.Attribute.IsEmpty() && Key.Group.IsEmpty())
	{
		Key.Group = VertexGroup.Name.ToString();
		Key.Attribute = BoneIndicesName;
	}
	return Key;
}

FCollectionAttributeKey FDataflowCollectionEditSkinWeightsNode::GetBoneWeightsKey(UE::Dataflow::FContext& Context) const
{
	// Get the pin value if plugged
	FCollectionAttributeKey Key = GetValue(Context, &BoneWeightsKey, BoneWeightsKey);

	// If nothing set used the local value
	if(Key.Attribute.IsEmpty() && Key.Group.IsEmpty())
	{
		Key.Group = VertexGroup.Name.ToString();
		Key.Attribute = BoneWeightsName;
	}
	return Key;
}

void FDataflowCollectionEditSkinWeightsNode::OnInvalidate()
{
	bValidSkeletalMeshes = false;
	DynamicMeshes.Reset();
	SkeletonObject = nullptr;
}

#if WITH_EDITOR

void FDataflowCollectionEditSkinWeightsNode::DebugDraw(UE::Dataflow::FContext& Context,
		IDataflowDebugDrawInterface& DataflowRenderingInterface,
		const FDebugDrawParameters& DebugDrawParameters) const
{
	SkeletonObject = nullptr;

	TObjectPtr<USkeletalMesh> InSkeletalMesh = GetValue(Context, &SkeletalMesh, SkeletalMesh);
	if(InSkeletalMesh)
	{
		SkeletonObject = MakeDebugDrawObject<FDataflowDebugDrawSkeletonObject>(
			DataflowRenderingInterface.ModifyDataflowElements(), InSkeletalMesh->GetRefSkeleton());

		DataflowRenderingInterface.DrawObject(TRefCountPtr<IDataflowDebugDrawObject>(SkeletonObject));

		SkeletonObject->OnBoneSelectionChanged.AddLambda([this](const TArray<FName>& BoneNames)
		{
			OnBoneSelectionChanged.Broadcast(BoneNames);
		});
	}
}

bool FDataflowCollectionEditSkinWeightsNode::CanDebugDrawViewMode(const FName& ViewModeName) const
{
	return true;
}

#endif

int32 FDataflowCollectionEditSkinWeightsNode::GetSkeletalMeshOffset(const TObjectPtr<USkeletalMesh>& InSkeletalMesh) const
{
#if WITH_EDITORONLY_DATA
	int32 SkeletalMeshOffset = 0;
	for(int32 SkeletalMeshIndex = 0, NumMeshes = SkeletalMeshes.Num(); SkeletalMeshIndex < NumMeshes; ++SkeletalMeshIndex)
	{
		if (SkeletalMeshes[SkeletalMeshIndex] == InSkeletalMesh)
		{
			return SkeletalMeshOffset;
		}
		else if (FMeshDescription* MeshDescription = SkeletalMeshes[SkeletalMeshIndex]->GetMeshDescription(0))
		{
			SkeletalMeshOffset += MeshDescription->Vertices().Num();
		}
	}
#endif
	return INDEX_NONE;
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// 
// Object encapsulating a change to the vertex skin weights values. Used for Undo/Redo.
class FDataflowCollectionEditSkinWeightsNode::FEditNodeToolChange final : public FToolCommandChange
{
public:
	FEditNodeToolChange(const FDataflowCollectionEditSkinWeightsNode& Node) :
		NodeGuid(Node.GetGuid()),
		SavedSkinWeightData(Node.VertexSkinWeights)
	{}

private:
	virtual FString ToString() const final
	{
		return TEXT("FDataflowCollectionEditSkinWeightsNode::FEditNodeToolChange");
	}

	virtual void Apply(UObject* Object) final
	{
		SwapApplyRevert(Object);
	}

	virtual void Revert(UObject* Object) final
	{
		SwapApplyRevert(Object);
	}

	void SwapApplyRevert(UObject* Object)
	{
		if (UDataflow* const Dataflow = Cast<UDataflow>(Object))
		{
			if (const TSharedPtr<FDataflowNode> BaseNode = Dataflow->GetDataflow()->FindBaseNode(NodeGuid))
			{
				if (FDataflowCollectionEditSkinWeightsNode* const Node = BaseNode->AsType<FDataflowCollectionEditSkinWeightsNode>())
				{
					Swap(SavedSkinWeightData, Node->VertexSkinWeights);
					Node->Invalidate();
				}
			}
		}
	}

	FGuid NodeGuid;
	FDataflowVertexSkinWeightData SavedSkinWeightData;
};

TUniquePtr<FToolCommandChange> FDataflowCollectionEditSkinWeightsNode::MakeEditNodeToolChange()
{
	return MakeUnique<FDataflowCollectionEditSkinWeightsNode::FEditNodeToolChange>(*this);
}

#undef LOCTEXT_NAMESPACE
