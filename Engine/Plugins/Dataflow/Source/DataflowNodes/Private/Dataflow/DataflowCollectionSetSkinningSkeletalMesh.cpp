// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowCollectionSetSkinningSkeletalMesh.h"
#include "Dataflow/DataflowObjectInterface.h"
#include "GeometryCollection/Facades/CollectionVertexBoneWeightsFacade.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowCollectionSetSkinningSkeletalMesh)

FDataflowCollectionSetSkinningSkeletalMesh::FDataflowCollectionSetSkinningSkeletalMesh(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&SkeletalMesh);
	RegisterInputConnection(&GeometrySelection);
	RegisterInputConnection(&LODIndex).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterOutputConnection(&Collection, &Collection);
}


void FDataflowCollectionSetSkinningSkeletalMesh::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Collection))
	{
		FManagedArrayCollection GeometryCollection = GetValue(Context, &Collection);
		GeometryCollection::Facades::FVertexBoneWeightsFacade SkinningFacade(GeometryCollection, false);
		SkinningFacade.DefineSchema();

		const TObjectPtr<USkeletalMesh> InputSkelMesh = GetValue(Context, &SkeletalMesh);
		const FDataflowGeometrySelection InputSelection = GetValue(Context, &GeometrySelection);
		const bool bValidSelection = InputSelection.IsValidForCollection(GeometryCollection);

		if (SkinningFacade.IsValid() && InputSkelMesh && InputSkelMesh->IsValidLODIndex(LODIndex))
		{
			for (int32 GeometryIndex = 0; GeometryIndex < SkinningFacade.NumGeometry(); ++GeometryIndex)
			{
				if (!bValidSelection || (bValidSelection && InputSelection.IsSelected(GeometryIndex)))
				{
					SkinningFacade.ModifyGeometryBinding(GeometryIndex, InputSkelMesh, LODIndex);
				}
			}
		}

		SetValue(Context, MoveTemp(GeometryCollection), &Collection);
	}
}


