// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "ChaosLog.h"
#include "Dataflow/DataflowConnectionTypes.h"
#include "Dataflow/DataflowCore.h"
#include "Dataflow/DataflowSelection.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "Engine/SkeletalMesh.h"

#include "DataflowCollectionSetSkinningSkeletalMesh.generated.h"

/** Set the skeletal mesh for the collection */
USTRUCT(meta = (Experimental, DataflowGroom))
struct FDataflowCollectionSetSkinningSkeletalMesh : public FDataflowNode
{
	GENERATED_BODY()

	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowCollectionSetSkinningSkeletalMesh, "SetSkinningSkeletalMesh", "Collection", "")

public:
	FDataflowCollectionSetSkinningSkeletalMesh(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface

	/** Managed array collection to be used to store datas */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DisplayName = "Collection", DataflowPassthrough = "Collection", DataflowRenderGroups = "Surface"))
	FManagedArrayCollection Collection;

	/** Skeletal mesh binding to be stored in the collection */
	UPROPERTY(EditAnywhere, Category = "Skeletal Mesh", meta = (DataflowInput, DataflowPassthrough = "SkeletalMesh"))
	TObjectPtr<USkeletalMesh> SkeletalMesh = nullptr;

	UPROPERTY(EditAnywhere, Category = "Skeletal Mesh", meta = (DataflowInput, DisplayName = "LOD Index"))
	int32 LODIndex = 0;

	/** Geometries to set skinning skeletal mesh on*/
	UPROPERTY(EditAnywhere, Category = "Skeletal Mesh", meta = (DataflowInput))
	FDataflowGeometrySelection GeometrySelection;
};

