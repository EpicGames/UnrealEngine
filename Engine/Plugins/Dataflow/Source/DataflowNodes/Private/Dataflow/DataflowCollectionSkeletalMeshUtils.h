// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Dataflow/CollectionRenderingPatternUtility.h"
#include "DynamicMeshToMeshDescription.h"
#include "MeshConversionOptions.h"
#include "SkeletalMeshAttributes.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "GeometryCollection/Facades/CollectionRenderingFacade.h"
#include "Rendering/SkeletalMeshModel.h"
#include "Engine/SkeletalMesh.h"
#include "Animation/Skeleton.h"
#include "Components/SkeletalMeshComponent.h"
#include "Materials/Material.h"
#include "MaterialDomain.h"

#if WITH_EDITOR
#include "StaticToSkeletalMeshConverter.h"
#endif

namespace UE::Dataflow::Private
{

static bool CreateSkeletalMesh(USkeletalMesh* SkeletalMesh, const TArray<UE::Geometry::FDynamicMesh3>& DynamicMeshes, const FReferenceSkeleton& ReferenceSkeleton)
{
#if WITH_EDITOR
	TArray<const FMeshDescription*> MeshDescriptions;

#if WITH_EDITORONLY_DATA
	SkeletalMesh->PreEditChange(nullptr);
	SkeletalMesh->GetImportedModel()->LODModels.Empty();
#endif
	SkeletalMesh->SetNumSourceModels(0);

	TArray<FMeshDescription> LocalDescriptions;
	for (const UE::Geometry::FDynamicMesh3& DynamicMesh : DynamicMeshes)
	{
		// Create a mesh description
		FMeshDescription& MeshDescription = LocalDescriptions.AddDefaulted_GetRef();
	
		// Add skeletal mesh attributes to the mesh description
		FSkeletalMeshAttributes Attributes(MeshDescription);
		Attributes.Register();

		// Convert dynamic mesh to the mesh description
		FConversionToMeshDescriptionOptions ConverterOptions;
		FDynamicMeshToMeshDescription Converter(ConverterOptions);
		Converter.Convert(&DynamicMesh, MeshDescription, false);

		// Add the created description to the list
		MeshDescriptions.Add(&MeshDescription);
	}

	TArray<FSkeletalMaterial> Materials;
	TConstArrayView<FSkeletalMaterial> MaterialView;

	// ensure there is at least one material
	if (MaterialView.IsEmpty())
	{
		Materials.Add( UMaterial::GetDefaultMaterial(MD_Surface));
		MaterialView = Materials;
	}

	FStaticToSkeletalMeshConverter::FInitializationParams Parameters;
	Parameters.Materials = MaterialView;
	Parameters.bRecomputeNormals = false;
	Parameters.bRecomputeTangents = false;
	Parameters.bCacheOptimize = false;
	
	if (!FStaticToSkeletalMeshConverter::InitializeSkeletalMeshFromMeshDescriptions(
		SkeletalMesh, MeshDescriptions, ReferenceSkeleton, Parameters))
	{
		return false;
	}
	return true;
#else
	return false;
#endif
}

static bool BuildSkeletalMeshes(TArray<TObjectPtr<USkeletalMesh>>& SkeletalMeshes, 
	const TSharedPtr<const FManagedArrayCollection>& RenderCollection, const FReferenceSkeleton& ReferenceSkeleton)
{
	if(RenderCollection.IsValid())
	{
		GeometryCollection::Facades::FRenderingFacade Facade(*RenderCollection);
		if(Facade.IsValid())
		{
			bool bValidSkeletalMeshes = true;
			const int32 NumGeometry = Facade.NumGeometry();

			if(NumGeometry == SkeletalMeshes.Num())
			{
				for (int32 MeshIndex = 0; MeshIndex < NumGeometry; ++MeshIndex)
				{
					UE::Geometry::FDynamicMesh3 DynamicMesh;
					UE::Dataflow::Conversion::RenderingFacadeToDynamicMesh(Facade, MeshIndex, DynamicMesh);

					if(!CreateSkeletalMesh(SkeletalMeshes[MeshIndex], {DynamicMesh}, ReferenceSkeleton))
					{
						bValidSkeletalMeshes = false;
					}
				}
			}
			return bValidSkeletalMeshes;
		}
	}
	return false;
}
	
static int32 ComputeHasFromRefSkeleton(const FReferenceSkeleton& ReferenceSkeleton)
{
	const TArray<FMeshBoneInfo>& BonesInfo = ReferenceSkeleton.GetRefBoneInfo();
	int32 Hash = 0;
	for (const FMeshBoneInfo& BoneInfo : BonesInfo)
	{
		Hash = HashCombine(Hash, 
					HashCombine(
						GetTypeHash(BoneInfo.Name), 
						GetTypeHash(BoneInfo.ParentIndex)
					));
	}
	return Hash;
}

}

