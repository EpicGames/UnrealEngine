// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowContextOverridesNodes.h"

#include "Dataflow/DataflowEngineUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowContextOverridesNodes)

namespace UE::Dataflow
{
	void RegisterContextOverridesNodes()
	{
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FFloatOverrideDataflowNode);
	}
}

void FFloatOverrideDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<float>(&ValueOut))
	{
		float Value = 0.f;
		if (const UE::Dataflow::FEngineContext* EngineContext = Context.AsType<UE::Dataflow::FEngineContext>())
		{
			FString Result = UE::Dataflow::Reflection::FindOverrideProperty< FString >(EngineContext->Owner, PropertyName, KeyName);
			Value = FCString::Atof(*Result);
		}
		SetValue(Context, Value, &ValueOut);
	}
}


