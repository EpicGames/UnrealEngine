// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Dataflow/DataflowCore.h"
#include "Dataflow/DataflowEngine.h"

#include "DataflowContextOverridesNodes.generated.h"

#define UE_API DATAFLOWNODES_API

USTRUCT(meta = (DataflowFlesh))
struct FFloatOverrideDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FFloatOverrideDataflowNode, "FloatOverride", "Dataflow", "")

public:
	UPROPERTY(EditAnywhere, Category = "Overrides")
	FName PropertyName = "Overrides";

	UPROPERTY(EditAnywhere, Category = "Overrides")
	FName KeyName = "FloatAttr";

	UPROPERTY(meta = (DataflowOutput, DisplayName = "Float"))
	float ValueOut = 0.f;

	FFloatOverrideDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterOutputConnection(&ValueOut);
	}

	UE_API virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};


namespace UE::Dataflow
{
	void RegisterContextOverridesNodes();
}

#undef UE_API
