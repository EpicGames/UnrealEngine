// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowGetCollectionBoundsNode.h"

#include "Dataflow/DataflowNodeFactory.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/Facades/CollectionBoundsFacade.h"
#include "GeometryCollection/Facades/CollectionTransformSelectionFacade.h"
#include "GeometryCollection/Facades/CollectionMeshFacade.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowGetCollectionBoundsNode)

#define LOCTEXT_NAMESPACE "DataflowGetCollectionBoundsNode"

namespace UE::Dataflow
{
	void RegisterGetCollectionBoundsNode()
	{
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowGetCollectionBoundsNode);
	}
};



FDataflowGetCollectionBoundsNode::FDataflowGetCollectionBoundsNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&Selection);
	RegisterOutputConnection(&Collection, &Collection);
	RegisterOutputConnection(&Bounds);
	RegisterOutputConnection(&Sphere);
}

void FDataflowGetCollectionBoundsNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Collection))
	{
		SafeForwardInput(Context, &Collection, &Collection);
		return;
	}
	else if (Out->IsA(&Bounds) || Out->IsA(&Sphere))
	{
		FManagedArrayCollection InCollection = GetValue(Context, &Collection, Collection);
		GeometryCollection::Facades::FCollectionTransformSelectionFacade TransformSelectionFacade(InCollection);

		const FDataflowInput* SelectionInput = FindInput(&Selection);
		if (SelectionInput)
		{
			const FName InGroupName = FDataflowSelection::GetSelectionTypeFromCollection(*SelectionInput);

			if (InGroupName != NAME_None && InCollection.HasGroup(InGroupName))
			{
				const FDataflowSelection& InSelection = GetValue(Context, &Selection);
				TArray<int32> SelectionArray;

				if (InSelection.AnySelected())
				{
					if (InGroupName == FDataflowVertexSelection::VerticesGroupName)
					{
						SelectionArray = InSelection.AsArray();
					}
					else if (InGroupName == FDataflowTransformSelection::TransformGroupName)
					{
						SelectionArray = TransformSelectionFacade.ConvertTransformSelectionToVertexSelection(InSelection.AsArray());
					}
					else if (InGroupName == FDataflowGeometrySelection::GeometryGroupName)
					{
						SelectionArray = TransformSelectionFacade.ConvertGeometrySelectionToVertexSelection(InSelection.AsArray());
					}
					else if (InGroupName == FDataflowFaceSelection::FacesGroupName)
					{
						SelectionArray = TransformSelectionFacade.ConvertFaceSelectionToVertexSelection(InSelection.AsArray());
					}
					else if (InGroupName == FDataflowCurveSelection::CurveGroupName)
					{
						SelectionArray = TransformSelectionFacade.ConvertCurveSelectionToVertexSelection(InSelection.AsArray());
					}
					else
					{
						SetError(Context, &Selection, LOCTEXT("CollectionSelectionConvertInvalidConversion", "Invalid conversion specified.").ToString());
						return;
					}

					TArray<FVector3f> VerticesInCollectionSpace;
					TArray<FVector> SelectedVerticesInCollectionSpace;
					SelectedVerticesInCollectionSpace.Reserve(SelectionArray.Num());

					GeometryCollection::Facades::FCollectionMeshFacade MeshFacade(InCollection);
					MeshFacade.GetVerticesInCollectionSpace(VerticesInCollectionSpace);

					// Output bounding box
					FBox OutBox;
					for (int32 Index = 0; Index < SelectionArray.Num(); ++Index)
					{
						if (VerticesInCollectionSpace.IsValidIndex(SelectionArray[Index]))
						{
							OutBox += FVector(VerticesInCollectionSpace[SelectionArray[Index]]);
							SelectedVerticesInCollectionSpace.Add(FVector(VerticesInCollectionSpace[SelectionArray[Index]]));
						}
					}

					SetValue(Context, OutBox, &Bounds);

					// Output bounding sphere
					if (SelectedVerticesInCollectionSpace.Num() > 0)
					{
						FSphere OutSphere(&SelectedVerticesInCollectionSpace[0], SelectedVerticesInCollectionSpace.Num());

						SetValue(Context, OutSphere, &Sphere);
					}
					else
					{
						SetValue(Context, FSphere(), &Sphere);
					}

					return;
				}
			}
		}

		const FName InVertexGroupName = VertexGroup.Name;
		if (InCollection.HasGroup(InVertexGroupName))
		{
			GeometryCollection::Facades::FBoundsFacade BoundsFacade(InCollection);
			if (BoundsFacade.IsValid())
			{
				const FBox OutBox = BoundsFacade.GetBoundingBoxInCollectionSpace();
				SetValue(Context, OutBox, &Bounds);

				const FSphere OutSphere = BoundsFacade.GetBoundingSphereInCollectionSpace();
				SetValue(Context, OutSphere, &Sphere);

				return;
			}

			// handling of cloth collection 
			// cannot really include cloth here so we need to directly serach for the vertex position attribute in the render vertices ( eventually we may converge to use only geometry collections )
			const TManagedArray<FVector3f>* VertexAttribute = InCollection.FindAttribute<FVector3f>("RenderPosition", InVertexGroupName);
			if (!VertexAttribute)
			{
				// cannot really include cloth here so we need to directly serach for the vertex position attribute in the render vertices ( eventually we may converge to use only geometry collections )
				VertexAttribute = InCollection.FindAttribute<FVector3f>("SimPosition3D", InVertexGroupName);
			}
			if (VertexAttribute && VertexAttribute->Num() > 0)
			{
				TArray<FVector> Vertices;
				Vertices.Reserve(VertexAttribute->Num());

				FBox OutBox;
				for (int32 Index = 0; Index < VertexAttribute->Num(); ++Index)
				{
					OutBox += FVector((*VertexAttribute)[Index]);

					Vertices.Add(FVector((*VertexAttribute)[Index]));
				}
				SetValue(Context, OutBox, &Bounds);

				if (Vertices.Num() > 0)
				{
					FSphere OutSphere(&Vertices[0], Vertices.Num());
					SetValue(Context, OutSphere, &Sphere);
				}
				else
				{
					SetValue(Context, FSphere(), &Sphere);
				}

				return;
			}
		
			Context.Warning(LOCTEXT("NoVertexPosition_Msg", "No valid vertex position attribute could be found in this collection, bounds will be empty"), this, Out);
		}
		else
		{
			Context.Error(LOCTEXT("InvalidVertexGroup_Msg", "Vertex group does not exists, bounds will be empty"), this, Out);
		}

		const FBox OutBox(EForceInit::ForceInitToZero);
		SetValue(Context, OutBox, &Bounds);

		const FSphere OutSphere(EForceInit::ForceInitToZero);
		SetValue(Context, OutSphere, &Sphere);
	}
}

#undef LOCTEXT_NAMESPACE 

