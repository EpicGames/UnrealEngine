// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowNode.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "Dataflow/DataflowCollectionAddScalarVertexPropertyNode.h"

#include "DataflowGetCollectionBoundsNode.generated.h"

/**
* Get the bounds of the vertices of a specific collection
*/
USTRUCT()
struct FDataflowGetCollectionBoundsNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowGetCollectionBoundsNode, "CollectionBounds", "Collection", "BoundingBox Size Limits")

public:
	FDataflowGetCollectionBoundsNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	/** Collection to compute the bounds from */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = Collection))
	FManagedArrayCollection Collection;

	/** Selection input. To set the selection type rt mouse click on the Selection output */
	UPROPERTY(meta = (DataflowInput))
	FDataflowSelectionTypes Selection;

	/** resulting bounding box of the collection */
	UPROPERTY(EditAnywhere, Category = Target, meta = (DisplayName = "Box", DataflowOutput))
	FBox Bounds = FBox(EForceInit::ForceInit);

	/** resulting bounding sphere of the collection */
	UPROPERTY(EditAnywhere, Category = Target, meta = (DataflowOutput))
	FSphere Sphere = FSphere(EForceInit::ForceInit);

	/** Vertex group to use */
	UPROPERTY(EditAnywhere, Category = Target)
	FScalarVertexPropertyGroup VertexGroup;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

namespace UE::Dataflow
{
	void RegisterGetCollectionBoundsNode();
}

