// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowMeshMakeNodes.h"
#include "Dataflow/DataflowCore.h"

#include "Engine/Engine.h"
#include "Engine/StaticMesh.h"
#include "Logging/LogMacros.h"
#include "Templates/SharedPointer.h"
#include "UObject/UnrealTypePrivate.h"
#include "DynamicMeshToMeshDescription.h"
#include "MeshDescriptionToDynamicMesh.h"
#include "StaticMeshAttributes.h"
#include "DynamicMeshEditor.h"
#include "Operations/MeshBoolean.h"
#include "Materials/Material.h"

#include "EngineGlobals.h"
#include "DynamicMesh/MeshTransforms.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Dataflow/DataflowDebugDraw.h"
#include "Dataflow/DataflowSimpleDebugDrawMesh.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Generators/SphereGenerator.h"
#include "Generators/BoxSphereGenerator.h"
#include "Generators/CapsuleGenerator.h"
#include "Generators/SweepGenerator.h"
#include "Generators/GridBoxMeshGenerator.h"
#include "Generators/DiscMeshGenerator.h"
#include "Generators/StairGenerator.h"
#include "Generators/RectangleMeshGenerator.h"
#include "Algo/Reverse.h"
#include "Dataflow/DataflowUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowMeshMakeNodes)

namespace UE::Dataflow
{
	void DataflowMeshMakeNodes()
	{
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMakeSphereMeshDataflowNode_v2);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMakeCapsuleMeshDataflowNode_v2);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMakeCylinderMeshDataflowNode_v2);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMakeBoxMeshDataflowNode_v2);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMakeDiscMeshDataflowNode_v2);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMakeStairMeshDataflowNode_v2);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMakeRectangleMeshDataflowNode_v2);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMakeTorusMeshDataflowNode_v2);

		// Deprecated
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMakeSphereMeshDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMakeCapsuleMeshDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMakeCylinderMeshDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMakeBoxMeshDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMakeDiscMeshDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMakeStairMeshDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMakeRectangleMeshDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMakeTorusMeshDataflowNode);
	}
}

/* -------------------------------------------------------------------------------- */

FMakeSphereMeshDataflowNode::FMakeSphereMeshDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterOutputConnection(&Mesh);
}

void FMakeSphereMeshDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace UE::Geometry;

	if (Out->IsA(&Mesh))
	{
		TObjectPtr<UDynamicMesh> NewMesh = NewObject<UDynamicMesh>();
		NewMesh->Reset();

		FDynamicMesh3& DynMesh = NewMesh->GetMeshRef();

		FSphereGenerator SphereGenerator;
		SphereGenerator.Radius = FMath::Max(FMathf::ZeroTolerance, Radius);
		SphereGenerator.NumPhi = FMath::Max(3, NumPhi);
		SphereGenerator.NumTheta = FMath::Max(3, NumTheta);
		SphereGenerator.bPolygroupPerQuad = false;
		SphereGenerator.Generate();

		DynMesh.Copy(&SphereGenerator);
		SetValue(Context, NewMesh, &Mesh);
	}
}

FMakeSphereMeshDataflowNode_v2::FMakeSphereMeshDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterOutputConnection(&Mesh);
}

void FMakeSphereMeshDataflowNode_v2::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace UE::Geometry;

	if (Out->IsA(&Mesh))
	{
		TObjectPtr<UDataflowMesh> NewMesh = NewObject<UDataflowMesh>();

		FSphereGenerator SphereGenerator;
		SphereGenerator.Radius = FMath::Max(FMathf::ZeroTolerance, Radius);
		SphereGenerator.NumPhi = FMath::Max(3, NumPhi);
		SphereGenerator.NumTheta = FMath::Max(3, NumTheta);
		SphereGenerator.bPolygroupPerQuad = false;
		SphereGenerator.Generate();

		FDynamicMesh3 DynMesh;
		DynMesh.Copy(&SphereGenerator);

		NewMesh->SetDynamicMesh(DynMesh);

		SetValue(Context, NewMesh, &Mesh);
	}
}

/* -------------------------------------------------------------------------------- */

FMakeCapsuleMeshDataflowNode::FMakeCapsuleMeshDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterOutputConnection(&Mesh);
}

void FMakeCapsuleMeshDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace UE::Geometry;

	if (Out->IsA(&Mesh))
	{
		TObjectPtr<UDynamicMesh> NewMesh = NewObject<UDynamicMesh>();
		NewMesh->Reset();

		FDynamicMesh3& DynMesh = NewMesh->GetMeshRef();

		FCapsuleGenerator CapsuleGenerator;
		CapsuleGenerator.Radius = FMath::Max(FMathf::ZeroTolerance, Radius);
		CapsuleGenerator.SegmentLength = FMath::Max(FMathf::ZeroTolerance, SegmentLength);
		CapsuleGenerator.NumHemisphereArcSteps = FMath::Max(5, NumHemisphereArcSteps);
		CapsuleGenerator.NumCircleSteps = FMath::Max(3, NumCircleSteps);
		CapsuleGenerator.NumSegmentSteps = FMath::Max(0, NumSegmentSteps);
		CapsuleGenerator.bPolygroupPerQuad = false;
		CapsuleGenerator.Generate();

		DynMesh.Copy(&CapsuleGenerator);
		SetValue(Context, NewMesh, &Mesh);
	}
}

FMakeCapsuleMeshDataflowNode_v2::FMakeCapsuleMeshDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterOutputConnection(&Mesh);
}

void FMakeCapsuleMeshDataflowNode_v2::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace UE::Geometry;

	if (Out->IsA(&Mesh))
	{
		TObjectPtr<UDataflowMesh> NewMesh = NewObject<UDataflowMesh>();

		FCapsuleGenerator CapsuleGenerator;
		CapsuleGenerator.Radius = FMath::Max(FMathf::ZeroTolerance, Radius);
		CapsuleGenerator.SegmentLength = FMath::Max(FMathf::ZeroTolerance, SegmentLength);
		CapsuleGenerator.NumHemisphereArcSteps = FMath::Max(5, NumHemisphereArcSteps);
		CapsuleGenerator.NumCircleSteps = FMath::Max(3, NumCircleSteps);
		CapsuleGenerator.NumSegmentSteps = FMath::Max(0, NumSegmentSteps);
		CapsuleGenerator.bPolygroupPerQuad = false;
		CapsuleGenerator.Generate();

		FDynamicMesh3 DynMesh;
		DynMesh.Copy(&CapsuleGenerator);

		NewMesh->SetDynamicMesh(DynMesh);

		SetValue(Context, NewMesh, &Mesh);
	}
}

/* -------------------------------------------------------------------------------- */

FMakeCylinderMeshDataflowNode::FMakeCylinderMeshDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterOutputConnection(&Mesh);
}

void FMakeCylinderMeshDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace UE::Geometry;

	if (Out->IsA(&Mesh))
	{
		TObjectPtr<UDynamicMesh> NewMesh = NewObject<UDynamicMesh>();
		NewMesh->Reset();

		FDynamicMesh3& DynMesh = NewMesh->GetMeshRef();

		FCylinderGenerator CylinderGenerator;
		CylinderGenerator.Radius[0] = FMath::Max(FMathf::ZeroTolerance, Radius1);
		CylinderGenerator.Radius[1] = FMath::Max(FMathf::ZeroTolerance, Radius2);
		CylinderGenerator.Height = FMath::Max(FMathf::ZeroTolerance, Height);
		CylinderGenerator.LengthSamples = LengthSamples;
		CylinderGenerator.AngleSamples = AngleSamples;
		CylinderGenerator.bCapped = true;
		CylinderGenerator.bPolygroupPerQuad = false;
		CylinderGenerator.Generate();

		DynMesh.Copy(&CylinderGenerator);
		SetValue(Context, NewMesh, &Mesh);
	}
}

FMakeCylinderMeshDataflowNode_v2::FMakeCylinderMeshDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterOutputConnection(&Mesh);
}

void FMakeCylinderMeshDataflowNode_v2::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace UE::Geometry;

	if (Out->IsA(&Mesh))
	{
		TObjectPtr<UDataflowMesh> NewMesh = NewObject<UDataflowMesh>();

		FCylinderGenerator CylinderGenerator;
		CylinderGenerator.Radius[0] = FMath::Max(FMathf::ZeroTolerance, Radius1);
		CylinderGenerator.Radius[1] = FMath::Max(FMathf::ZeroTolerance, Radius2);
		CylinderGenerator.Height = FMath::Max(FMathf::ZeroTolerance, Height);
		CylinderGenerator.LengthSamples = LengthSamples;
		CylinderGenerator.AngleSamples = AngleSamples;
		CylinderGenerator.bCapped = true;
		CylinderGenerator.bPolygroupPerQuad = false;
		CylinderGenerator.Generate();

		FDynamicMesh3 DynMesh;
		DynMesh.Copy(&CylinderGenerator);

		NewMesh->SetDynamicMesh(DynMesh);

		SetValue(Context, NewMesh, &Mesh);
	}
}

/* -------------------------------------------------------------------------------- */

FMakeBoxMeshDataflowNode::FMakeBoxMeshDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterOutputConnection(&Mesh);
}

void FMakeBoxMeshDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace UE::Geometry;

	if (Out->IsA(&Mesh))
	{
		TObjectPtr<UDynamicMesh> NewMesh = NewObject<UDynamicMesh>();
		NewMesh->Reset();

		FDynamicMesh3& DynMesh = NewMesh->GetMeshRef();

		FGridBoxMeshGenerator GridBoxMeshGenerator;
		GridBoxMeshGenerator.Box = UE::Geometry::FOrientedBox3d(Center, 0.5 * Size);
		GridBoxMeshGenerator.EdgeVertices = FIndex3i(SubdivisionsX + 1, SubdivisionsY + 1, SubdivisionsZ + 1);
		GridBoxMeshGenerator.bPolygroupPerQuad = false;
		GridBoxMeshGenerator.Generate();

		DynMesh.Copy(&GridBoxMeshGenerator);
		SetValue(Context, NewMesh, &Mesh);
	}
}

FMakeBoxMeshDataflowNode_v2::FMakeBoxMeshDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterOutputConnection(&Mesh);
}

void FMakeBoxMeshDataflowNode_v2::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace UE::Geometry;

	if (Out->IsA(&Mesh))
	{
		TObjectPtr<UDataflowMesh> NewMesh = NewObject<UDataflowMesh>();

		FGridBoxMeshGenerator GridBoxMeshGenerator;
		GridBoxMeshGenerator.Box = UE::Geometry::FOrientedBox3d(Center, 0.5 * Size);
		GridBoxMeshGenerator.EdgeVertices = FIndex3i(SubdivisionsX + 1, SubdivisionsY + 1, SubdivisionsZ + 1);
		GridBoxMeshGenerator.bPolygroupPerQuad = false;
		GridBoxMeshGenerator.Generate();

		FDynamicMesh3 DynMesh;
		DynMesh.Copy(&GridBoxMeshGenerator);

		NewMesh->SetDynamicMesh(DynMesh);

		SetValue(Context, NewMesh, &Mesh);
	}
}

/* -------------------------------------------------------------------------------- */

FMakeDiscMeshDataflowNode::FMakeDiscMeshDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterOutputConnection(&Mesh);
}

void FMakeDiscMeshDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace UE::Geometry;

	if (Out->IsA(&Mesh))
	{
		TObjectPtr<UDynamicMesh> NewMesh = NewObject<UDynamicMesh>();
		NewMesh->Reset();

		FDynamicMesh3& DynMesh = NewMesh->GetMeshRef();

		FDiscMeshGenerator DiscGenerator;
		DiscGenerator.Radius = FMath::Max(FMathf::ZeroTolerance, Radius);
		DiscGenerator.Normal = FVector3f(Normal);
		DiscGenerator.AngleSamples = AngleSamples;
		DiscGenerator.RadialSamples = RadialSamples;
		DiscGenerator.StartAngle = StartAngle;
		DiscGenerator.EndAngle = EndAngle;
		DiscGenerator.bSinglePolygroup = true;
		DiscGenerator.Generate();

		DynMesh.Copy(&DiscGenerator);
		SetValue(Context, NewMesh, &Mesh);
	}
}

FMakeDiscMeshDataflowNode_v2::FMakeDiscMeshDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterOutputConnection(&Mesh);
}

void FMakeDiscMeshDataflowNode_v2::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace UE::Geometry;

	if (Out->IsA(&Mesh))
	{
		TObjectPtr<UDataflowMesh> NewMesh = NewObject<UDataflowMesh>();

		FDiscMeshGenerator DiscGenerator;
		DiscGenerator.Radius = FMath::Max(FMathf::ZeroTolerance, Radius);
		DiscGenerator.Normal = FVector3f(Normal);
		DiscGenerator.AngleSamples = AngleSamples;
		DiscGenerator.RadialSamples = RadialSamples;
		DiscGenerator.StartAngle = StartAngle;
		DiscGenerator.EndAngle = EndAngle;
		DiscGenerator.bSinglePolygroup = true;
		DiscGenerator.Generate();

		FDynamicMesh3 DynMesh;
		DynMesh.Copy(&DiscGenerator);

		NewMesh->SetDynamicMesh(DynMesh);

		SetValue(Context, NewMesh, &Mesh);
	}
}

/* -------------------------------------------------------------------------------- */

FMakeStairMeshDataflowNode::FMakeStairMeshDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterOutputConnection(&Mesh);
}

void FMakeStairMeshDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace UE::Geometry;

	if (Out->IsA(&Mesh))
	{
		TObjectPtr<UDynamicMesh> NewMesh = NewObject<UDynamicMesh>();
		NewMesh->Reset();

		FDynamicMesh3& DynMesh = NewMesh->GetMeshRef();

		if (StairType == EDataflowStairTypeEnum::Linear)
		{
			FLinearStairGenerator StairGenerator;
			StairGenerator.NumSteps = NumSteps;
			StairGenerator.StepWidth = StepWidth;
			StairGenerator.StepHeight = StepHeight;
			StairGenerator.StepDepth = StepDepth;
			StairGenerator.bScaleUVByAspectRatio = true;
			StairGenerator.bPolygroupPerQuad = true;
			StairGenerator.Generate();

			DynMesh.Copy(&StairGenerator);
		}
		else if (StairType == EDataflowStairTypeEnum::Floating)
		{
			FFloatingStairGenerator StairGenerator;
			StairGenerator.NumSteps = NumSteps;
			StairGenerator.StepWidth = StepWidth;
			StairGenerator.StepHeight = StepHeight;
			StairGenerator.StepDepth = StepDepth;
			StairGenerator.bScaleUVByAspectRatio = true;
			StairGenerator.bPolygroupPerQuad = true;
			StairGenerator.Generate();

			DynMesh.Copy(&StairGenerator);
		}
		else if (StairType == EDataflowStairTypeEnum::Curved)
		{
			FCurvedStairGenerator StairGenerator;
			StairGenerator.NumSteps = NumSteps;
			StairGenerator.StepWidth = StepWidth;
			StairGenerator.StepHeight = StepHeight;
			StairGenerator.CurveAngle = CurveAngle;
			StairGenerator.InnerRadius = InnerRadius;
			StairGenerator.bScaleUVByAspectRatio = true;
			StairGenerator.bPolygroupPerQuad = true;
			StairGenerator.Generate();

			DynMesh.Copy(&StairGenerator);
		}
		else if (StairType == EDataflowStairTypeEnum::Spiral)
		{
			FSpiralStairGenerator StairGenerator;
			StairGenerator.NumSteps = NumSteps;
			StairGenerator.StepWidth = StepWidth;
			StairGenerator.StepHeight = StepHeight;
			StairGenerator.CurveAngle = CurveAngle;
			StairGenerator.InnerRadius = InnerRadius;
			StairGenerator.bScaleUVByAspectRatio = true;
			StairGenerator.bPolygroupPerQuad = true;
			StairGenerator.Generate();

			DynMesh.Copy(&StairGenerator);
		}

		SetValue(Context, NewMesh, &Mesh);
	}
}

FMakeStairMeshDataflowNode_v2::FMakeStairMeshDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterOutputConnection(&Mesh);
}

void FMakeStairMeshDataflowNode_v2::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace UE::Geometry;

	if (Out->IsA(&Mesh))
	{
		TObjectPtr<UDataflowMesh> NewMesh = NewObject<UDataflowMesh>();

		FDynamicMesh3 DynMesh;
		if (StairType == EDataflowStairTypeEnum::Linear)
		{
			FLinearStairGenerator StairGenerator;
			StairGenerator.NumSteps = NumSteps;
			StairGenerator.StepWidth = StepWidth;
			StairGenerator.StepHeight = StepHeight;
			StairGenerator.StepDepth = StepDepth;
			StairGenerator.bScaleUVByAspectRatio = true;
			StairGenerator.bPolygroupPerQuad = true;
			StairGenerator.Generate();

			DynMesh.Copy(&StairGenerator);
		}
		else if (StairType == EDataflowStairTypeEnum::Floating)
		{
			FFloatingStairGenerator StairGenerator;
			StairGenerator.NumSteps = NumSteps;
			StairGenerator.StepWidth = StepWidth;
			StairGenerator.StepHeight = StepHeight;
			StairGenerator.StepDepth = StepDepth;
			StairGenerator.bScaleUVByAspectRatio = true;
			StairGenerator.bPolygroupPerQuad = true;
			StairGenerator.Generate();

			DynMesh.Copy(&StairGenerator);
		}
		else if (StairType == EDataflowStairTypeEnum::Curved)
		{
			FCurvedStairGenerator StairGenerator;
			StairGenerator.NumSteps = NumSteps;
			StairGenerator.StepWidth = StepWidth;
			StairGenerator.StepHeight = StepHeight;
			StairGenerator.CurveAngle = CurveAngle;
			StairGenerator.InnerRadius = InnerRadius;
			StairGenerator.bScaleUVByAspectRatio = true;
			StairGenerator.bPolygroupPerQuad = true;
			StairGenerator.Generate();

			DynMesh.Copy(&StairGenerator);
		}
		else if (StairType == EDataflowStairTypeEnum::Spiral)
		{
			FSpiralStairGenerator StairGenerator;
			StairGenerator.NumSteps = NumSteps;
			StairGenerator.StepWidth = StepWidth;
			StairGenerator.StepHeight = StepHeight;
			StairGenerator.CurveAngle = CurveAngle;
			StairGenerator.InnerRadius = InnerRadius;
			StairGenerator.bScaleUVByAspectRatio = true;
			StairGenerator.bPolygroupPerQuad = true;
			StairGenerator.Generate();

			DynMesh.Copy(&StairGenerator);
		}

		NewMesh->SetDynamicMesh(DynMesh);

		SetValue(Context, NewMesh, &Mesh);
	}
}

/* -------------------------------------------------------------------------------- */

FMakeRectangleMeshDataflowNode::FMakeRectangleMeshDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Origin).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&Normal).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterOutputConnection(&Mesh);
}

void FMakeRectangleMeshDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace UE::Geometry;

	if (Out->IsA(&Mesh))
	{
		const FVector InOrigin = GetValue(Context, &Origin);
		const FVector InNormal = GetValue(Context, &Normal);

		TObjectPtr<UDynamicMesh> NewMesh = NewObject<UDynamicMesh>();
		NewMesh->Reset();

		FDynamicMesh3& DynMesh = NewMesh->GetMeshRef();

		FRectangleMeshGenerator RectangleGenerator;
		RectangleGenerator.Origin = InOrigin;
		RectangleGenerator.Normal = FVector3f(InNormal);
		RectangleGenerator.Width = Width;
		RectangleGenerator.Height = Height;
		RectangleGenerator.WidthVertexCount = WidthVertexCount;
		RectangleGenerator.HeightVertexCount = HeightVertexCount;
		RectangleGenerator.bSinglePolyGroup = true;
		RectangleGenerator.Generate();

		DynMesh.Copy(&RectangleGenerator);
		SetValue(Context, NewMesh, &Mesh);
	}
}

FMakeRectangleMeshDataflowNode_v2::FMakeRectangleMeshDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Origin).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&Normal).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterOutputConnection(&Mesh);
}

void FMakeRectangleMeshDataflowNode_v2::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace UE::Geometry;

	if (Out->IsA(&Mesh))
	{
		const FVector InOrigin = GetValue(Context, &Origin);
		const FVector InNormal = GetValue(Context, &Normal);

		TObjectPtr<UDataflowMesh> NewMesh = NewObject<UDataflowMesh>();

		FRectangleMeshGenerator RectangleGenerator;
		RectangleGenerator.Origin = InOrigin;
		RectangleGenerator.Normal = FVector3f(InNormal);
		RectangleGenerator.Width = Width;
		RectangleGenerator.Height = Height;
		RectangleGenerator.WidthVertexCount = WidthVertexCount;
		RectangleGenerator.HeightVertexCount = HeightVertexCount;
		RectangleGenerator.bSinglePolyGroup = true;
		RectangleGenerator.Generate();

		FDynamicMesh3 DynMesh;
		DynMesh.Copy(&RectangleGenerator);

		NewMesh->SetDynamicMesh(DynMesh);

		SetValue(Context, NewMesh, &Mesh);
	}
}

/* -------------------------------------------------------------------------------- */

FMakeTorusMeshDataflowNode::FMakeTorusMeshDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Origin).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterOutputConnection(&Mesh);
}

void FMakeTorusMeshDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace UE::Geometry;

	if (Out->IsA(&Mesh))
	{
		const FVector InOrigin = GetValue(Context, &Origin);

		TObjectPtr<UDynamicMesh> NewMesh = NewObject<UDynamicMesh>();
		NewMesh->Reset();

		FDynamicMesh3& DynMesh = NewMesh->GetMeshRef();

		TArray<FVector3d> ProfileCurve; ProfileCurve.SetNumUninitialized(ProfileVertexCount);
		TArray<FFrame3d> SweepCurve; SweepCurve.SetNumUninitialized(SweepVertexCount);

		FVector Vec1(0.0, -Radius1, 0.0);
		const float RotateAngle1 = 360.f / float(ProfileVertexCount);

		// Create profile curve
		for (int32 Idx = 0; Idx < ProfileVertexCount; ++Idx)
		{
			ProfileCurve[Idx] = Vec1;

			Vec1 = Vec1.RotateAngleAxis(RotateAngle1, FVector::XAxisVector);
		}

		FVector Vec2(0.0, -Radius2, 0.0);
		const float RotateAngle2 = 360.f / float(SweepVertexCount);
		TQuaternion<double> Quat(FVector::ZAxisVector, RotateAngle2, true);

		FFrame3d Frame; // Construct a frame positioned at(0, 0, 0) aligned to the unit axes

		// Create sweep curve
		for (int32 Idx = 0; Idx < SweepVertexCount; ++Idx)
		{
			FFrame3d Frame1 = Frame;
			Frame1.Origin = Vec2 + InOrigin;

			SweepCurve[Idx] = Frame1;

			Frame.Rotate(Quat);
			Vec2 = Vec2.RotateAngleAxis(RotateAngle2, FVector::ZAxisVector);
		}

		FProfileSweepGenerator SweepGenerator;
		SweepGenerator.ProfileCurve = ProfileCurve;
		SweepGenerator.SweepCurve = SweepCurve;
		SweepGenerator.bSweepCurveIsClosed = true;
		SweepGenerator.bProfileCurveIsClosed = true;
		SweepGenerator.Generate();

		DynMesh.Copy(&SweepGenerator);
		SetValue(Context, NewMesh, &Mesh);
	}
}

FMakeTorusMeshDataflowNode_v2::FMakeTorusMeshDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Origin).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterOutputConnection(&Mesh);
}

void FMakeTorusMeshDataflowNode_v2::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace UE::Geometry;

	if (Out->IsA(&Mesh))
	{
		const FVector InOrigin = GetValue(Context, &Origin);

		TObjectPtr<UDataflowMesh> NewMesh = NewObject<UDataflowMesh>();

		TArray<FVector3d> ProfileCurve; ProfileCurve.SetNumUninitialized(ProfileVertexCount);
		TArray<FFrame3d> SweepCurve; SweepCurve.SetNumUninitialized(SweepVertexCount);

		FVector Vec1(0.0, -Radius1, 0.0);
		const float RotateAngle1 = 360.f / float(ProfileVertexCount);

		// Create profile curve
		for (int32 Idx = 0; Idx < ProfileVertexCount; ++Idx)
		{
			ProfileCurve[Idx] = Vec1;

			Vec1 = Vec1.RotateAngleAxis(RotateAngle1, FVector::XAxisVector);
		}

		FVector Vec2(0.0, -Radius2, 0.0);
		const float RotateAngle2 = 360.f / float(SweepVertexCount);
		TQuaternion<double> Quat(FVector::ZAxisVector, RotateAngle2, true);

		FFrame3d Frame; // Construct a frame positioned at(0, 0, 0) aligned to the unit axes

		// Create sweep curve
		for (int32 Idx = 0; Idx < SweepVertexCount; ++Idx)
		{
			FFrame3d Frame1 = Frame;
			Frame1.Origin = Vec2 + InOrigin;

			SweepCurve[Idx] = Frame1;

			Frame.Rotate(Quat);
			Vec2 = Vec2.RotateAngleAxis(RotateAngle2, FVector::ZAxisVector);
		}

		FProfileSweepGenerator SweepGenerator;
		SweepGenerator.ProfileCurve = ProfileCurve;
		SweepGenerator.SweepCurve = SweepCurve;
		SweepGenerator.bSweepCurveIsClosed = true;
		SweepGenerator.bProfileCurveIsClosed = true;
		SweepGenerator.Generate();

		FDynamicMesh3 DynMesh;
		DynMesh.Copy(&SweepGenerator);

		NewMesh->SetDynamicMesh(DynMesh);

		SetValue(Context, NewMesh, &Mesh);
	}
}

/* -------------------------------------------------------------------------------- */

