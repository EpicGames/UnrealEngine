// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Dataflow/DataflowEngine.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "UDynamicMesh.h"
#include "Dataflow/DataflowSelection.h"
#include "Math/MathFwd.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Dataflow/DataflowDebugDraw.h"
#include "Dataflow/DataflowPlane.h"
#include "Dataflow/DataflowPoints.h"
#include "Dataflow/DataflowPrimitiveTypes.h"
#include "Dataflow/SamplerNodes/DataflowSamplerTypes.h"
#include "Dataflow/DataflowMesh.h"

#include "DataflowMeshMakeNodes.generated.h"

UENUM(BlueprintType)
enum class EMakeMeshTypeEnum : uint8
{
	Sphere UMETA(DisplayName = "Sphere"),
	Capsule UMETA(DisplayName = "Capsule"),
	Cylinder UMETA(DisplayName = "Cylinder"),
};

/**
 * Make a sphere mesh
 * DEPRECATED 5.8 - use FMakeSphereMeshDataflowNode_v2 instead
 */
USTRUCT(meta = (Deprecated = "5.8"))
struct FMakeSphereMeshDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMakeSphereMeshDataflowNode, "MakeSphereMesh", "Generators|Mesh", "")

private:
	/** Sphere Radius */
	UPROPERTY(EditAnywhere, Category = "Sphere", meta = (UIMin = "0.1", ClampMin = "0.1"));
	float Radius = 1.f;

	/** Sphere numphi */
	UPROPERTY(EditAnywhere, Category = "Sphere", meta = (DisplayName = "Steps Phi", UIMin = "3", ClampMin = "3"));
	int32 NumPhi = 12;

	/** Sphere numtheta */
	UPROPERTY(EditAnywhere, Category = "Sphere", meta = (DisplayName = "Steps Theta", UIMin = "3", ClampMin = "3"));
	int32 NumTheta = 12;

	/** Output mesh */
	UPROPERTY(meta = (DataflowOutput))
	TObjectPtr<UDynamicMesh> Mesh;

public:
	FMakeSphereMeshDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 * Make a sphere mesh
 */
USTRUCT()
struct FMakeSphereMeshDataflowNode_v2 : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMakeSphereMeshDataflowNode_v2, "MakeSphereMesh", "Generators|Mesh", "")

private:
	/** Sphere Radius */
	UPROPERTY(EditAnywhere, Category = "Sphere", meta = (UIMin = "0.1", ClampMin = "0.1"));
	float Radius = 50.f;

	/** Sphere numphi */
	UPROPERTY(EditAnywhere, Category = "Sphere", meta = (DisplayName = "Steps Phi", UIMin = "3", ClampMin = "3"));
	int32 NumPhi = 12;

	/** Sphere numtheta */
	UPROPERTY(EditAnywhere, Category = "Sphere", meta = (DisplayName = "Steps Theta", UIMin = "3", ClampMin = "3"));
	int32 NumTheta = 12;

	/** Output mesh */
	UPROPERTY(meta = (DataflowOutput))
	TObjectPtr<UDataflowMesh> Mesh;

public:
	FMakeSphereMeshDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 * Make a capsule mesh
 * DEPRECATED 5.8 - use FMakeCapsuleMeshDataflowNode_v2 instead
 */
USTRUCT(meta = (Deprecated = "5.8"))
struct FMakeCapsuleMeshDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMakeCapsuleMeshDataflowNode, "MakeCapsuleMesh", "Generators|Mesh", "")

private:
	/** Radius of capsule */
	UPROPERTY(EditAnywhere, Category = "Capsule", meta = (UIMin = "0.1", ClampMin = "0.1"));
	float Radius = 1.f;

	/** Length of capsule line segment, so total height is SegmentLength + 2*Radius */
	UPROPERTY(EditAnywhere, Category = "Capsule", meta = (UIMin = "0.1", ClampMin = "0.1"));
	float SegmentLength = 1.f;

	/** Number of vertices along the 90-degree arc from the pole to edge of spherical cap. */
	UPROPERTY(EditAnywhere, Category = "Capsule", meta = (UIMin = "5", ClampMin = "5"));
	int32 NumHemisphereArcSteps = 5;

	/** Number of vertices along each circle */
	UPROPERTY(EditAnywhere, Category = "Capsule", meta = (UIMin = "3", ClampMin = "3"));
	int32 NumCircleSteps = 6;

	/** Number of subdivisions lengthwise along the cylindrical section */
	UPROPERTY(EditAnywhere, Category = "Capsule", meta = (UIMin = "0", ClampMin = "0"));
	int32 NumSegmentSteps = 0;

	/** Output mesh */
	UPROPERTY(meta = (DataflowOutput))
	TObjectPtr<UDynamicMesh> Mesh;

public:
	FMakeCapsuleMeshDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 * Make a capsule mesh
 */
USTRUCT()
struct FMakeCapsuleMeshDataflowNode_v2 : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMakeCapsuleMeshDataflowNode_v2, "MakeCapsuleMesh", "Generators|Mesh", "")

private:
	/** Radius of capsule */
	UPROPERTY(EditAnywhere, Category = "Capsule", meta = (UIMin = "0.1", ClampMin = "0.1"));
	float Radius = 50.f;

	/** Length of capsule line segment, so total height is SegmentLength + 2*Radius */
	UPROPERTY(EditAnywhere, Category = "Capsule", meta = (UIMin = "0.1", ClampMin = "0.1"));
	float SegmentLength = 100.f;

	/** Number of vertices along the 90-degree arc from the pole to edge of spherical cap. */
	UPROPERTY(EditAnywhere, Category = "Capsule", meta = (UIMin = "5", ClampMin = "5"));
	int32 NumHemisphereArcSteps = 5;

	/** Number of vertices along each circle */
	UPROPERTY(EditAnywhere, Category = "Capsule", meta = (UIMin = "3", ClampMin = "3"));
	int32 NumCircleSteps = 6;

	/** Number of subdivisions lengthwise along the cylindrical section */
	UPROPERTY(EditAnywhere, Category = "Capsule", meta = (UIMin = "0", ClampMin = "0"));
	int32 NumSegmentSteps = 0;

	/** Output mesh */
	UPROPERTY(meta = (DataflowOutput))
	TObjectPtr<UDataflowMesh> Mesh;

public:
	FMakeCapsuleMeshDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 * Make a cylinder mesh
 * DEPRECATED 5.8 - use FMakeCylinderMeshDataflowNode_v2 instead
 */
USTRUCT(meta = (Deprecated = "5.8"))
struct FMakeCylinderMeshDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMakeCylinderMeshDataflowNode, "MakeCylinderMesh", "Generators|Mesh", "")

private:
	/** Radius1 of cylinder */
	UPROPERTY(EditAnywhere, Category = "Cylinder", meta = (UIMin = "0.1", ClampMin = "0.1"));
	float Radius1 = 1.f;

	/** Radius2 of cylinder */
	UPROPERTY(EditAnywhere, Category = "Cylinder", meta = (UIMin = "0.1", ClampMin = "0.1"));
	float Radius2 = 1.f;

	/** Height of cylinder */
	UPROPERTY(EditAnywhere, Category = "Cylinder", meta = (UIMin = "0.1", ClampMin = "0.1"));
	float Height = 5.f;

	/** LengthSamples of cylinder */
	UPROPERTY(EditAnywhere, Category = "Cylinder", meta = (UIMin = "0", ClampMin = "0"));
	int32 LengthSamples = 0;

	/** AngleSamples of cylinder */
	UPROPERTY(EditAnywhere, Category = "Cylinder", meta = (UIMin = "4", ClampMin = "4"));
	int32 AngleSamples = 12;

	/** Output mesh */
	UPROPERTY(meta = (DataflowOutput))
	TObjectPtr<UDynamicMesh> Mesh;

public:
	FMakeCylinderMeshDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 * Make a cylinder mesh
 */
USTRUCT()
struct FMakeCylinderMeshDataflowNode_v2 : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMakeCylinderMeshDataflowNode_v2, "MakeCylinderMesh", "Generators|Mesh", "")

private:
	/** Radius1 of cylinder */
	UPROPERTY(EditAnywhere, Category = "Cylinder", meta = (UIMin = "0.1", ClampMin = "0.1"));
	float Radius1 = 50.f;

	/** Radius2 of cylinder */
	UPROPERTY(EditAnywhere, Category = "Cylinder", meta = (UIMin = "0.1", ClampMin = "0.1"));
	float Radius2 = 50.f;

	/** Height of cylinder */
	UPROPERTY(EditAnywhere, Category = "Cylinder", meta = (UIMin = "0.1", ClampMin = "0.1"));
	float Height = 100.f;

	/** LengthSamples of cylinder */
	UPROPERTY(EditAnywhere, Category = "Cylinder", meta = (UIMin = "0", ClampMin = "0"));
	int32 LengthSamples = 0;

	/** AngleSamples of cylinder */
	UPROPERTY(EditAnywhere, Category = "Cylinder", meta = (UIMin = "4", ClampMin = "4"));
	int32 AngleSamples = 12;

	/** Output mesh */
	UPROPERTY(meta = (DataflowOutput))
	TObjectPtr<UDataflowMesh> Mesh;

public:
	FMakeCylinderMeshDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 * Make a box mesh
 * DEPRECATED 5.8 - use FMakeBoxMeshDataflowNode_v2 instead
 */
USTRUCT(meta = (Deprecated = "5.8"))
struct FMakeBoxMeshDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMakeBoxMeshDataflowNode, "MakeBoxMesh", "Generators|Mesh", "")

private:
	/**  */
	UPROPERTY(EditAnywhere, Category = "Box");
	FVector Center = FVector(0.0);

	/**  */
	UPROPERTY(EditAnywhere, Category = "Box", meta = (UIMin = "0.1", ClampMin = "0.1"));
	FVector Size = FVector(5.0);

	/**  */
	UPROPERTY(EditAnywhere, Category = "Box", meta = (UIMin = "1", ClampMin = "1"));
	int32 SubdivisionsX = 3;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Box", meta = (UIMin = "1", ClampMin = "1"));
	int32 SubdivisionsY = 3;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Box", meta = (UIMin = "1", ClampMin = "1"));
	int32 SubdivisionsZ = 3;

	/** Output mesh */
	UPROPERTY(meta = (DataflowOutput))
	TObjectPtr<UDynamicMesh> Mesh;

public:
	FMakeBoxMeshDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 * Make a box mesh
 */
USTRUCT()
struct FMakeBoxMeshDataflowNode_v2 : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMakeBoxMeshDataflowNode_v2, "MakeBoxMesh", "Generators|Mesh", "")

private:
	/** Center of box */
	UPROPERTY(EditAnywhere, Category = "Box", meta = (GizmoType = "Translate"));
	FVector Center = FVector(0.0);

	/** Size of box */
	UPROPERTY(EditAnywhere, Category = "Box", meta = (UIMin = "0.1", ClampMin = "0.1"));
	FVector Size = FVector(100.0);

	/** Subdivisions in X */
	UPROPERTY(EditAnywhere, Category = "Box", meta = (UIMin = "1", ClampMin = "1"));
	int32 SubdivisionsX = 3;

	/** Subdivisions in Y */
	UPROPERTY(EditAnywhere, Category = "Box", meta = (UIMin = "1", ClampMin = "1"));
	int32 SubdivisionsY = 3;

	/** Subdivisions in Z */
	UPROPERTY(EditAnywhere, Category = "Box", meta = (UIMin = "1", ClampMin = "1"));
	int32 SubdivisionsZ = 3;

	/** Output mesh */
	UPROPERTY(meta = (DataflowOutput))
	TObjectPtr<UDataflowMesh> Mesh;

public:
	FMakeBoxMeshDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 * Make a disc mesh
 * DEPRECATED 5.8 - use FMakeDiscMeshDataflowNode_v2 instead
 */
USTRUCT(meta = (Deprecated = "5.8"))
struct FMakeDiscMeshDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMakeDiscMeshDataflowNode, "MakeDiscMesh", "Generators|Mesh", "")

private:
	/** Radius */
	UPROPERTY(EditAnywhere, Category = "Disc", meta = (UIMin = "0.1"));
	float Radius = 1.f;

	/** Normal vector of all vertices will be set to this value. Default is +Z axis. */
	UPROPERTY(EditAnywhere, Category = "Disc");
	FVector Normal = FVector::UnitZ();

	/** Number of vertices around circumference */
	UPROPERTY(EditAnywhere, Category = "Disc", meta = (UIMin = "2", ClampMin = "2"));
	int32 AngleSamples = 12;

	/** Number of vertices along radial spokes */
	UPROPERTY(EditAnywhere, Category = "Disc", meta = (UIMin = "2", ClampMin = "2"));
	int32 RadialSamples = 2;

	/** Start of angle range spanned by disc, in degrees */
	UPROPERTY(EditAnywhere, Category = "Disc");
	float StartAngle = 0.f;

	/** End of angle range spanned by disc, in degrees */
	UPROPERTY(EditAnywhere, Category = "Disc");
	float EndAngle = 360.f;

	/** Output mesh */
	UPROPERTY(meta = (DataflowOutput))
	TObjectPtr<UDynamicMesh> Mesh;

public:
	FMakeDiscMeshDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 * Make a disc mesh
 */
USTRUCT()
struct FMakeDiscMeshDataflowNode_v2 : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMakeDiscMeshDataflowNode_v2, "MakeDiscMesh", "Generators|Mesh", "")

private:
	/** Radius */
	UPROPERTY(EditAnywhere, Category = "Disc", meta = (UIMin = "0.1"));
	float Radius = 50.f;

	/** Normal vector of all vertices will be set to this value. Default is +Z axis. */
	UPROPERTY(EditAnywhere, Category = "Disc");
	FVector Normal = FVector::UnitZ();

	/** Number of vertices around circumference */
	UPROPERTY(EditAnywhere, Category = "Disc", meta = (UIMin = "2", ClampMin = "2"));
	int32 AngleSamples = 12;

	/** Number of vertices along radial spokes */
	UPROPERTY(EditAnywhere, Category = "Disc", meta = (UIMin = "2", ClampMin = "2"));
	int32 RadialSamples = 2;

	/** Start of angle range spanned by disc, in degrees */
	UPROPERTY(EditAnywhere, Category = "Disc", meta = (Units = "Degrees"));
	float StartAngle = 0.f;

	/** End of angle range spanned by disc, in degrees */
	UPROPERTY(EditAnywhere, Category = "Disc", meta = (Units = "Degrees"));
	float EndAngle = 360.f;

	/** Output mesh */
	UPROPERTY(meta = (DataflowOutput))
	TObjectPtr<UDataflowMesh> Mesh;

public:
	FMakeDiscMeshDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

UENUM(BlueprintType)
enum class EDataflowStairTypeEnum : uint8
{
	Linear UMETA(DisplayName = "Linear"),
	Floating UMETA(DisplayName = "Floating"),
	Curved UMETA(DisplayName = "Curved"),
	Spiral UMETA(DisplayName = "Spiral"),
};

/**
 * Make a stair mesh
 * DEPRECATED 5.8 - use FMakeStairMeshDataflowNode_v2 instead
 */
USTRUCT(meta = (Deprecated = "5.8"))
struct FMakeStairMeshDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMakeStairMeshDataflowNode, "MakeStairMesh", "Generators|Mesh", "")

private:
	/** Type of staircase */
	UPROPERTY(EditAnywhere, Category = "Stair", meta = (UIMin = "2"));
	EDataflowStairTypeEnum StairType = EDataflowStairTypeEnum::Linear;

	/** The number of steps in this staircase. */
	UPROPERTY(EditAnywhere, Category = "Stair", meta = (UIMin = "2"));
	int32 NumSteps = 8;

	/** The width of each step. */
	UPROPERTY(EditAnywhere, Category = "Stair", meta = (UIMin = "10.", ClampMin = "10"));
	float StepWidth = 150.f;

	/** The height of each step. */
	UPROPERTY(EditAnywhere, Category = "Stair", meta = (UIMin = "10.", ClampMin = "10"));
	float StepHeight = 20.f;

	/** The depth of each step. */
	UPROPERTY(EditAnywhere, Category = "Stair", meta = (UIMin = "10.", ClampMin = "10", EditCondition = "StairType == EDataflowStairTypeEnum::Linear || StairType == EDataflowStairTypeEnum::Floating", EditConditionHides));
	float StepDepth = 30.f;

	/** Curve angle of the staircase (in degrees) */
	UPROPERTY(EditAnywhere, Category = "Stair", meta = (UIMin = "10.", ClampMin = "10", EditCondition = "StairType == EDataflowStairTypeEnum::Curved || StairType == EDataflowStairTypeEnum::Spiral", EditConditionHides));
	float CurveAngle = 90.f;

	/** Inner radius of the curved staircase */
	UPROPERTY(EditAnywhere, Category = "Stair", meta = (UIMin = "10.", ClampMin = "10", EditCondition = "StairType == EDataflowStairTypeEnum::Curved || StairType == EDataflowStairTypeEnum::Spiral", EditConditionHides));
	float InnerRadius = 150.f;

	/** Output mesh */
	UPROPERTY(meta = (DataflowOutput))
	TObjectPtr<UDynamicMesh> Mesh;

public:
	FMakeStairMeshDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 * Make a stair mesh
 */
USTRUCT()
struct FMakeStairMeshDataflowNode_v2 : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMakeStairMeshDataflowNode_v2, "MakeStairMesh", "Generators|Mesh", "")

private:
	/** Type of staircase */
	UPROPERTY(EditAnywhere, Category = "Stair", meta = (UIMin = "2"));
	EDataflowStairTypeEnum StairType = EDataflowStairTypeEnum::Linear;

	/** The number of steps in this staircase. */
	UPROPERTY(EditAnywhere, Category = "Stair", meta = (UIMin = "2"));
	int32 NumSteps = 8;

	/** The width of each step. */
	UPROPERTY(EditAnywhere, Category = "Stair", meta = (UIMin = "10.", ClampMin = "10"));
	float StepWidth = 150.f;

	/** The height of each step. */
	UPROPERTY(EditAnywhere, Category = "Stair", meta = (UIMin = "10.", ClampMin = "10"));
	float StepHeight = 20.f;

	/** The depth of each step. */
	UPROPERTY(EditAnywhere, Category = "Stair", meta = (UIMin = "10.", ClampMin = "10", EditCondition = "StairType == EDataflowStairTypeEnum::Linear || StairType == EDataflowStairTypeEnum::Floating", EditConditionHides));
	float StepDepth = 30.f;

	/** Curve angle of the staircase (in degrees) */
	UPROPERTY(EditAnywhere, Category = "Stair", meta = (Units = "Degrees", UIMin = "10.", ClampMin = "10", EditCondition = "StairType == EDataflowStairTypeEnum::Curved || StairType == EDataflowStairTypeEnum::Spiral", EditConditionHides));
	float CurveAngle = 90.f;

	/** Inner radius of the curved staircase */
	UPROPERTY(EditAnywhere, Category = "Stair", meta = (UIMin = "10.", ClampMin = "10", EditCondition = "StairType == EDataflowStairTypeEnum::Curved || StairType == EDataflowStairTypeEnum::Spiral", EditConditionHides));
	float InnerRadius = 150.f;

	/** Output mesh */
	UPROPERTY(meta = (DataflowOutput))
	TObjectPtr<UDataflowMesh> Mesh;

public:
	FMakeStairMeshDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 * Make a rectangle mesh
 * DEPRECATED 5.8 - use FMakeRectangleMeshDataflowNode_v2 instead
 */
USTRUCT(meta = (Deprecated = "5.8"))
struct FMakeRectangleMeshDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMakeRectangleMeshDataflowNode, "MakeRectangleMesh", "Generators|Mesh", "")

private:
	/** Rectangle will be translated so that center is at this point */
	UPROPERTY(EditAnywhere, Category = "Rectangle", meta = (DataflowInput));
	FVector Origin = FVector(0.0);

	/** Normal vector of all vertices will be set to this value. Default is +Z axis. */
	UPROPERTY(EditAnywhere, Category = "Rectangle", meta = (DataflowInput));
	FVector Normal = FVector::UnitZ();

	/** Width of rectangle */
	UPROPERTY(EditAnywhere, Category = "Rectangle", meta = (UIMin = "0.1", ClampMin = "0.1"));
	float Width = 5.f;

	/** Height of rectangle */
	UPROPERTY(EditAnywhere, Category = "Rectangle", meta = (UIMin = "0.1", ClampMin = "0.1"));
	float Height = 5.f;

	/** Number of vertices along Width axis */
	UPROPERTY(EditAnywhere, Category = "Rectangle", meta = (UIMin = "2", ClampMin = "2"));
	int32 WidthVertexCount = 3;

	/** Number of vertices along Height axis */
	UPROPERTY(EditAnywhere, Category = "Rectangle", meta = (UIMin = "2", ClampMin = "2"));
	int32 HeightVertexCount = 3;

	/** Output mesh */
	UPROPERTY(meta = (DataflowOutput))
	TObjectPtr<UDynamicMesh> Mesh;

public:
	FMakeRectangleMeshDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 * Make a rectangle mesh
 */
USTRUCT()
struct FMakeRectangleMeshDataflowNode_v2 : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMakeRectangleMeshDataflowNode_v2, "MakeRectangleMesh", "Generators|Mesh", "")

private:
	/** Rectangle will be translated so that center is at this point */
	UPROPERTY(EditAnywhere, Category = "Rectangle", meta = (DataflowInput, GizmoType = "Translate"));
	FVector Origin = FVector(0.0);

	/** Normal vector of all vertices will be set to this value. Default is +Z axis. */
	UPROPERTY(EditAnywhere, Category = "Rectangle", meta = (DataflowInput));
	FVector Normal = FVector::UnitZ();

	/** Width of rectangle */
	UPROPERTY(EditAnywhere, Category = "Rectangle", meta = (UIMin = "0.1", ClampMin = "0.1"));
	float Width = 100.f;

	/** Height of rectangle */
	UPROPERTY(EditAnywhere, Category = "Rectangle", meta = (UIMin = "0.1", ClampMin = "0.1"));
	float Height = 100.f;

	/** Number of vertices along Width axis */
	UPROPERTY(EditAnywhere, Category = "Rectangle", meta = (UIMin = "2", ClampMin = "2"));
	int32 WidthVertexCount = 3;

	/** Number of vertices along Height axis */
	UPROPERTY(EditAnywhere, Category = "Rectangle", meta = (UIMin = "2", ClampMin = "2"));
	int32 HeightVertexCount = 3;

	/** Output mesh */
	UPROPERTY(meta = (DataflowOutput))
	TObjectPtr<UDataflowMesh> Mesh;

public:
	FMakeRectangleMeshDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 * Make a torus mesh
 * DEPRECATED 5.8 - use FMakeTorusMeshDataflowNode_v2 instead
 */
USTRUCT(meta = (Deprecated = "5.8"))
struct FMakeTorusMeshDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMakeTorusMeshDataflowNode, "MakeTorusMesh", "Generators|Mesh", "")

private:
	/** Torus will be translated so that center is at this point */
	UPROPERTY(EditAnywhere, Category = "Torus", meta = (DataflowInput));
	FVector Origin = FVector(0.0);

	/** Radius of the profile */
	UPROPERTY(EditAnywhere, Category = "Torus", meta = (UIMin = "0.01", ClampMin = "0.01"));
	float Radius1 = 4.f;

	/** Number of vertices on the profile */
	UPROPERTY(EditAnywhere, Category = "Torus", meta = (UIMin = "3", ClampMin = "3"));
	int32 ProfileVertexCount = 12;

	/** Radius of sweep curve */
	UPROPERTY(EditAnywhere, Category = "Torus", meta = (UIMin = "0.01", ClampMin = "0.01"));
	float Radius2 = 10.f;

	/** Number of vertices on the sweep curve */
	UPROPERTY(EditAnywhere, Category = "Torus", meta = (UIMin = "3", ClampMin = "3"));
	int32 SweepVertexCount = 12;

	/** Output mesh */
	UPROPERTY(meta = (DataflowOutput))
	TObjectPtr<UDynamicMesh> Mesh;

public:
	FMakeTorusMeshDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 * Make a torus mesh
 */
USTRUCT()
struct FMakeTorusMeshDataflowNode_v2 : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMakeTorusMeshDataflowNode_v2, "MakeTorusMesh", "Generators|Mesh", "")

private:
	/** Torus will be translated so that center is at this point */
	UPROPERTY(EditAnywhere, Category = "Torus", meta = (DataflowInput, GizmoType = "Translate"));
	FVector Origin = FVector(0.0);

	/** Radius of the profile */
	UPROPERTY(EditAnywhere, Category = "Torus", meta = (UIMin = "0.01", ClampMin = "0.01"));
	float Radius1 = 25.f;

	/** Number of vertices on the profile */
	UPROPERTY(EditAnywhere, Category = "Torus", meta = (UIMin = "3", ClampMin = "3"));
	int32 ProfileVertexCount = 12;

	/** Radius of sweep curve */
	UPROPERTY(EditAnywhere, Category = "Torus", meta = (UIMin = "0.01", ClampMin = "0.01"));
	float Radius2 = 50.f;

	/** Number of vertices on the sweep curve */
	UPROPERTY(EditAnywhere, Category = "Torus", meta = (UIMin = "3", ClampMin = "3"));
	int32 SweepVertexCount = 12;

	/** Output mesh */
	UPROPERTY(meta = (DataflowOutput))
	TObjectPtr<UDataflowMesh> Mesh;

public:
	FMakeTorusMeshDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

namespace UE::Dataflow
{
	void DataflowMeshMakeNodes();
}
