// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowMeshNodes.h"
#include "Dataflow/DataflowCore.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/MeshTransforms.h"
#include "UDynamicMesh.h"
#include "DynamicMesh/MeshNormals.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "Dataflow/DataflowEngineUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowMeshNodes)

namespace UE::Dataflow
{
	void DataflowMeshNodes()
	{
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDisplaceDataflowMeshDataflowNode);

		// deprecated
	}
}

/* ----------------------------------------------------------------------------------------------------------------------- */

FDisplaceDataflowMeshDataflowNode::FDisplaceDataflowMeshDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Mesh);
	RegisterInputConnection(&Sampler);
	RegisterInputConnection(&Selection);
	RegisterOutputConnection(&Mesh, &Mesh);
}

void FDisplaceDataflowMeshDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Mesh))
	{
		if (const UDataflowMesh* const InDataflowMesh = GetValue(Context, &Mesh))
		{
			if (const UE::Geometry::FDynamicMesh3* InDynMeshPtr = InDataflowMesh->GetDynamicMesh())
			{
				const int32 NumVertices = InDynMeshPtr->VertexCount();

				if (NumVertices)
				{
					// Get vertices
					TArray<FVector3f> Vertices;
					UE::Dataflow::Mesh::GetMeshVertices(InDynMeshPtr, Vertices);

					// Get vertex normals
					TArray<FVector3f> VertexNormals;
					UE::Dataflow::Mesh::GetMeshVertexNormals(InDynMeshPtr, VertexNormals);

					TObjectPtr<UDataflowMesh> OutMesh = NewObject<UDataflowMesh>();
					OutMesh->SetMaterials(InDataflowMesh->GetMaterials());

					FDynamicMesh3 OutDynMesh = *InDynMeshPtr;

					if (IsConnected(&Sampler))
					{
						if (const FDataflowFloatSampler* FloatSampler = GetValue(Context, &Sampler).TryGet <FDataflowFloatSampler>())
						{
							TArray<float> Values;
							Values.SetNumUninitialized(NumVertices);

							FloatSampler->Sample(Vertices, Values);

							if (IsConnected(&Selection))
							{
								const FDataflowVertexSelection& InSelection = GetValue(Context, &Selection);

								if (NumVertices == InSelection.Num())
								{
									for (int32 VertexIdx = 0; VertexIdx < NumVertices; ++VertexIdx)
									{
										if (InSelection.IsSelected(VertexIdx))
										{
											OutDynMesh.SetVertex(VertexIdx, (FVector)(Vertices[VertexIdx] + VertexNormals[VertexIdx] * Values[VertexIdx]));
										}
									}
								}
							}
							else
							{
								for (int32 VertexIdx = 0; VertexIdx < NumVertices; ++VertexIdx)
								{
									OutDynMesh.SetVertex(VertexIdx, (FVector)(Vertices[VertexIdx] + VertexNormals[VertexIdx] * Values[VertexIdx]));
								}
							}
						}
						else if (const FDataflowVectorSampler* VectorSampler = GetValue(Context, &Sampler).TryGet<FDataflowVectorSampler>())
						{
							TArray<FVector3f> Values; Values.SetNumUninitialized(Vertices.Num());
							VectorSampler->Sample(Vertices, Values);

							if (IsConnected(&Selection))
							{
								const FDataflowVertexSelection& InSelection = GetValue(Context, &Selection);

								if (NumVertices == InSelection.Num())
								{
									for (int32 VertexIdx = 0; VertexIdx < NumVertices; ++VertexIdx)
									{
										if (InSelection.IsSelected(VertexIdx))
										{
											OutDynMesh.SetVertex(VertexIdx, (FVector)(Vertices[VertexIdx] + Values[VertexIdx]));
										}
									}
								}
							}
							else
							{
								for (int32 VertexIdx = 0; VertexIdx < NumVertices; ++VertexIdx)
								{
									OutDynMesh.SetVertex(VertexIdx, (FVector)(Vertices[VertexIdx] + Values[VertexIdx]));
								}
							}
						}

						if (bRecomputeNormals)
						{
							UE::Geometry::FMeshNormals::QuickComputeVertexNormals(OutDynMesh);
						}

						OutMesh->SetDynamicMesh(OutDynMesh);

						SetValue(Context, OutMesh, &Mesh);

						return;
					}
				}
			}

			SafeForwardInput(Context, &Mesh, &Mesh);
		}
	}
}

/* -------------------------------------------------------------------------------- */

