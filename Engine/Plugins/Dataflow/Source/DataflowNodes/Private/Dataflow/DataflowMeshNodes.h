// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Dataflow/SamplerNodes/DataflowSamplerTypes.h"
#include "Dataflow/DataflowSelection.h"
#include "Dataflow/DataflowMesh.h"
#include "Dataflow/DataflowNode.h"

#include "DataflowMeshNodes.generated.h"

/**
 * Displace mesh vertices by a sampler
 * If the connected sampler is a VectorSampler, the selected vertices will be displaced by the sampled vector value.
 * If the connected sampler is a FloatSampler, the selected vertices will be displaced along the vertex normal by the sampled float value.
 */
USTRUCT()
struct FDisplaceDataflowMeshDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDisplaceDataflowMeshDataflowNode, "DisplaceDataflowMesh", "DataflowMesh|Transform", "Offset Deform Sampler Normals")

private:
	/** Output mesh */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Mesh", DataflowIntrinsic))
	TObjectPtr<UDataflowMesh> Mesh;

	/** VertexSelection to specify which vertices will be displaced. If it is not connected, all vertices will be displaced. */
	UPROPERTY(meta = (DataflowInput))
	FDataflowVertexSelection Selection;

	/** Sampler to displace vertices
	 * If the sampler is a VectorSampler, the selected vertices will be displaced by the sampled vector value.
	 * If the sampler is a FloatSampler, the selected vertices will be displaced along the vertex normal by the sampled float value. */
	UPROPERTY(meta = (DataflowInput))
	FDataflowSamplerTypes Sampler;

	/** If it is true, vertex normals will be recomputed after diplacement */
	UPROPERTY(EditAnywhere, Category = "Normals", DisplayName = "Recompute Vertex Normals")
	bool bRecomputeNormals = true;

public:
	FDisplaceDataflowMeshDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

namespace UE::Dataflow
{
	void DataflowMeshNodes();
}

