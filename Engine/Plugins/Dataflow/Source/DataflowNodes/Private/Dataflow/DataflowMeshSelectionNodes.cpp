// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowMeshSelectionNodes.h"
#include "Dataflow/DataflowCore.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/MeshTransforms.h"
#include "UDynamicMesh.h"
#include "Dataflow/DataflowUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowMeshSelectionNodes)

namespace UE::Dataflow
{
	void DataflowMeshSelectionNodes()
	{
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowMeshSelectionCustomDataflowNode);

		// deprecated
	}
}

/* ----------------------------------------------------------------------------------------------------------------------- */

FDataflowMeshSelectionCustomDataflowNode::FDataflowMeshSelectionCustomDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Mesh);
	RegisterInputConnection(&Indices);
	RegisterOutputConnection(&Mesh, &Mesh);
	RegisterOutputConnection(&Selection);
}

void FDataflowMeshSelectionCustomDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Selection))
	{
		if (const UDataflowMesh* const InDataflowMesh = GetValue(Context, &Mesh))
		{
			if (const UE::Geometry::FDynamicMesh3* InDynMeshPtr = InDataflowMesh->GetDynamicMesh())
			{
				const FDataflowOutput* SelectionOutput = FindOutput(&Selection);
				if (SelectionOutput)
				{
					const FName GroupName = FDataflowSelection::GetSelectionTypeFromCollection(*SelectionOutput);

					FDataflowSelection NewSelection(GroupName);
					int32 NumElements = 0;

					if (GroupName == FDataflowVertexSelection::VerticesGroupName)
					{
						NumElements = InDynMeshPtr->VertexCount();
					}
					else if (GroupName == FDataflowFaceSelection::FacesGroupName)
					{
						NumElements = InDynMeshPtr->TriangleCount();
					}

					NewSelection.Initialize(NumElements, false);

					const FString InIndices = GetValue(Context, &Indices);

					TArray<int32> IndicesArray;

					using namespace UE::Dataflow::Utils;

					EErrorCode ErrorCode = ParseIndicesStr(InIndices, IndicesArray);

					if (ErrorCode == EErrorCode::None)
					{
						if (!NewSelection.SetSelectedWithCheck(IndicesArray))
						{
							SetError(Context, &Selection, TEXT("Invalid index specified"));
						}
					}
					else
					{
						if (ErrorCode == EErrorCode::InvalidChars)
						{
							SetError(Context, &Selection, TEXT("Invalid character(s) specified in list"));
						}
						else if (ErrorCode == EErrorCode::InvalidFormatInSegment)
						{
							SetError(Context, &Selection, TEXT("Invalid format in segment"));
						}
					}

					SetValue(Context, MoveTemp(NewSelection), &Selection);

					return;
				}
			}
		}

		SetValue(Context, FDataflowSelection(), &Selection);
	}
	else if (Out->IsA(&Mesh))
	{
		SafeForwardInput(Context, &Mesh, &Mesh);
	}
}

/* -------------------------------------------------------------------------------- */

