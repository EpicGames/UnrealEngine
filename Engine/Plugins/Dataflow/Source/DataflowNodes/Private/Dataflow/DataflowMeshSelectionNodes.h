// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Dataflow/DataflowEngine.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "UDynamicMesh.h"
#include "Dataflow/DataflowSelection.h"
#include "Math/MathFwd.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Dataflow/DataflowMesh.h"

#include "DataflowMeshSelectionNodes.generated.h"

/**
 *
 * Creates any type of selection from a DataflowMesh by using a comma separated list
 * Currentlly supperted types: Vertex/Face
 *
 */
USTRUCT()
struct FDataflowMeshSelectionCustomDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowMeshSelectionCustomDataflowNode, "DataflowMeshSelectionCustom", "DataflowMesh|Selection", "")

private:
	/** DataflowMesh for the selection */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Mesh", DataflowIntrinsic))
	TObjectPtr<UDataflowMesh> Mesh;

	/** Comma separated list of indices (example: "0, 2, 5-10, 12-15") to specify the selection */
	UPROPERTY(EditAnywhere, Category = "Selection", meta = (DataflowInput))
	FString Indices;

	/** Selection from the indices. To set the selection type rt mouse click on the Selection output */
	UPROPERTY(meta = (DataflowOutput))
	FDataflowMeshSelectionTypes Selection;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FDataflowMeshSelectionCustomDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

};

namespace UE::Dataflow
{
	void DataflowMeshSelectionNodes();
}

