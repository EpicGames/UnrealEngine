// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowNodesPlugin.h"

#include "CoreMinimal.h"
#include "Dataflow/DataflowCollectionAttributeKeyNodes.h"
#include "Dataflow/DataflowSkeletalMeshNodes.h"
#include "Dataflow/DataflowStaticMeshNodes.h"
#include "Dataflow/DataflowStaticMeshAssetNodes.h"
#include "Dataflow/DataflowNodeFactory.h"
#include "Dataflow/DataflowSelectionNodes.h"
#include "Dataflow/DataflowContextOverridesNodes.h"
#include "Dataflow/DataflowCollectionAddScalarVertexPropertyNode.h"
#include "Dataflow/DataflowCollectionEditSkinWeightsNode.h"
#include "Dataflow/DataflowCollectionEditSkeletonBonesNode.h"
#include "Dataflow/DataflowCollectionSetSkinningSkeletalMesh.h"
#include "Dataflow/DataflowSkeletonAssetTerminalNode.h"
#include "Dataflow/DataflowTextureToAttributeNode.h"
#include "Dataflow/DataflowVertexColorToAttributeNode.h"
#include "Dataflow/DataflowVisualizeAttributeNode.h"
#include "Dataflow/DataflowSamplerToAttributeNode.h"
#include "Dataflow/DataflowGetCollectionBoundsNode.h"
#include "Dataflow/SamplerNodes/DataflowSamplerTypes.h"
#include "Dataflow/DataflowSelectionToolNode.h"
#include "Dataflow/SamplerNodes/DataflowGradientSamplerNode.h"
#include "Dataflow/SamplerNodes/DataflowTransformSamplerNode.h"
#include "Dataflow/Customization/DataflowToolNodeSnapshotCustomization.h"
#include "Dataflow/Transfer/DataflowMeshAttributesTransferNodes.h"
#include "Dataflow/DataflowPointsNodes.h"
#include "Dataflow/DataflowMeshNodes.h"
#include "Dataflow/DataflowMeshMakeNodes.h"
#include "Dataflow/DataflowMeshSelectionNodes.h"

#define LOCTEXT_NAMESPACE "DataflowNodes"

class FGeometryCollectionAddScalarVertexPropertyCallbacks : public IDataflowAddScalarVertexPropertyCallbacks
{
public:

	const static FName Name;

	virtual ~FGeometryCollectionAddScalarVertexPropertyCallbacks() = default;

	virtual FName GetName() const override
	{
		return Name;
	}

	virtual TArray<FName> GetTargetGroupNames() const override
	{
		return { FGeometryCollection::VerticesGroup };
	}

	virtual void GetTargetGroupInfos(TArray<FTargetGroupInfo>& OutInfos) const override
	{
		OutInfos.Add({
			.TargetGroup = FGeometryCollection::VerticesGroup,
			.PositionAttributeKey = { FGeometryCollection::VertexPositionAttribute, FGeometryCollection::VerticesGroup  },
			.IndicesAttributeKey = { FGeometryCollection::FaceIndicesAttribute, FGeometryCollection::FacesGroup },
			.MappingFrom2DTo3DAttributeKey = {}
			});
	}

	virtual TArray<UE::Dataflow::FRenderingParameter> GetRenderingParameters() const override
	{
		return { { TEXT("SurfaceRender"), FGeometryCollection::StaticType(), {TEXT("Collection")} } };
	}
};

const FName FGeometryCollectionAddScalarVertexPropertyCallbacks::Name = FName("FGeometryCollectionAddScalarVertexPropertyCallbacks");

void IDataflowNodesPlugin::StartupModule()
{
	UE::Dataflow::RegisterSkeletalMeshNodes();
	UE::Dataflow::RegisterStaticMeshNodes();
	UE::Dataflow::RegisterStaticMeshAssetNodes();
	UE::Dataflow::RegisterSelectionNodes();
	UE::Dataflow::RegisterSelectionToolNode();
	UE::Dataflow::RegisterContextOverridesNodes();
	UE::Dataflow::RegisterGetCollectionBoundsNode();

	UE::Dataflow::RegisterTextureToAttributeNodes();
	UE::Dataflow::RegisterVertexColorToAttributeNodes();
	UE::Dataflow::RegisterVisualizeAttributeNodes();

	UE::Dataflow::RegisterSamplerTypes();

	UE::Dataflow::RegisterSamplerToAttributeNodes();

	UE::Dataflow::RegisterSamplerNodes();
	UE::Dataflow::DataflowCollectionAttributeKeyNodes();

	UE::Dataflow::DataflowPointsNodes();
	UE::Dataflow::DataflowMeshNodes();
	UE::Dataflow::DataflowMeshMakeNodes();
	UE::Dataflow::DataflowMeshSelectionNodes();

	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowCollectionAddScalarVertexPropertyNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowCollectionEditSkinWeightsNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowCollectionEditSkeletonBonesNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowCollectionSetSkinningSkeletalMesh);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FSkeletonAssetTerminalNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FTransferMeshAttributesDataflowNode);

	UE::Dataflow::RegisterNodeFilter(FDataflowTerminalNode::StaticType());

	FDataflowAddScalarVertexPropertyCallbackRegistry::Get().RegisterCallbacks(MakeUnique<FGeometryCollectionAddScalarVertexPropertyCallbacks>());

#if WITH_EDITOR
	if (FPropertyEditorModule* const PropertyModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor"))
	{
		PropertyModule->RegisterCustomPropertyTypeLayout(FDataflowToolNodeSnapshot::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FDataflowToolNodeSnapshotCustomization::MakeInstance));
		PropertyModule->RegisterCustomPropertyTypeLayout(FDataflowToolNodeSnapshotSet::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FDataflowToolNodeSnapshotSetCustomization::MakeInstance));
	}
#endif
}

void IDataflowNodesPlugin::ShutdownModule()
{
#if WITH_EDITOR
	if (FPropertyEditorModule* const PropertyModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor"))
	{
		PropertyModule->UnregisterCustomPropertyTypeLayout("DataflowToolNodeSnapshot");
		PropertyModule->UnregisterCustomPropertyTypeLayout("DataflowToolNodeSnapshotSet");
	}
#endif

	FDataflowAddScalarVertexPropertyCallbackRegistry::Get().DeregisterCallbacks(FGeometryCollectionAddScalarVertexPropertyCallbacks::Name);
}

IMPLEMENT_MODULE(IDataflowNodesPlugin, DataflowNodes)

#undef LOCTEXT_NAMESPACE
