// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowPointsNodes.h"
#include "Dataflow/DataflowCore.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "GeometryCollection/ManagedArrayCollection.h"

#include "GeometryCollection/Facades/PointsFacade.h"
#include "GeometryCollection/Facades/CollectionMeshFacade.h"
#include "GeometryCollection/Facades/CollectionTransformSelectionFacade.h"
#include "Containers/Array.h"
#include "Dataflow/DataflowAnyTypeRegistry.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowPointsNodes)

#define LOCTEXT_NAMESPACE "DataflowPointsNodes"

namespace UE::Dataflow
{
	void DataflowPointsNodes()
	{
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FFilterPointsByAttributeDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FGetPointsCollectionDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowSamplerToPointsNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowCollectionVerticesToPointsNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FPointsToDataflowPointsDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowPointsToPointsDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowGetPointsBoundsNode);

		// deprecated

		// AutoConvert
		UE_DATAFLOW_REGISTER_AUTOCONVERT(TArray<FVector>, FDataflowPoints, FPointsToDataflowPointsDataflowNode);
		UE_DATAFLOW_REGISTER_AUTOCONVERT(FDataflowPoints, TArray<FVector>, FDataflowPointsToPointsDataflowNode);
	}
}

/* ----------------------------------------------------------------------------------------------------------------------- */

FFilterPointsByAttributeDataflowNode::FFilterPointsByAttributeDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Points);
	RegisterInputConnection(&Attribute);

	RegisterOutputConnection(&FilteredPoints);
	RegisterOutputConnection(&PointSelection);
	RegisterOutputConnection(&Attribute, &Attribute);
}

FDataflowNode::FAttributeKey FFilterPointsByAttributeDataflowNode::GetVertexAttributeToVisualize(UE::Dataflow::FContext& Context, const FName OutputName, const FName RenderGroup) const
{
	FName GroupName = TEXT("Points");

	return FAttributeKey
	{
		.AttributeName = FName(GetValue(Context, &Attribute)),
		.GroupName = GroupName
	};
}

void FFilterPointsByAttributeDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&FilteredPoints))
	{
		FDataflowVertexSelection OutPointSelection;
		FDataflowPoints OutPoints;

		if (IsConnected(&Points))
		{
			const FDataflowPoints& InPoints = GetValue(Context, &Points);
			const FString InAttribute = GetValue(Context, &Attribute);
			const FName InAttributeName = FName(*InAttribute);

			GeometryCollection::Facades::FPointsFacade PointFacadeInPoints = InPoints.GetPointsFacade();

			const int32 NumPoints = PointFacadeInPoints.GetNumPoints();

			if (NumPoints)
			{
				FManagedArrayCollection OutCollection = FManagedArrayCollection();
				PointFacadeInPoints.FilterByFloatAttributeToCollection(InAttributeName, Operation, Value, Value2, OutCollection);
				OutPoints.Set(OutCollection);

				TArray<int32> Indices;
				PointFacadeInPoints.FilterByFloatAttributeToSelection(InAttributeName, Operation, Value,Value2,  Indices);

				OutPointSelection.Initialize(NumPoints, false);
				OutPointSelection.SetFromArray(Indices);
			}
		}

		SetValue(Context, MoveTemp(OutPointSelection), &PointSelection);
		SetValue(Context, MoveTemp(OutPoints), &FilteredPoints);
	}
	else if (Out->IsA(&Attribute))
	{
		SafeForwardInput(Context, &Attribute, &Attribute);
	}
}

/* ----------------------------------------------------------------------------------------------------------------------- */

FGetPointsCollectionDataflowNode::FGetPointsCollectionDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Points);
	RegisterOutputConnection(&Points, &Points);
	RegisterOutputConnection(&Collection);
}

void FGetPointsCollectionDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Collection))
	{
		const FDataflowPoints& InPoints = GetValue(Context, &Points);
		GeometryCollection::Facades::FPointsFacade PointFacade = InPoints.GetPointsFacade();

		const FManagedArrayCollection& ConstCollection = InPoints.GetPointsFacade().GetConstCollection();
		SetValue(Context, ConstCollection, &Collection);
	}
	else if (Out->IsA(&Points))
	{
		SafeForwardInput(Context, &Points, &Points);
	}
}

/* ----------------------------------------------------------------------------------------------------------------------- */

FDataflowSamplerToPointsNode::FDataflowSamplerToPointsNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Points);
	RegisterInputConnection(&Sampler);
	RegisterInputConnection(&Attribute);

	RegisterOutputConnection(&Points, &Points);
	RegisterOutputConnection(&Attribute, &Attribute);
}

FDataflowNode::FAttributeKey FDataflowSamplerToPointsNode::GetVertexAttributeToVisualize(UE::Dataflow::FContext& Context, const FName OutputName, const FName RenderGroup) const
{
	FName GroupName = TEXT("Points");

	return FAttributeKey
	{
		.AttributeName = FName(GetValue(Context, &Attribute)),
		.GroupName = GroupName
	};
}

void FDataflowSamplerToPointsNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Points))
	{
		if (IsConnected(&Points))
		{
			const FString InAttribute = GetValue(Context, &Attribute);
			const FName InAttributeName = FName(*InAttribute);
			if (InAttributeName.IsNone())
			{
				Context.Error("Attribute name is empty, data will not be set", this, Out);
				SafeForwardInput(Context, &Points, &Points);
				return;
			}

			const FDataflowPoints& InPoints = GetValue(Context, &Points);
			GeometryCollection::Facades::FPointsFacade PointFacadeInPoints = InPoints.GetPointsFacade();

			const int32 NumPoints = PointFacadeInPoints.GetNumPoints();
			if (NumPoints)
			{
				FDataflowPoints OutPoints;
				GeometryCollection::Facades::FPointsFacade PointFacadeOutPoints = OutPoints.GetPointsFacade();
				PointFacadeOutPoints.AddPoints(PointFacadeInPoints.GetPointsAsArray());

				if (IsConnected(&Sampler))
				{
					const FDataflowFloatSampler* FloatSampler = GetValue(Context, &Sampler).TryGet <FDataflowFloatSampler>();
					if (FloatSampler)
					{
						TArray<float> Values; Values.SetNumUninitialized(NumPoints);
						FloatSampler->Sample(TArray<FVector3f>(PointFacadeInPoints.GetPointsAsArray()), Values);

						if (!PointFacadeOutPoints.AddFloatAttribute(InAttributeName, Values))
						{
							SetError(Context, &Points, LOCTEXT("SamplerToPointsAttrAlreadyExist", "Attribute already exist.").ToString());
						}

						SetValue(Context, MoveTemp(OutPoints), &Points);
						return;
					}

					const FDataflowVectorSampler* VectorSampler = GetValue(Context, &Sampler).TryGet<FDataflowVectorSampler>();
					if (VectorSampler)
					{
						TArray<FVector3f> Values; Values.SetNumUninitialized(NumPoints);
						VectorSampler->Sample(TArray<FVector3f>(PointFacadeInPoints.GetPointsAsArray()), Values);

						PointFacadeOutPoints.AddVector3Attribute(InAttributeName, Values);

						SetValue(Context, MoveTemp(OutPoints), &Points);
						return;
					}
				}
				else
				{
					SafeForwardInput(Context, &Points, &Points);
					return;
				}
			}
		}

		SetValue(Context, FDataflowPoints(), &Points);
	}
	else if (Out->IsA(&Attribute))
	{
		SafeForwardInput(Context, &Attribute, &Attribute);
	}
}

/* ----------------------------------------------------------------------------------------------------------------------- */

FDataflowCollectionVerticesToPointsNode::FDataflowCollectionVerticesToPointsNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&Selection);
	RegisterOutputConnection(&Points);
}

void FDataflowCollectionVerticesToPointsNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Points))
	{
		if (IsConnected(&Collection))
		{
			const FManagedArrayCollection& InCollection = GetValue(Context, &Collection);

			TArray<FVector3f> VerticesInCollectionSpace;

			GeometryCollection::Facades::FCollectionMeshFacade MeshFacade(InCollection);
			MeshFacade.GetVerticesInCollectionSpace(VerticesInCollectionSpace);

			FDataflowPoints OutPoints;
			GeometryCollection::Facades::FPointsFacade PointFacadeOutPoints = OutPoints.GetPointsFacade();

			PointFacadeOutPoints.AddPoints(VerticesInCollectionSpace);

			// Copy attributes
			if (bCopyAttributes)
			{
				const TArray<FName> Attributes = InCollection.AttributeNames(FGeometryCollection::VerticesGroup);

				for (const FName& Attr : Attributes)
				{
					if (Attr == "Vertex")
					{
						continue;
					}

					if (InCollection.GetAttributeType(Attr, FGeometryCollection::VerticesGroup) == EManagedArrayType::FFloatType)
					{
						const TManagedArray<float>& FloatAttr = InCollection.GetAttribute<float>(Attr, FGeometryCollection::VerticesGroup);

						if (!PointFacadeOutPoints.AddFloatAttribute(Attr, FloatAttr.GetConstArray()))
						{
							Context.Warning(LOCTEXT("CollectionVerticesToPointsAttrAlreadyExist", "Attribute already exist.").ToString(), this, Out);
						}
					}
					else if (InCollection.GetAttributeType(Attr, FGeometryCollection::VerticesGroup) == EManagedArrayType::FInt32Type)
					{
						const TManagedArray<int32>& IntAttr = InCollection.GetAttribute<int32>(Attr, FGeometryCollection::VerticesGroup);

						if (!PointFacadeOutPoints.AddIntAttribute(Attr, IntAttr.GetConstArray()))
						{
							SetError(Context, &Points, LOCTEXT("CollectionVerticesToPointsAttrAlreadyExist", "Attribute already exist.").ToString());
						}
					}
					else if (InCollection.GetAttributeType(Attr, FGeometryCollection::VerticesGroup) == EManagedArrayType::FBoolType)
					{
						const TManagedArray<bool>& BoolAttr = InCollection.GetAttribute<bool>(Attr, FGeometryCollection::VerticesGroup);

						if (!PointFacadeOutPoints.AddBoolAttribute(Attr, BoolAttr.GetConstArray()))
						{
							SetError(Context, &Points, LOCTEXT("CollectionVerticesToPointsAttrAlreadyExist", "Attribute already exist.").ToString());
						}
					}
					else if (InCollection.GetAttributeType(Attr, FGeometryCollection::VerticesGroup) == EManagedArrayType::FVectorType)
					{
						const TManagedArray<FVector3f>& VectorAttr = InCollection.GetAttribute<FVector3f>(Attr, FGeometryCollection::VerticesGroup);

						if (!PointFacadeOutPoints.AddVector3Attribute(Attr, VectorAttr.GetConstArray()))
						{
							SetError(Context, &Points, LOCTEXT("CollectionVerticesToPointsAttrAlreadyExist", "Attribute already exist.").ToString());
						}
					}
				}
			}

			FDataflowSelection FaceSelection(FGeometryCollection::FacesGroup);
			FaceSelection.Initialize(InCollection.NumElements(FGeometryCollection::FacesGroup), true);

			if (IsConnected(&Selection))
			{
				const FDataflowVertexSelection& InVertexSelection = GetValue(Context, &Selection);

				TArray<int32> DeletionList;
				for (int32 Idx2 = 0; Idx2 < InVertexSelection.Num(); ++Idx2)
				{
					if (!InVertexSelection.IsSelected(Idx2))
					{
						DeletionList.Add(Idx2);
					}
				}

				PointFacadeOutPoints.DeletePoints(DeletionList);

				GeometryCollection::Facades::FCollectionTransformSelectionFacade TransformSelectionFacade(InCollection);
				TArray<int32> SelectionArray = TransformSelectionFacade.ConvertVertexSelectionToFaceSelection(InVertexSelection.AsArray(), true);
				FaceSelection.SetFromArray(SelectionArray);
			}

			// Scatter points
			if (bScatterExtraPoints)
			{
				FRandomStream RandomStream;

				if (bDeterministic)
				{
					RandomStream.Initialize(RandomSeed);
				}
				else
				{
					RandomStream.GenerateNewSeed();
				}

				const TManagedArray<FTransform3f>& BoneTransforms = InCollection.GetAttribute<FTransform3f>("Transform", FGeometryCollection::TransformGroup);

				TArray<FVector3f> NewPoints;

				for (int32 Idx = 0; Idx < BoneTransforms.Num(); ++Idx)
				{
					const TArrayView<const FIntVector> Triangles = MeshFacade.GetTriangles(Idx);

					for (int32 IdxTriangle = 0; IdxTriangle < Triangles.Num(); ++IdxTriangle)
					{
						if (FaceSelection.IsSelected(IdxTriangle))
						{
							const FVector VertexA = FVector(VerticesInCollectionSpace[Triangles[IdxTriangle].X]);
							const FVector VertexB = FVector(VerticesInCollectionSpace[Triangles[IdxTriangle].Y]);
							const FVector VertexC = FVector(VerticesInCollectionSpace[Triangles[IdxTriangle].Z]);

							// Area of triangle
							double AreaTriangleABC = 0.5 * FVector::CrossProduct(VertexB - VertexA, VertexC - VertexA).Length();

							const int32 NumPointsToScatter = AreaTriangleABC * PointDensity;

							for (int32 IdxPoint = 0; IdxPoint < NumPointsToScatter; ++IdxPoint)
							{
								const float U = RandomStream.FRandRange(0.f, 1.f);
								const float V = RandomStream.FRandRange(0.f, 1.f - U);
								const float W = 1.0 - U - V;

								FVector NewPoint = VertexA * U + VertexB * V + VertexC * W;
								NewPoints.Add(FVector3f(NewPoint));
							}
						}
					}
				}

				PointFacadeOutPoints.AppendPoints(NewPoints);
			}

			SetValue(Context, MoveTemp(OutPoints), &Points);
			return;
		}

		SetValue(Context, FDataflowPoints(), &Points);
	}
}

/* ----------------------------------------------------------------------------------------------------------------------- */

FPointsToDataflowPointsDataflowNode::FPointsToDataflowPointsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Points);
	RegisterOutputConnection(&OutPoints);
}

void FPointsToDataflowPointsDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&OutPoints))
	{
		const TArray<FVector>& InPoints = GetValue(Context, &Points);

		FDataflowPoints DataflowPoints;
		GeometryCollection::Facades::FPointsFacade PointFacade = DataflowPoints.GetPointsFacade();
		PointFacade.AddPoints(InPoints);

		FDataflowPoints OutDataflowPoints;
		OutDataflowPoints.Set(PointFacade.GetConstCollection());

		SetValue(Context, MoveTemp(OutDataflowPoints), &OutPoints);
	}
}

/* ----------------------------------------------------------------------------------------------------------------------- */

FDataflowPointsToPointsDataflowNode::FDataflowPointsToPointsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Points);
	RegisterOutputConnection(&OutPoints);
}

void FDataflowPointsToPointsDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&OutPoints))
	{
		const FDataflowPoints& InPoints = GetValue(Context, &Points);
		GeometryCollection::Facades::FPointsFacade PointFacade = InPoints.GetPointsFacade();

		TArray<FVector> PointsArr = PointFacade.GetPointsAsArray();

		SetValue(Context, MoveTemp(PointsArr), &OutPoints);
	}
}

/* ----------------------------------------------------------------------------------------------------------------------- */

FDataflowGetPointsBoundsNode::FDataflowGetPointsBoundsNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Points);
	RegisterInputConnection(&Selection);

	RegisterOutputConnection(&Points, &Points);
	RegisterOutputConnection(&Bounds);
	RegisterOutputConnection(&Sphere);
}

void FDataflowGetPointsBoundsNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Points))
	{
		SafeForwardInput(Context, &Points, &Points);
		return;
	}
	else if (Out->IsA(&Bounds) || Out->IsA(&Sphere))
	{
		FDataflowPoints InPoints = GetValue(Context, &Points, Points);

		GeometryCollection::Facades::FPointsFacade PointsFacade = InPoints.GetPointsFacade();
		TArray<FVector> InPointsArr = PointsFacade.GetPointsAsArray();

		FDataflowVertexSelection InSelection;
		if (IsConnected(&Selection))
		{
			InSelection = GetValue(Context, &Selection);
		}
		else
		{
			InSelection.Initialize(InPointsArr.Num(), true);
		}

		TArray<int32> SelectionArray = InSelection.AsArray();

		TArray<FVector> SelectedVertices;
		SelectedVertices.Reserve(SelectionArray.Num());
		
		// Output bounding box
		FBox OutBox;
		for (int32 Index = 0; Index < SelectionArray.Num(); ++Index)
		{
			if (InPointsArr.IsValidIndex(SelectionArray[Index]))
			{
				OutBox += FVector(InPointsArr[SelectionArray[Index]]);
				SelectedVertices.Add(FVector(InPointsArr[SelectionArray[Index]]));
			}
		}

		SetValue(Context, OutBox, &Bounds);

		// Output bounding sphere
		if (SelectedVertices.Num() > 0)
		{
			FSphere OutSphere(&SelectedVertices[0], SelectedVertices.Num());
				
			SetValue(Context, OutSphere, &Sphere);
		}
		else
		{
			SetValue(Context, FSphere(), &Sphere);
		}
	}
}

/* ----------------------------------------------------------------------------------------------------------------------- */

#undef LOCTEXT_NAMESPACE
