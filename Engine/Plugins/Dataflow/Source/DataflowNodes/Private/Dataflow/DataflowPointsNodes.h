// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Dataflow/DataflowPoints.h"
#include "GeometryCollection/Facades/PointsFacade.h"
#include "Dataflow/SamplerNodes/DataflowSamplerTypes.h"
#include "Dataflow/DataflowNode.h"

#include "DataflowPointsNodes.generated.h"

/**
 *
 * Selects specified points from an incoming point set by an attribute value
 * Currently supported attribute types: float, int32, bool
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FFilterPointsByAttributeDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FFilterPointsByAttributeDataflowNode, "FilterPointsByAttribute", "Points|Filter", "")

private:
	/** Points to filter */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	FDataflowPoints Points;

	/** Operation */
	UPROPERTY(EditAnywhere, Category = "Filter")
	EFilterByAttributeOperation Operation = EFilterByAttributeOperation::Equal;

	/** Attribute used for the filtering operation */
	UPROPERTY(EditAnywhere, Category = "Filter", meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Attribute"))
	FString Attribute = FString("Dummy");

	/** Attribute value for the operation */
	UPROPERTY(EditAnywhere, Category = "Filter")
	float Value = 0.f;

	/** Attribute value for the operation */
	UPROPERTY(EditAnywhere, Category = "Filter", meta = (EditCondition = "Operation == EFilterByAttributeOperation::InRangeInclusive || Operation == EFilterByAttributeOperation::InRangeExclusive"))
	float Value2 = 0.f;

	/** Points after filtering with the operation */
	UPROPERTY(meta = (DataflowOutput))
	FDataflowPoints FilteredPoints;

	/** VertexSelection storing the filtered points */
	UPROPERTY(meta = (DataflowOutput))
	FDataflowVertexSelection PointSelection;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	virtual FAttributeKey GetVertexAttributeToVisualize(UE::Dataflow::FContext& Context, const FName OutputName, const FName RenderGroup) const override;

public:
	FFilterPointsByAttributeDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/**
 * Adds access to the collection of the DataflowPoints
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FGetPointsCollectionDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FGetPointsCollectionDataflowNode, "GetPointsCollection", "Points | Utilities", "")

private:
	/** FDataflowPoints  for the information */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Points", DataflowIntrinsic))
	FDataflowPoints Points;

	/** Formatted string containing the groups and attributes */
	UPROPERTY(meta = (DataflowOutput))
	FManagedArrayCollection Collection;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FGetPointsCollectionDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/**
* Use a sampler to generate an attribute on points
*/
USTRUCT(meta = (Icon = "Icons.EyeDropper"))
struct FDataflowSamplerToPointsNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowSamplerToPointsNode, "SamplerToPoints", "Samplers", "Transfer Generate")

public:
	FDataflowSamplerToPointsNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	/**  */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Points", DataflowIntrinsic))
	FDataflowPoints Points;

	/** Sampler to use to set the value of the attribute */
	UPROPERTY(EditAnywhere, Category = "Source", meta = (DataflowInput))
	FDataflowSamplerTypes Sampler;

	/** Name of the attribute to set the value on */
	UPROPERTY(EditAnywhere, Category = "Attribute", meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Attribute"))
	FString Attribute = FString("Dummy");

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	virtual FAttributeKey GetVertexAttributeToVisualize(UE::Dataflow::FContext& Context, const FName OutputName, const FName RenderGroup) const override;
};

/**
* Create points from vertices in a collection
*/
USTRUCT()
struct FDataflowCollectionVerticesToPointsNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowCollectionVerticesToPointsNode, "CollectionVerticesToPoints", "Points", "")

public:
	FDataflowCollectionVerticesToPointsNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	/** GeometryCollection for the vertices */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Copy attributes from vertices to points, it is ignored when ScatterExtraPoints is on */
	UPROPERTY(EditAnywhere, Category = "Attributes", meta = (EditCondition = "!bScatterExtraPoints"))
	bool bCopyAttributes = true;

	/** If connected, only the selected vertices will be part of the output */
	UPROPERTY(meta = (DataflowInput))
	FDataflowVertexSelection Selection;

	/** Add extra points */
	UPROPERTY(EditAnywhere, Category = "Extra Points")
	bool bScatterExtraPoints = false;

	/** Use deterministic noise for the extra points generation */
	UPROPERTY(EditAnywhere, Category = "Extra Points")
	bool bDeterministic = true;

	/** Seed for the noise for the extra points generation */
	UPROPERTY(EditAnywhere, Category = "Extra Points")
	int32 RandomSeed = 0;

	/** Density for the extra points generation */
	UPROPERTY(EditAnywhere, Category = "Extra Points", meta = (UIMin = 0.1f, ClampMin = 0.1f))
	float PointDensity = 0.1f;

	/** Output points */
	UPROPERTY(meta = (DataflowOutput))
	FDataflowPoints Points;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 * Converts points (TArray<FVector>) to DataflowPoints
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FPointsToDataflowPointsDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FPointsToDataflowPointsDataflowNode, "PointsToDataflowPoints", "Points|Utilities", "")

private:
	/** Points to convert */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	TArray<FVector> Points;

	/** DataflowPoints output */
	UPROPERTY(meta = (DataflowOutput))
	FDataflowPoints OutPoints;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FPointsToDataflowPointsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/**
 * Converts DataflowPoints to points (TArray<FVector>)
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FDataflowPointsToPointsDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowPointsToPointsDataflowNode, "DataflowPointsToPoints", "Points|Utilities", "")

private:
	/** DataflowPoints to convert */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	FDataflowPoints Points;

	/** Points output */
	UPROPERTY(meta = (DataflowOutput))
	TArray<FVector> OutPoints;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FDataflowPointsToPointsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/**
* Get the bounds of the points
*/
USTRUCT()
struct FDataflowGetPointsBoundsNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowGetPointsBoundsNode, "PointsBounds", "Points|Utilities", "BoundingBox Size Limits")

public:
	FDataflowGetPointsBoundsNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	/** Collection to compute the bounds from */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = Points))
	FDataflowPoints Points;

	/** Selection input. To set the selection type rt mouse click on the Selection output */
	UPROPERTY(meta = (DataflowInput))
	FDataflowVertexSelection Selection;

	/** resulting bounding box of the collection */
	UPROPERTY(EditAnywhere, Category = Target, meta = (DisplayName = "Box", DataflowOutput))
	FBox Bounds = FBox(EForceInit::ForceInit);

	/** resulting bounding sphere of the collection */
	UPROPERTY(EditAnywhere, Category = Target, meta = (DataflowOutput))
	FSphere Sphere = FSphere(EForceInit::ForceInit);

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};
namespace UE::Dataflow
{
	void DataflowPointsNodes();
}

