// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowSamplerToAttributeNode.h"

#include "Dataflow/DataflowNodeFactory.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/Facades/CollectionMeshFacade.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowSamplerToAttributeNode)

#define LOCTEXT_NAMESPACE "DataflowSamplerToAttributeNode"

namespace UE::Dataflow
{
	void RegisterSamplerToAttributeNodes()
	{
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowSamplerToAttributeNode);

		// Deprecated
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowFloatSamplerToAttributeNode);
	}
};

// ----------------------------------------------------------------------------------------------------------------------

FDataflowFloatSamplerToAttributeNode::FDataflowFloatSamplerToAttributeNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&Sampler);
	RegisterInputConnection(&AttributeName);
	RegisterOutputConnection(&Collection, &Collection);
}

void FDataflowFloatSamplerToAttributeNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Collection))
	{
		const FName InAttributeName = FName(GetValue(Context, &AttributeName, AttributeName));
		if (InAttributeName.IsNone())
		{
			Context.Error(LOCTEXT("NoAttributeName_Msg", "Attribute name is empty, data will not be set"), this, Out);
			SafeForwardInput(Context, &Collection, &Collection);
			return;
		}

		FManagedArrayCollection InCollection = GetValue(Context, &Collection, Collection);

		const FName InVertexGroupName = VertexGroup.Name;
		if (!InCollection.HasGroup(InVertexGroupName))
		{
			Context.Error(LOCTEXT("InvalidVertexGroup_Msg", "Vertex group does not exists, data will not be set"), this, Out);
			SafeForwardInput(Context, &Collection, &Collection);
			return;
		}

		// Create or get the new attribute 
		TManagedArray<float>& TargetAttribute = InCollection.AddAttribute<float>(InAttributeName, InVertexGroupName);
		TargetAttribute.Fill(0);

		const TManagedArray<FVector3f>* VertexAttribute = InCollection.FindAttribute<FVector3f>(FGeometryCollection::VertexPositionAttribute, InVertexGroupName);
		if (!VertexAttribute)
		{
			// cannot really include cloth here so we need to directly search for the vertex position attribute in the render vertices ( eventually we may converge to use only geometry collections )
			VertexAttribute = InCollection.FindAttribute<FVector3f>("RenderPosition", InVertexGroupName);
		}
		if (!VertexAttribute)
		{
			// cannot really include cloth here so we need to directly search for the vertex position  attribute in the render vertices ( eventually we may converge to use only geometry collections )
			VertexAttribute = InCollection.FindAttribute<FVector3f>("SimPosition3D", InVertexGroupName);
		}
		if (!VertexAttribute)
		{
			Context.Warning(LOCTEXT("NoVertexPosition_Msg", "No vertex position could be found in this collection, no data will be set and the attribute will be inititialized to zeros"), this, Out);
		}
		else
		{
			const FDataflowFloatSampler InSampler = GetValue(Context, &Sampler);
			TArrayView<float> Values = MakeArrayView<float>(TargetAttribute.GetData(), TargetAttribute.Num());
			InSampler.Sample(VertexAttribute->GetConstArray(), Values);
		}

		SetValue(Context, MoveTemp(InCollection), &Collection);
	}
}

FDataflowNode::FAttributeKey FDataflowFloatSamplerToAttributeNode::GetVertexAttributeToVisualize(UE::Dataflow::FContext& Context, const FName OutputName, const FName RenderGroup) const
{
	return FAttributeKey
	{
		.AttributeName = FName(GetValue(Context, &AttributeName, AttributeName)),
		.GroupName = VertexGroup.Name,
	};
}

// ----------------------------------------------------------------------------------------------------------------------

FDataflowSamplerToAttributeNode::FDataflowSamplerToAttributeNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&Sampler);
	RegisterInputConnection(&AttributeName);
	RegisterInputConnection(&VertexSelection);
	RegisterOutputConnection(&Collection, &Collection);
	RegisterOutputConnection(&AttributeName, &AttributeName);
}

void FDataflowSamplerToAttributeNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Collection))
	{
		if (IsConnected(&Collection))
		{
			const FName InAttributeName = FName(GetValue(Context, &AttributeName, AttributeName));
			if (InAttributeName.IsNone())
			{
				Context.Error(LOCTEXT("NoAttributeName_Msg", "Attribute name is empty, data will not be set"), this, Out);
				SafeForwardInput(Context, &Collection, &Collection);
				return;
			}

			FManagedArrayCollection InCollection = GetValue(Context, &Collection, Collection);

			const FName InVertexGroupName = VertexGroup.Name;
			if (!InCollection.HasGroup(InVertexGroupName))
			{
				Context.Error(LOCTEXT("InvalidVertexGroup_Msg", "Vertex group does not exists, data will not be set"), this, Out);
			
				SafeForwardInput(Context, &Collection, &Collection);
				return;
			}

			const TManagedArray<FVector3f>* VertexAttribute = InCollection.FindAttribute<FVector3f>(FGeometryCollection::VertexPositionAttribute, InVertexGroupName);
			
			TArray<FVector3f> VerticesInCollectionSpace;
			if (VertexAttribute)
			{
				GeometryCollection::Facades::FCollectionMeshFacade MeshFacade(InCollection);
				MeshFacade.GetVerticesInCollectionSpace(VerticesInCollectionSpace);
			}
			else
			{
				// cannot really include cloth here so we need to directly search for the vertex position attribute in the render vertices ( eventually we may converge to use only geometry collections )
				VertexAttribute = InCollection.FindAttribute<FVector3f>("RenderPosition", InVertexGroupName);
			
				if (!VertexAttribute)
				{
					// cannot really include cloth here so we need to directly search for the vertex position  attribute in the render vertices ( eventually we may converge to use only geometry collections )
					VertexAttribute = InCollection.FindAttribute<FVector3f>("SimPosition3D", InVertexGroupName);
				}
				if (!VertexAttribute)
				{
					Context.Error(LOCTEXT("NoVertexPosition_Msg", "No vertex position could be found in this collection, no data will be set and the attribute will be inititialized to zeros"), this, Out);

					SafeForwardInput(Context, &Collection, &Collection);
					return;
				}

				VerticesInCollectionSpace = VertexAttribute->GetConstArray();
			}

			const FDataflowVertexSelection& InVertexSelection = GetValue(Context, &VertexSelection);
			const bool bIsValidSelection = InVertexSelection.IsValidForCollection(InCollection);

			if (const FDataflowFloatSampler* InFloatSampler = GetValue(Context, &Sampler).TryGet <FDataflowFloatSampler>())
			{
				TArray<float> SampledValues;
				SampledValues.SetNum(InCollection.NumElements(InVertexGroupName));

				InFloatSampler->Sample(VerticesInCollectionSpace, SampledValues);

				if (!bSaveAsColor)
				{
					TManagedArray<float>& TargetAttribute = InCollection.AddAttribute<float>(InAttributeName, InVertexGroupName);

					for (int32 Idx = 0; Idx < TargetAttribute.Num(); ++Idx)
					{
						if (!bIsValidSelection || InVertexSelection.IsSelected(Idx))
						{
							TargetAttribute[Idx] = SampledValues[Idx];
						}
						else if (bUseDefaultValue)
						{
							TargetAttribute[Idx] = DefaultValue;
						}
					}
				}
				else
				{
					TManagedArray<FLinearColor>& TargetAttribute = InCollection.AddAttribute<FLinearColor>(InAttributeName, InVertexGroupName);

					for (int32 Idx = 0; Idx < TargetAttribute.Num(); ++Idx)
					{
						if (!bIsValidSelection || InVertexSelection.IsSelected(Idx))
						{
							TargetAttribute[Idx] = FLinearColor(FVector3f(SampledValues[Idx]));
						}
						else if (bUseDefaultValue)
						{
							TargetAttribute[Idx] = FLinearColor(FVector3f(DefaultValue));
						}
					}
				}
			}
			else if (const FDataflowVectorSampler* InVectorSampler = GetValue(Context, &Sampler).TryGet<FDataflowVectorSampler>())
			{
				TArray<FVector3f> SampledValues;
				SampledValues.SetNum(InCollection.NumElements(InVertexGroupName));

				InVectorSampler->Sample(VerticesInCollectionSpace, SampledValues);

				if (!bSaveAsColor)
				{
					TManagedArray<FVector3f>& TargetAttribute = InCollection.AddAttribute<FVector3f>(InAttributeName, InVertexGroupName);

					for (int32 Idx = 0; Idx < TargetAttribute.Num(); ++Idx)
					{
						if (!bIsValidSelection || InVertexSelection.IsSelected(Idx))
						{
							TargetAttribute[Idx] = SampledValues[Idx];
						}
						else if (bUseDefaultValue)
						{
							TargetAttribute[Idx] = FVector3f(DefaultVectorValue);
						}
					}
				}
				else
				{
					TManagedArray<FLinearColor>& TargetAttribute = InCollection.AddAttribute<FLinearColor>(InAttributeName, InVertexGroupName);

					for (int32 Idx = 0; Idx < TargetAttribute.Num(); ++Idx)
					{
						if (!bIsValidSelection || InVertexSelection.IsSelected(Idx))
						{
							TargetAttribute[Idx] = FLinearColor(SampledValues[Idx]);
						}
						else if (bUseDefaultValue)
						{
							TargetAttribute[Idx] = FLinearColor(DefaultVectorValue);
						}
					}
				}
			}

			SetValue(Context, MoveTemp(InCollection), &Collection);
		}
	}
	else if (Out->IsA(&AttributeName))
	{
		SafeForwardInput(Context, &AttributeName, &AttributeName);
	}
}

FDataflowNode::FAttributeKey FDataflowSamplerToAttributeNode::GetVertexAttributeToVisualize(UE::Dataflow::FContext& Context, const FName OutputName, const FName RenderGroup) const
{
	return FAttributeKey
	{
		.AttributeName = FName(GetValue(Context, &AttributeName, AttributeName)),
		.GroupName = VertexGroup.Name,
	};
}

#undef LOCTEXT_NAMESPACE 

