// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowNode.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "Dataflow/DataflowCollectionAddScalarVertexPropertyNode.h"
#include "Dataflow/SamplerNodes/DataflowSamplerTypes.h"
#include "Dataflow/DataflowSelection.h"

#include "DataflowSamplerToAttributeNode.generated.h"

/**
* Use a foat sampler to generate the value of an attribute
* DEPRECATED 5.8 - use FDataflowSamplerToAttributeNode instead
*/
USTRUCT(meta = (Icon = "Icons.EyeDropper", Deprecated = "5.8"))
struct FDataflowFloatSamplerToAttributeNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowFloatSamplerToAttributeNode, "FloatSamplerToAttribute", "Dataflow", "Transfer Generate")

public:
	FDataflowFloatSamplerToAttributeNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	/** Collection to set the attribute data to */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = Collection))
	FManagedArrayCollection Collection;

	/** Sampler to use to set the value of the attribute */
	UPROPERTY(EditAnywhere, Category = Source, meta = (DataflowInput))
	FDataflowFloatSampler Sampler;

	/** Name of the attribute to set the value on */
	UPROPERTY(EditAnywhere, Category = Target, meta = (DataflowInput))
	FString AttributeName;

	/** Vertex group to use */
	UPROPERTY(EditAnywhere, Category = Target)
	FScalarVertexPropertyGroup VertexGroup;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	virtual FAttributeKey GetVertexAttributeToVisualize(UE::Dataflow::FContext& Context, const FName OutputName, const FName RenderGroup) const override;
};

// ----------------------------------------------------------------------------------------------------------------------

/**
* Use a float or vector sampler to generate the value of an attribute
*/
USTRUCT(meta = (Icon = "Icons.EyeDropper"))
struct FDataflowSamplerToAttributeNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowSamplerToAttributeNode, "SamplerToAttribute", "Dataflow", "Transfer Generate")

public:
	FDataflowSamplerToAttributeNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	/** Collection to set the attribute data to */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = Collection, DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Sampler to use to set the value of the attribute */
	UPROPERTY(EditAnywhere, Category = Source, meta = (DataflowInput))
	FDataflowSamplerTypes Sampler;

	/** Name of the attribute to set the value on */
	UPROPERTY(EditAnywhere, Category = Attribute, meta = (DataflowInput, DataflowOutput, DataflowPassthrough = AttributeName))
	FString AttributeName = FString("Dummy");

	/** If Sampler input is a FloatSampler and this flag is on then data will be saved as LinearColor using the 
	* sampled float values to create grayscale colors,
	* If Sampler input is a VectorSampler type and this flag is on then data will be saved as LinearColor using 
	* the sampled vector values to set the colors */
	UPROPERTY(EditAnywhere, Category = Attribute)
	bool bSaveAsColor = false;

	/** Vertex group to use */
	UPROPERTY(EditAnywhere, Category = Target)
	FScalarVertexPropertyGroup VertexGroup;

	/** Select vertices to sample value on them */
	UPROPERTY(meta = (DataflowInput))
	FDataflowVertexSelection VertexSelection;

	UPROPERTY(EditAnywhere, Category = Selection)
	bool bUseDefaultValue = false;

	/** This value gets assigned to the non-selected vertices, when the input sampler is a FloatSampler */
	UPROPERTY(EditAnywhere, Category = Selection, meta = (EditCondition = bUseDefaultValue))
	float DefaultValue = 0.f;

	/** This value gets assigned to the non-selected vertices, when the input sampler is a VectorSampler */
	UPROPERTY(EditAnywhere, Category = Selection, meta = (EditCondition = bUseDefaultValue))
	FVector DefaultVectorValue = FVector::ZeroVector;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	virtual FAttributeKey GetVertexAttributeToVisualize(UE::Dataflow::FContext& Context, const FName OutputName, const FName RenderGroup) const override;
};

namespace UE::Dataflow
{
	void RegisterSamplerToAttributeNodes();
}

