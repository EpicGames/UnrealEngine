// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowSelectionNodes.h"
#include "Engine/StaticMesh.h"
#include "GeometryCollection/Facades/CollectionSelectionFacade.h"
#include "Dataflow/DataflowSelection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowSelectionNodes)

namespace UE::Dataflow
{
	void RegisterSelectionNodes()
	{
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FSelectionSetDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMakeSelectionSetDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FGetSelectionSetDataflowNode);
	}
};

void FSelectionSetDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<DataType>(&IndicesOut))
	{
		TArray<int32> IntArray;
		TArray<FString> StrArray;
		Indices.ParseIntoArray(StrArray, *FString(" "));
		for (FString Elem : StrArray)
		{
			if (FCString::IsNumeric(*Elem))
			{
				IntArray.Add(FCString::Atoi(*Elem));
			}
		}

		Out->SetValue<DataType>(MoveTemp(IntArray), Context);
	}
}

// --------------------------------------------------------------------------------------------------------------------------------------

FMakeSelectionSetDataflowNode::FMakeSelectionSetDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&Selection);
	RegisterInputConnection(&Attribute);
	RegisterOutputConnection(&Collection, &Collection);
}

FDataflowNode::FAttributeKey FMakeSelectionSetDataflowNode::GetVertexAttributeToVisualize(UE::Dataflow::FContext& Context, const FName OutputName, const FName RenderGroup) const
{
	FName GroupName;
	if (const FDataflowInput* SelectionInput = FindInput(&Selection))
	{
		GroupName = FDataflowSelection::GetSelectionTypeFromCollection(*SelectionInput);
	}

	return FAttributeKey
	{
		.AttributeName = FName(GetValue(Context, &Attribute)),
		.GroupName = GroupName
	};
}

void FMakeSelectionSetDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Collection))
	{
		if (IsConnected(&Collection))
		{
			if (IsConnected(&Selection))
			{
				FManagedArrayCollection InCollection = GetValue(Context, &Collection);
				GeometryCollection::Facades::FSelectionFacade SelectionFacade(InCollection);

				FString InAttribute = GetValue(Context, &Attribute);

				const FDataflowInput* SelectionInput = FindInput(&Selection);
				if (SelectionInput)
				{
					const FDataflowSelection& InSelection = GetValue(Context, &Selection);

					if (SelectionInput->IsType<FDataflowTransformSelection>())
					{
						FDataflowTransformSelection NewSelection;
						NewSelection.Initialize(InSelection);

						SelectionFacade.AddSelectionToCollection(FDataflowTransformSelection::TransformGroupName, NewSelection.GetBitArray(), InAttribute, bStoreAsFloat);
					}
					else if (SelectionInput->IsType<FDataflowVertexSelection>())
					{
						FDataflowVertexSelection NewSelection;
						NewSelection.Initialize(InSelection);

						SelectionFacade.AddSelectionToCollection(FDataflowVertexSelection::VerticesGroupName, NewSelection.GetBitArray(), InAttribute, bStoreAsFloat);
					}
					else if (SelectionInput->IsType<FDataflowFaceSelection>())
					{
						FDataflowFaceSelection NewSelection;
						NewSelection.Initialize(InSelection);

						SelectionFacade.AddSelectionToCollection(FDataflowFaceSelection::FacesGroupName, NewSelection.GetBitArray(), InAttribute, bStoreAsFloat);
					}
					else if (SelectionInput->IsType<FDataflowGeometrySelection>())
					{
						FDataflowGeometrySelection NewSelection;
						NewSelection.Initialize(InSelection);

						SelectionFacade.AddSelectionToCollection(FDataflowGeometrySelection::GeometryGroupName, NewSelection.GetBitArray(), InAttribute, bStoreAsFloat);
					}
					else if (SelectionInput->IsType<FDataflowCurveSelection>())
					{
						FDataflowCurveSelection NewSelection;
						NewSelection.Initialize(InSelection);

						SelectionFacade.AddSelectionToCollection(FDataflowCurveSelection::CurveGroupName, NewSelection.GetBitArray(), InAttribute, bStoreAsFloat);
					}
					else if (SelectionInput->IsType<FDataflowPointsSelection>())
					{
						FDataflowPointsSelection NewSelection;
						NewSelection.Initialize(InSelection);

						SelectionFacade.AddSelectionToCollection(FDataflowPointsSelection::PointsGroupName, NewSelection.GetBitArray(), InAttribute, bStoreAsFloat);
					}
					else if (SelectionInput->IsType<FDataflowMaterialSelection>())
					{
						FDataflowMaterialSelection NewSelection;
						NewSelection.Initialize(InSelection);

						SelectionFacade.AddSelectionToCollection(FDataflowMaterialSelection::MaterialGroupName, NewSelection.GetBitArray(), InAttribute, bStoreAsFloat);
					}

					SetValue(Context, MoveTemp(InCollection), &Collection);
					return;
				}
			}

			SafeForwardInput(Context, &Collection, &Collection);
			return;
		}
		
		SetValue(Context, FManagedArrayCollection(), &Collection);
	}
}

// --------------------------------------------------------------------------------------------------------------------------------------

FGetSelectionSetDataflowNode::FGetSelectionSetDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&Attribute);
	RegisterOutputConnection(&Collection, &Collection);
	RegisterOutputConnection(&Selection);
}

FDataflowNode::FAttributeKey FGetSelectionSetDataflowNode::GetVertexAttributeToVisualize(UE::Dataflow::FContext& Context, const FName OutputName, const FName RenderGroup) const
{
	FName GroupName;
	if (const FDataflowOutput* SelectionOutput = FindOutput(&Selection))
	{
		GroupName = FDataflowSelection::GetSelectionTypeFromCollection(*SelectionOutput);
	}

	return FAttributeKey
	{
		.AttributeName = FName(GetValue(Context, &Attribute)),
		.GroupName = GroupName
	};
}

void FGetSelectionSetDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Selection))
	{
		if (IsConnected(&Collection))
		{
			FManagedArrayCollection InCollection = GetValue(Context, &Collection);
			GeometryCollection::Facades::FSelectionFacade SelectionFacade(InCollection);

			const FDataflowOutput* SelectionOutput = FindOutput(&Selection);

			FString InAttribute = GetValue(Context, &Attribute);

			FName GroupName;
			if (SelectionOutput)
			{
				using namespace GeometryCollection::Facades;

				GroupName = FDataflowSelection::GetSelectionTypeFromCollection(*SelectionOutput);

				TBitArray<> SelectionArray;
				FSelectionFacade::EErrorCode ErrorCode = SelectionFacade.GetSelectionFromCollection(GroupName, InAttribute, SelectionArray);

				if (ErrorCode == FSelectionFacade::EErrorCode::SelectionGroupCantBeFound)
				{
					SetError(Context, &Collection, TEXT("Collection doesn't contain selection(s)"));
				}
				else if (ErrorCode == FSelectionFacade::EErrorCode::AttributeCantBeFound)
				{
					SetError(Context, &Collection, TEXT("Collection doesn't have a selection with the specified attribute"));
				}

				FDataflowSelection OutSelection(GroupName);
				OutSelection.SetFromBitArray(MoveTemp(SelectionArray));

				SetValue(Context, MoveTemp(OutSelection), &Selection);

				return;
			}
		}

		SetValue(Context, FDataflowSelection(), &Selection);
	}
	else if (Out->IsA(&Collection))
	{
		SafeForwardInput(Context, &Collection, &Collection);
	}
}

