// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Dataflow/DataflowCore.h"
#include "Dataflow/DataflowEngine.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "Dataflow/DataflowSelection.h"
#include "Dataflow/DataflowAnyType.h"

#include "DataflowSelectionNodes.generated.h"

#define UE_API DATAFLOWNODES_API

USTRUCT(meta = (DataflowFlesh))
struct FSelectionSetDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FSelectionSetDataflowNode, "SelectionSet", "Dataflow", "")

public:
	typedef TArray<int32> DataType;

	UPROPERTY(EditAnywhere, Category = "Dataflow")
		FString Indices = FString("1 2 3");

	UPROPERTY(meta = (DataflowOutput, DisplayName = "SelectionSet"))
		TArray<int32> IndicesOut;

	FSelectionSetDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterOutputConnection(&IndicesOut);
	}

	UE_API virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

USTRUCT(meta = (GeometryCollection))
struct FMakeSelectionSetDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMakeSelectionSetDataflowNode, "MakeSelectionSet", "Dataflow", "")

private:
	/** Collection to use */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Selection to store in collection */
	UPROPERTY(meta = (DataflowInput, DisplayName = "Selection", DataflowIntrinsic))
	FDataflowSelectionTypes Selection;

	/** Attribute used to store the selection in the collection. The group decided by the incoming selection's type */
	UPROPERTY(EditAnywhere, Category = "Attribute", meta = (DataflowInput))
	FString Attribute = FString("Dummy");

	/** Store the selection as float instead of bool */
	UPROPERTY(EditAnywhere, Category = "Attribute")
	bool bStoreAsFloat = false;

	UE_API virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	virtual FAttributeKey GetVertexAttributeToVisualize(UE::Dataflow::FContext& Context, const FName OutputName, const FName RenderGroup) const override;

public:
	FMakeSelectionSetDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

USTRUCT(meta = (GeometryCollection))
struct FGetSelectionSetDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FGetSelectionSetDataflowNode, "GetSelectionSet", "Dataflow", "")

private:
	/** Collection to use */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Attribute used to store the selection in the collection. 
	To set the selection type rt mouse click on the Selection output */
	UPROPERTY(EditAnywhere, Category = "Attribute", meta = (DataflowInput))
	FString Attribute = FString("Dummy");

	/** Output selection */
	UPROPERTY(meta = (DataflowOutput))
	FDataflowSelectionTypes Selection;

	UE_API virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	virtual FAttributeKey GetVertexAttributeToVisualize(UE::Dataflow::FContext& Context, const FName OutputName, const FName RenderGroup) const override;

public:
	FGetSelectionSetDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

namespace UE::Dataflow
{
	void RegisterSelectionNodes();
}

#undef UE_API
