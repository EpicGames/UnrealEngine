// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowSelectionToolNode.h"

#include "Dataflow/DataflowNodeFactory.h"
#include "GeometryCollection/Facades/CollectionTransformSelectionFacade.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowSelectionToolNode)

#define LOCTEXT_NAMESPACE "DataflowSelectionToolNode"

namespace UE::Dataflow
{
	void RegisterSelectionToolNode()
	{
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowSelectionToolNode);
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////

void FDataflowSelectionToolNodeData::DeselectVertices(TConstArrayView<int32> Vertices)
{
	for (const int32 Index : Vertices)
	{
		if (Selection.IsValidIndex(Index))
		{
			Selection.SetNotSelected(Index);
		}
	}
}

void FDataflowSelectionToolNodeData::DeselectVertices(const TSet<int32>& Vertices)
{
	for (const int32 Index : Vertices)
	{
		if (Selection.IsValidIndex(Index))
		{
			Selection.SetNotSelected(Index);
		}
	}
}

void FDataflowSelectionToolNodeData::DeselectAllVertices()
{
	Selection.Clear();
}

void FDataflowSelectionToolNodeData::SelectVertices(TConstArrayView<int32> Vertices)
{
	for (const int32 Index : Vertices)
	{
		if (Selection.IsValidIndex(Index))
		{
			Selection.SetSelected(Index);
		}
	}
}

void FDataflowSelectionToolNodeData::SelectVertices(const TSet<int32>& Vertices)
{
	for (const int32 Index : Vertices)
	{
		if (Selection.IsValidIndex(Index))
		{
			Selection.SetSelected(Index);
		}
	}
}

void FDataflowSelectionToolNodeData::GetSelectedVertices(TSet<int32>& OutSelection) const
{
	OutSelection.Reserve(Selection.NumSelected());

	const int32 NumVertices = Selection.Num();
	for (int32 Index = 0; Index < NumVertices; ++Index)
	{
		if (Selection.IsSelected(Index))
		{
			OutSelection.Add(Index);
		}
	}
}

const FDataflowVertexSelection& FDataflowSelectionToolNodeData::GetVertexSelection() const
{
	return Selection;
}

namespace DataflowSelectionToolNodeData::Private
{
	static FName SelectionGroupName = TEXT("Selection");
	static FName SelectionAttributeName = TEXT("Selection");
}

void FDataflowSelectionToolNodeData::Init(const FManagedArrayCollection& InCollection)
{
	Selection.InitializeFromCollection(InCollection, false);
}

void FDataflowSelectionToolNodeData::LoadFromSnapshot(const FDataflowToolNodeSnapshot& InSnapshot)
{
	using namespace DataflowSelectionToolNodeData::Private;

	//const int32 NumVerticesInSnaphot = InSnapshot.Data.NumElements(FDataflowVertexSelection::VerticesGroupName);
	//if (NumVerticesInSnaphot != Selection.Num())
	//{
	//	// TODO : transfer selection if the snapshot does not match the incoming collection
	//}
	//else
	{
		if (const TManagedArray<int32>* SelectionAttribute = InSnapshot.Data.FindAttribute<int32>(SelectionGroupName, SelectionAttributeName))
		{
			for (const int32 Index : *SelectionAttribute)
			{
				if (Selection.IsValidIndex(Index))
				{
					Selection.SetSelected(Index);
				}
			}
		}
	}
}

void FDataflowSelectionToolNodeData::SaveToSnapshot(FDataflowToolNodeSnapshot& OutSnapshot) const
{
	using namespace DataflowSelectionToolNodeData::Private;

	const int32 NumVertices = Selection.Num();
	const int32 NumSelected = Selection.NumSelected();

	OutSnapshot.Name = TEXT("Selection");
	OutSnapshot.Description = FString::Format(TEXT("{0} out of {1} vertices selected"), { FString::FormatAsNumber(NumSelected), FString::FormatAsNumber(NumVertices) });
	OutSnapshot.Data.Reset();
	OutSnapshot.Data.AddGroup(SelectionGroupName);
	OutSnapshot.Data.AddElements(NumSelected, SelectionGroupName);
	TManagedArray<int32>& SelectionAttribute = OutSnapshot.Data.AddAttribute<int32>(SelectionGroupName, SelectionAttributeName);
	int32 SelectedIndex = 0;
	for (int32 Index = 0; Index < NumVertices; ++Index)
	{
		if (Selection.IsSelected(Index) && SelectionAttribute.IsValidIndex(SelectedIndex))
		{
			SelectionAttribute[SelectedIndex] = Index;
			SelectedIndex++;
		}
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FDataflowSelectionToolNode::FDataflowSelectionToolNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowToolNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterOutputConnection(&Collection, &Collection);
	RegisterOutputConnection(&Selection);
}

void FDataflowSelectionToolNode::LoadData(UE::Dataflow::FContext& Context, FDataflowSelectionToolNodeData& OutData) const
{
	const FManagedArrayCollection& InCollection = GetValue(Context, &Collection);
	OutData.Init(InCollection);

	if (const FDataflowToolNodeSnapshot* LastSnaphot = GetActiveSnapshot())
	{
		OutData.LoadFromSnapshot(*LastSnaphot);
	}
}

void FDataflowSelectionToolNode::SaveData(const FDataflowSelectionToolNodeData& InData)
{
	FDataflowToolNodeSnapshot& NewSnapshot = AddSnapshot();
	InData.SaveToSnapshot(NewSnapshot);
	Invalidate();
}

void FDataflowSelectionToolNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Collection))
	{
		SafeForwardInput(Context, &Collection, &Collection);
	}
	else if (Out->IsA(&Selection))
	{
		FDataflowSelectionToolNodeData NodeData;
		LoadData(Context, NodeData);

		const FManagedArrayCollection& InCollection = GetValue(Context, &Collection);
		GeometryCollection::Facades::FCollectionTransformSelectionFacade SelectionFacade(InCollection);

		constexpr bool bAllElementsMustBeSelected = false;

		if (const FDataflowOutput* SelectionOutput = FindOutput(&Selection))
		{
			if (SelectionOutput->IsType<FDataflowTransformSelection>())
			{
				FDataflowTransformSelection OutTransformSelection;
				TArray<int32> TransformIndices = SelectionFacade.ConvertVertexSelectionToTransformSelection(NodeData.GetVertexSelection().AsArray(), bAllElementsMustBeSelected);
				OutTransformSelection.InitFromArray(InCollection, TransformIndices);

				SetValue(Context, OutTransformSelection, &Selection);
			}
			else if (SelectionOutput->IsType<FDataflowVertexSelection>())
			{
				SetValue(Context, NodeData.GetVertexSelection(), &Selection);
			}
			else if (SelectionOutput->IsType<FDataflowFaceSelection>())
			{
				FDataflowFaceSelection OutFaceSelection;
				TArray<int32> FaceIndices = SelectionFacade.ConvertVertexSelectionToFaceSelection(NodeData.GetVertexSelection().AsArray(), bAllElementsMustBeSelected);
				OutFaceSelection.InitFromArray(InCollection, FaceIndices);

				SetValue(Context, OutFaceSelection, &Selection);
			}
			else if (SelectionOutput->IsType<FDataflowGeometrySelection>())
			{
				FDataflowGeometrySelection OutGeoSelection;
				TArray<int32> GeoIndices = SelectionFacade.ConvertVertexSelectionToGeometrySelection(NodeData.GetVertexSelection().AsArray(), bAllElementsMustBeSelected);
				OutGeoSelection.InitFromArray(InCollection, GeoIndices);

				SetValue(Context, OutGeoSelection, &Selection);
			}
			else if (SelectionOutput->IsType<FDataflowMaterialSelection>())
			{
				Context.Error(LOCTEXT("MaterialSelectionNotSupported", "Material Selection output is not supported"));
				SetValue(Context, FDataflowSelection(), &Selection);
			}
			else if (SelectionOutput->IsType<FDataflowCurveSelection>())
			{
				FDataflowCurveSelection OutCurveSelection;
				TBitArray<> CurveSelectionArray;
				SelectionFacade.ConvertVertexSelectionToCurveSelection(NodeData.GetVertexSelection().GetBitArray(), bAllElementsMustBeSelected, CurveSelectionArray);
				OutCurveSelection.InitializeFromCollection(InCollection, false);
				OutCurveSelection.SetFromBitArray(MoveTemp(CurveSelectionArray));

				SetValue(Context, OutCurveSelection, &Selection);
			}
			else
			{
				Context.Error(LOCTEXT("SelectionTypeNotSupported", "The selection output type is not supported"));
				SetValue(Context, FDataflowSelection(), &Selection);
			}
		}
		else
		{
			Context.Error(LOCTEXT("OutputNotFound", "The selection output could not be found"));
			SetValue(Context, FDataflowSelection(), &Selection);
		}
	}
}

#undef LOCTEXT_NAMESPACE