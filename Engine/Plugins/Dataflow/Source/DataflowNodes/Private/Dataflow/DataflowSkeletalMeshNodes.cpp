// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowSkeletalMeshNodes.h"
#include "Dataflow/DataflowContent.h"
#include "Dataflow/DataflowCore.h"
#include "Dataflow/DataflowEngine.h"
#include "Dataflow/DataflowEngineTypes.h"
#include "Logging/LogMacros.h"
#include "UObject/UnrealTypePrivate.h"
#include "Animation/Skeleton.h"
#include "Engine/SkeletalMesh.h"
#include "SkeletalDebugRendering.h"
#include "Dataflow/DataflowDebugDrawInterface.h"
#include "Dataflow/DataflowElement.h"
#include "PhysicsEngine/PhysicsAsset.h"

//#if WITH_EDITOR
//#include "Dataflow/DataflowRenderingViewMode.h"
//#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowSkeletalMeshNodes)

namespace UE::Dataflow
{
	void RegisterSkeletalMeshNodes()
	{
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FGetSkeletalMeshDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FGetSkeletonDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FSkeletalMeshBoneDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FSkeletalMeshReferenceTransformDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FGetPhysicsAssetFromSkeletalMeshDataflowNode);
		
		DATAFLOW_NODE_REGISTER_GETTER_FOR_ASSET(USkeletalMesh, FGetSkeletalMeshDataflowNode);
		DATAFLOW_NODE_REGISTER_GETTER_FOR_ASSET(USkeleton, FGetSkeletonDataflowNode);
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
///

void FDataflowDebugDrawSkeletonObject::OverrideBoneTransforms(const TArray<FTransform>& InBoneTransforms)
{
	FScopeLock Lock(&TransformOverridesLockGuard);
	TransformOverrides = InBoneTransforms;
}

const FReferenceSkeleton& FDataflowDebugDrawSkeletonObject::GetReferenceSkeleton() const
{
	return ReferenceSkeleton;
}

void FDataflowDebugDrawSkeletonObject::PopulateDataflowElements()
{
	ElementsOffset = DataflowElements.Num();
	ElementsSize = ReferenceSkeleton.GetNum();
	
	DataflowElements.Reserve(ElementsOffset + ElementsSize);
	PreviousSelection.Init(false, ElementsSize);
	for(int32 BoneIndex = 0; BoneIndex < ElementsSize; ++BoneIndex)
	{
		int32 ParentIndex = ReferenceSkeleton.GetParentIndex(BoneIndex);
		FBox BoundingBox;
		BoundingBox += ReferenceSkeleton.GetBoneAbsoluteTransform(BoneIndex).GetLocation();
		if(ParentIndex == INDEX_NONE)
		{
			ParentIndex = 0;
		}
		else
		{
			ParentIndex += ElementsOffset;
			if(ParentIndex < DataflowElements.Num())
			{
				DataflowElements[ParentIndex]->BoundingBox += ReferenceSkeleton.GetBoneAbsoluteTransform(BoneIndex).GetLocation();
			}
		}
		if (ParentIndex < DataflowElements.Num())
		{
			TSharedPtr<FDataflowProxyElement> ProxyElement = MakeShared<FDataflowProxyElement>(ReferenceSkeleton.GetBoneName(BoneIndex).ToString(),
				DataflowElements[ParentIndex].Get(), BoundingBox, true);
			ProxyElement->ElementProxy = MakeRefCount<HDataflowElementHitProxy>(ElementsOffset + BoneIndex, FName(ProxyElement->ElementName));
			DataflowElements.Add(ProxyElement);
		}
	}
}

FBox FDataflowDebugDrawSkeletonObject::ComputeBoundingBox() const
{
	FBox BoundingBox(ForceInitToZero);

	const uint32 ElementsEnd = ElementsSize + ElementsOffset;;
	for(uint32 ElementIndex = ElementsOffset; ElementIndex < ElementsEnd; ++ElementIndex)
	{
		BoundingBox += DataflowElements[ElementIndex]->BoundingBox;
	}

	return BoundingBox;
}

void FDataflowDebugDrawSkeletonObject::DrawDataflowElements(FPrimitiveDrawInterface* PDI) 
{
	const int32 NumBones = ReferenceSkeleton.GetNum();

	TArray<FBoneIndexType> RequiredBones;
	TArray<TRefCountPtr<HHitProxy>> HitProxies;
	TArray<FLinearColor> BoneColors;
	TArray<FTransform> WorldTransforms;
	TArray<int32> SelectedBones;
	TArray<FName> BoneNames;
	
	ReferenceSkeleton.GetBoneAbsoluteTransforms(WorldTransforms);

	// apply the transform overrides
	{
		FScopeLock Lock(&TransformOverridesLockGuard);
		check(NumBones == WorldTransforms.Num());
		for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
		{
			if (TransformOverrides.IsValidIndex(BoneIndex))
			{
				WorldTransforms[BoneIndex] = TransformOverrides[BoneIndex];
			}
		}
	}

	RequiredBones.Reserve(NumBones);

	bool bSelectionChanged = false;
	if(NumBones == ElementsSize)
	{
		for(int32 BoneIndex = 0; BoneIndex < ElementsSize; ++BoneIndex)
		{
			IDataflowDebugDrawInterface::FDataflowElementsType::ElementType& DataflowElement = DataflowElements[ElementsOffset+BoneIndex];
			if(DataflowElement.IsValid())
			{
				if(DataflowElement->bIsVisible)
				{
					RequiredBones.Add(BoneIndex);
				}
				if(DataflowElement->bIsSelected != PreviousSelection[BoneIndex])
				{
					PreviousSelection[BoneIndex] = DataflowElement->bIsSelected;
					bSelectionChanged = true;
				}
				if(DataflowElement->bIsSelected)
				{
					SelectedBones.Add(BoneIndex);
					BoneNames.Add(FName(DataflowElement->ElementName));
				}
				// Populate the proxies with the ones stored on the elements. 
				if(DataflowElement->IsA(FDataflowProxyElement::StaticType()))
				{
					if (FDataflowProxyElement* ProxyElement = StaticCast<FDataflowProxyElement*>(DataflowElement.Get()))
					{
						HitProxies.Add(ProxyElement->ElementProxy);
					}
				}
			}
		}
	}

	if(bSelectionChanged)
	{
		// this method can be called on a different thread than the game thread 
		// let's make sure the bon change broadcast happens on the game thread
		Async(EAsyncExecution::TaskGraphMainTick,
			[this, BoneNames]
			{
				OnBoneSelectionChanged.Broadcast(BoneNames);
			});
	}
	if(bBonesVisible)
	{
		FSkelDebugDrawConfig DrawConfig;
		DrawConfig.bUseMultiColorAsDefaultColor = true;
		DrawConfig.BoneDrawMode = EBoneDrawMode::Selected;
		DrawConfig.BoneDrawSize = 1.f;
		DrawConfig.bAddHitProxy = true;
		DrawConfig.bForceDraw = true;
		DrawConfig.DefaultBoneColor = FLinearColor(0.0f,0.0f,0.025f,1.0f);
		DrawConfig.AffectedBoneColor = FLinearColor(1.0f,1.0f,1.0f,1.0f);
		DrawConfig.SelectedBoneColor = FLinearColor(0.2f,1.0f,0.2f,1.0f);
		DrawConfig.ParentOfSelectedBoneColor = FLinearColor(0.85f,0.45f,0.12f,1.0f);
	
		SkeletalDebugRendering::DrawBones(
			PDI, FVector::Zero(),
			RequiredBones,
			ReferenceSkeleton,
			WorldTransforms,
			SelectedBones,
			BoneColors,
			HitProxies,
			DrawConfig
		);
	}
}

void FGetSkeletalMeshDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	typedef TObjectPtr<const USkeletalMesh> DataType;
	if (Out->IsA<DataType>(&SkeletalMesh))
	{
		SetValue(Context, SkeletalMesh, &SkeletalMesh);

		if (!SkeletalMesh)
		{
			if (const UE::Dataflow::FEngineContext* EngineContext = Context.AsType<UE::Dataflow::FEngineContext>())
			{
				if (const USkeletalMesh* SkeletalMeshFromOwner = UE::Dataflow::Reflection::FindObjectPtrProperty<USkeletalMesh>(
					EngineContext->Owner, PropertyName))
				{
					SetValue<DataType>(Context, DataType(SkeletalMeshFromOwner), &SkeletalMesh);
				}
			}
		}
	}
}

bool FGetSkeletalMeshDataflowNode::SupportsAssetProperty(UObject* Asset) const
{
	return (Cast<USkeletalMesh>(Asset) != nullptr);
}

void FGetSkeletalMeshDataflowNode::SetAssetProperty(UObject* Asset)
{
	if (USkeletalMesh* SkeletalMeshAsset = Cast<USkeletalMesh>(Asset))
	{
		SkeletalMesh = SkeletalMeshAsset;
	}
}

#if WITH_EDITOR
void FGetSkeletalMeshDataflowNode::DebugDraw(UE::Dataflow::FContext& Context,
		IDataflowDebugDrawInterface& DataflowRenderingInterface,
		const FDebugDrawParameters& DebugDrawParameters) const
{
	if(SkeletalMesh)
	{
		TRefCountPtr<IDataflowDebugDrawObject> SkeletonObject(MakeDebugDrawObject<FDataflowDebugDrawSkeletonObject>(
			DataflowRenderingInterface.ModifyDataflowElements(), SkeletalMesh->GetRefSkeleton()));
	
		DataflowRenderingInterface.DrawObject(SkeletonObject);
	}
}

bool FGetSkeletalMeshDataflowNode::CanDebugDrawViewMode(const FName& ViewModeName) const
{
	return true;// ViewModeName == UE::Dataflow::FDataflowConstruction3DViewMode::Name;
}
#endif

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

void FGetSkeletonDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	typedef TObjectPtr<const USkeleton> DataType;
	if (Out->IsA<DataType>(&Skeleton))
	{
		SetValue(Context, Skeleton, &Skeleton);

		if (!Skeleton)
		{
			if (const UE::Dataflow::FEngineContext* EngineContext = Context.AsType<UE::Dataflow::FEngineContext>())
			{
				if (const USkeleton* SkeletonFromOwner = UE::Dataflow::Reflection::FindObjectPtrProperty<USkeleton>(
					EngineContext->Owner, PropertyName))
				{
					SetValue<DataType>(Context, DataType(SkeletonFromOwner), &Skeleton);
				}
			}
		}
	}
}

bool FGetSkeletonDataflowNode::SupportsAssetProperty(UObject* Asset) const
{
	return (Cast<USkeleton>(Asset) != nullptr);
}

void FGetSkeletonDataflowNode::SetAssetProperty(UObject* Asset)
{
	if (USkeleton* SkeletonAsset = Cast<USkeleton>(Asset))
	{
		Skeleton = SkeletonAsset;
	}
}

#if WITH_EDITOR

void FGetSkeletonDataflowNode::DebugDraw(UE::Dataflow::FContext& Context,
		IDataflowDebugDrawInterface& DataflowRenderingInterface,
		const FDebugDrawParameters& DebugDrawParameters) const
{
	if(Skeleton)
	{
		TRefCountPtr<IDataflowDebugDrawObject> SkeletonObject(MakeDebugDrawObject<FDataflowDebugDrawSkeletonObject>(
			DataflowRenderingInterface.ModifyDataflowElements(), Skeleton->GetReferenceSkeleton()));
	
		DataflowRenderingInterface.DrawObject(SkeletonObject);
	}
}

bool FGetSkeletonDataflowNode::CanDebugDrawViewMode(const FName& ViewModeName) const
{
	return true;//ViewModeName == UE::Dataflow::FDataflowConstruction3DViewMode::Name;
}

#endif

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

void FSkeletalMeshBoneDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	typedef TObjectPtr<const USkeletalMesh> InDataType;
	if (Out->IsA<int>(&BoneIndexOut))
	{
		SetValue<int>(Context, INDEX_NONE, &BoneIndexOut);

		if( InDataType InSkeletalMesh = GetValue<InDataType>(Context, &SkeletalMesh) )
		{
			FName LocalBoneName = BoneName;
			if (LocalBoneName.IsNone())
			{
				if (const UE::Dataflow::FEngineContext* EngineContext = Context.AsType<UE::Dataflow::FEngineContext>())
				{
					LocalBoneName = FName(UE::Dataflow::Reflection::FindOverrideProperty< FString >(EngineContext->Owner, PropertyName, FName("BoneName")));
				}
			}

			int32 Index = InSkeletalMesh->GetRefSkeleton().FindBoneIndex(LocalBoneName);
			SetValue(Context, Index, &BoneIndexOut);
		}

	}
}


void FSkeletalMeshReferenceTransformDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	typedef TObjectPtr<const USkeletalMesh> InDataType;
	if (Out->IsA<FTransform>(&TransformOut))
	{
		SetValue(Context, FTransform::Identity, &TransformOut);
		
		int32 BoneIndex = GetValue<int32>(Context, &BoneIndexIn);
		if (0 <= BoneIndex)
		{
			if (InDataType SkeletalMesh = GetValue<InDataType>(Context, &SkeletalMeshIn))
			{
				TArray<FTransform> ComponentPose;
				UE::Dataflow::Animation::GlobalTransforms(SkeletalMesh->GetRefSkeleton(), ComponentPose);
				if (BoneIndex < ComponentPose.Num())
				{
					SetValue(Context, ComponentPose[BoneIndex], &TransformOut);
				}
			}
		}
	}
}

void FGetPhysicsAssetFromSkeletalMeshDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&PhysicsAsset))
	{
		TObjectPtr<const USkeletalMesh> InSkeletalMesh = GetValue(Context, &SkeletalMesh);
		UPhysicsAsset* OutPhysicsAsset = InSkeletalMesh ? InSkeletalMesh->GetPhysicsAsset() : nullptr;
		SetValue(Context, TObjectPtr<const UPhysicsAsset>(OutPhysicsAsset), &PhysicsAsset);
	}
}