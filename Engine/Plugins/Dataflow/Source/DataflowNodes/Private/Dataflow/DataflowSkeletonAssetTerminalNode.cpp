// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowSkeletonAssetTerminalNode.h"
#include "Animation/Skeleton.h"
#include "AnimationUtils.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Misc/PackageName.h"
#include "UObject/Package.h"
#include "UObject/SavePackage.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowSkeletonAssetTerminalNode)

namespace UE::Dataflow::Private
{
	static void OverrideSkeletonData(TObjectPtr<USkeleton> TargetSkeleton, TObjectPtr<const USkeleton> SourceSkeleton)
	{
		if (!TargetSkeleton || !SourceSkeleton || TargetSkeleton == SourceSkeleton)
		{
			return;
		}

#if WITH_EDITOR

		// Get the package and name
		UPackage* Package = TargetSkeleton->GetOutermost();
		FName ObjectName = TargetSkeleton->GetFName();

		// Duplicate into same package with same name
		USkeleton* NewSkeleton = DuplicateObject<USkeleton>(SourceSkeleton, Package, ObjectName);

		// Mark the package dirty 
		Package->MarkPackageDirty();
		FAssetRegistryModule::AssetCreated(NewSkeleton);

		// Save the package 
		FString FilePath = FPackageName::LongPackageNameToFilename(Package->GetName(), FPackageName::GetAssetPackageExtension());
		FSavePackageArgs SaveArgs;
		SaveArgs.TopLevelFlags = EObjectFlags::RF_Public | EObjectFlags::RF_Standalone;
		SaveArgs.SaveFlags = SAVE_NoError;

		UPackage::SavePackage(Package, NewSkeleton, *FilePath, SaveArgs);

#endif
	}
}

FSkeletonAssetTerminalNode::FSkeletonAssetTerminalNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowTerminalNode(InParam, InGuid)
{
	RegisterInputConnection(&SourceSkeleton);
	RegisterInputConnection(&SkeletonAsset);
}

void FSkeletonAssetTerminalNode::Evaluate(UE::Dataflow::FContext& Context) const
{}

void FSkeletonAssetTerminalNode::SetAssetValue(TObjectPtr<UObject> Asset, UE::Dataflow::FContext& Context) const
{
	USkeleton* AssetToSet = Cast<USkeleton>(Asset.Get());
	if (!AssetToSet)
	{
		AssetToSet = GetValue(Context, &SkeletonAsset);
	}
	if(AssetToSet)
	{
		UE::Dataflow::Private::OverrideSkeletonData(AssetToSet, GetValue(Context, &SourceSkeleton));
	}
}