// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Dataflow/DataflowCore.h"
#include "Dataflow/DataflowEngine.h"

#include "DataflowSkeletonAssetTerminalNode.generated.h"

class USkeleton;

/*
* Dataflow terminal node to save a skeleton asset
*/
USTRUCT(meta = (DataflowFlesh, DataflowTerminal))
struct FSkeletonAssetTerminalNode : public FDataflowTerminalNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FSkeletonAssetTerminalNode, "SkeletonAssetTerminal", "Terminal", "")

public:
	FSkeletonAssetTerminalNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	/** Source Skeleton used to override the skeleton asset */
	UPROPERTY(meta = (DataflowInput))
	TObjectPtr<const USkeleton> SourceSkeleton = nullptr;

	/** Skeleton Asset to be saved */
	UPROPERTY(EditAnywhere, Category = Asset, meta = (DataflowInput))
	TObjectPtr<USkeleton> SkeletonAsset = nullptr;

	//~ Begin FDataflowTerminalNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context) const override;
	virtual void SetAssetValue(TObjectPtr<UObject> Asset, UE::Dataflow::FContext& Context) const override;
	//~ End FDataflowTerminalNode interface
};
