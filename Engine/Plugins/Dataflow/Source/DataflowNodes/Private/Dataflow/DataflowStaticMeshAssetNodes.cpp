// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowStaticMeshAssetNodes.h"

#include "AssetUtils/StaticMeshMaterialUtil.h"
#include "Dataflow/DataflowNodeFactory.h"
#include "Dataflow/DataflowObjectInterface.h"
#include "DynamicMeshToMeshDescription.h"
#include "Engine/StaticMesh.h"
#include "Materials/MaterialInterface.h"
#include "MeshConversionOptions.h"
#include "StaticMeshDescription.h"
#include "StaticMeshOperations.h"
#include "UDynamicMesh.h"

#define LOCTEXT_NAMESPACE "DataflowStaticMeshAssetNodes"

namespace UE::Dataflow
{
	void RegisterStaticMeshAssetNodes()
	{
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowStaticMeshTerminalNode);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////

FDataflowStaticMeshTerminalNode::FDataflowStaticMeshTerminalNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowTerminalNode(InParam, InGuid)
{
	RegisterInputConnection(&Mesh);
	RegisterInputConnection(&Materials);
	RegisterInputConnection(&AssetPath);
	RegisterOutputConnection(&StaticMeshAsset);
}

void FDataflowStaticMeshTerminalNode::Evaluate(UE::Dataflow::FContext& Context) const
{
	// Asset is created and stored by the SetAssetValue
	// Evaluate returns this created asset
	const UE::Dataflow::FEngineContext* EngineContext = Context.AsType<UE::Dataflow::FEngineContext>();
	TObjectPtr<UObject> BoundAsset = EngineContext ? EngineContext->Owner : nullptr;

	const FString InAssetPath = GetAssetPath(GetValue(Context, &AssetPath), BoundAsset);
	TObjectPtr<UStaticMesh> OutStaticMesh = ::LoadObject<UStaticMesh>(nullptr, InAssetPath);
	SetValue(Context, OutStaticMesh, &StaticMeshAsset);
}

void FDataflowStaticMeshTerminalNode::SetAssetValue(TObjectPtr<UObject> Asset, UE::Dataflow::FContext& Context) const
{
	const TObjectPtr<UStaticMesh> NullStaticMeshPtr = nullptr;

	const TObjectPtr<UDynamicMesh> InMesh = GetValue(Context, &Mesh);
	if (!IsValid(InMesh))
	{
		Context.Error(LOCTEXT("InvalidInputMesh", "Input dynamic mesh mesh is empty"), this);
		SetValue(Context, NullStaticMeshPtr, &StaticMeshAsset);
		return;
	}

	// Materials are optional, no need to check for validity
	const TArray<TObjectPtr<UMaterialInterface>>& InMaterials = GetValue(Context, &Materials);

	const FString InAssetPath = GetAssetPath(GetValue(Context, &AssetPath), Asset);
	if (InAssetPath.IsEmpty())
	{
		Context.Error(FText::Format(
			LOCTEXT("InvalidPathAndBoundAsset", "Asset Path input is not a valid path and no static mesh asset is bound to this dataflow: path = {0}"), 
			FText::FromString(InAssetPath)), 
			this);
		SetValue(Context, NullStaticMeshPtr, &StaticMeshAsset);
		return;
	}

	UObject* OutAsset = GetOrCreateAsset(Context, InAssetPath, UStaticMesh::StaticClass());
	TObjectPtr<UStaticMesh> OutStaticMesh = Cast<UStaticMesh>(OutAsset);
	if (!OutStaticMesh)
	{
		Context.Error(FText::Format(
			LOCTEXT("InvalidOutputAsset", "Failed to convert the new asset to a Static mesh: path = {0}"),
			FText::FromString(InAssetPath)),
			this);
		SetValue(Context, NullStaticMeshPtr, &StaticMeshAsset);
		return;
	}

	if (!ConvertDynamicMeshToStaticMesh(Context, *InMesh, InMaterials, *OutStaticMesh))
	{
		SetValue(Context, NullStaticMeshPtr, &StaticMeshAsset);
		return;
	}
	SetValue(Context, OutStaticMesh, &StaticMeshAsset);

}

bool FDataflowStaticMeshTerminalNode::ConvertDynamicMeshToStaticMesh(
	UE::Dataflow::FContext& Context, 
	const UDynamicMesh& InMesh, 
	const TArray<TObjectPtr<UMaterialInterface>>& InMaterials, 
	UStaticMesh& OutMesh) const
{
	using namespace UE::Geometry;

	UStaticMeshDescription* StaticMeshDescription = OutMesh.GetStaticMeshDescription(0);
	if (!StaticMeshDescription)
	{
		StaticMeshDescription = UStaticMesh::CreateStaticMeshDescription(&OutMesh);
	}
	if (!StaticMeshDescription)
	{
		Context.Error(LOCTEXT("FailToCreateStaticMeshDesc", "Failed to create static mesh description"), this);
		return false;
	}

	// start fresh, empty the description 
	FMeshDescription& MeshDescription = StaticMeshDescription->GetMeshDescription();
	MeshDescription.Empty();

	InMesh.ProcessMesh([&MeshDescription](const FDynamicMesh3& InSourceMesh)
		{
			FConversionToMeshDescriptionOptions ConversionOptions;
			ConversionOptions.bUpdateTangents = true;
			ConversionOptions.bUpdateUVs = true;

			FDynamicMeshToMeshDescription Converter(ConversionOptions);
			Converter.Convert(&InSourceMesh, MeshDescription, /** CopyTangents */true);
		});

	// assign the materials
	TArray<FStaticMaterial> NewStaticMaterials;
	NewStaticMaterials.Reserve(InMaterials.Num());
	for (int32 MaterialIndex = 0; MaterialIndex < InMaterials.Num(); ++MaterialIndex)
	{
		FStaticMaterial& NewMaterial = NewStaticMaterials.AddDefaulted_GetRef();
		NewMaterial.MaterialInterface = InMaterials[MaterialIndex].Get();
		NewMaterial.MaterialSlotName = UE::AssetUtils::GenerateNewMaterialSlotName(NewStaticMaterials, NewMaterial.MaterialInterface, MaterialIndex);
		NewMaterial.ImportedMaterialSlotName = NewMaterial.MaterialSlotName;
		NewMaterial.UVChannelData = FMeshUVChannelInfo(1.f);
	}
	OutMesh.SetStaticMaterials(NewStaticMaterials);

	TPolygonGroupAttributesRef<FName> PolygonGroupImportedMaterialSlotNames = StaticMeshDescription->GetRequiredAttributes().GetPolygonGroupMaterialSlotNames();
	for (int32 SlotIdx = 0; SlotIdx < NewStaticMaterials.Num(); ++SlotIdx)
	{
		if (SlotIdx < PolygonGroupImportedMaterialSlotNames.GetNumElements())
		{
			PolygonGroupImportedMaterialSlotNames.Set(SlotIdx, NewStaticMaterials[SlotIdx].MaterialSlotName);
		}
	}

	FStaticMeshOperations::ComputeTriangleTangentsAndNormals(MeshDescription);
	FStaticMeshOperations::ComputeTangentsAndNormals(MeshDescription, EComputeNTBsFlags::Normals);

	// Build the static mesh
	UStaticMesh::FBuildMeshDescriptionsParams Params;
	Params.bFastBuild = true;
	Params.bAllowCpuAccess = true;
	Params.bCommitMeshDescription = true;
	const TArray<const FMeshDescription*> MeshDescriptions{ &MeshDescription };
	const bool bResult = OutMesh.BuildFromMeshDescriptions(MeshDescriptions, Params);

#if WITH_EDITOR
	UStaticMesh::FBuildParameters BuildParameters;
	BuildParameters.bInSilent = true;
	BuildParameters.bInRebuildUVChannelData = true;
	BuildParameters.bInEnforceLightmapRestrictions = true;
	OutMesh.Build(BuildParameters);
#endif

	return bResult;
}



#undef LOCTEXT_NAMESPACE
