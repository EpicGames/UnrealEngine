// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowTerminalNode.h"

#include "DataflowStaticMeshAssetNodes.generated.h"

class UStaticMesh;
class UDynamicMesh;
class UMaterialInterface;

/*
* terminal node to a save a static mesh asset
*/
USTRUCT(meta = (DataflowTerminal))
struct FDataflowStaticMeshTerminalNode : public FDataflowTerminalNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowStaticMeshTerminalNode, "StaticMeshTerminal", "Terminal", "")

private:
	/** Input dynamic mesh to convert to a static mesh */
	UPROPERTY(meta = (DataflowInput))
	TObjectPtr<UDynamicMesh> Mesh = nullptr;

	/** Materials to set on the static mesh */
	UPROPERTY(meta = (DataflowInput))
	TArray<TObjectPtr<UMaterialInterface>> Materials;

	/** 
	* Path of the asset to save to 
	* if not specified this will use the asset bound the dataflow graph this node belong to
	*/
	UPROPERTY(EditAnywhere, Category = Asset, meta = (DataflowInput))
	FString AssetPath;

	/** Output static mesh asset created by this node */
	UPROPERTY(meta = (DataflowOutput))
	TObjectPtr<UStaticMesh> StaticMeshAsset = nullptr;

	virtual void Evaluate(UE::Dataflow::FContext& Context) const override;
	virtual void SetAssetValue(TObjectPtr<UObject> Asset, UE::Dataflow::FContext& Context) const override;

	bool ConvertDynamicMeshToStaticMesh(UE::Dataflow::FContext& Context, const UDynamicMesh& InMesh, const TArray<TObjectPtr<UMaterialInterface>>& InMaterials, UStaticMesh& OutMesh) const;

public:
	FDataflowStaticMeshTerminalNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

namespace UE::Dataflow
{
	void RegisterStaticMeshAssetNodes();
}

