// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowStaticMeshNodes.h"
#include "Engine/StaticMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowStaticMeshNodes)

namespace UE::Dataflow
{
	void RegisterStaticMeshNodes()
	{
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FGetStaticMeshDataflowNode_v2);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FGetStaticMeshBoundingBoxDataflowNode);

		DATAFLOW_NODE_REGISTER_GETTER_FOR_ASSET(UStaticMesh, FGetStaticMeshDataflowNode_v2);

		// deprecated 
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FGetStaticMeshDataflowNode);
	}
}

//////////////////////////////////////////////////////////////////////////////////////////////

FGetStaticMeshDataflowNode_v2::FGetStaticMeshDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterOutputConnection(&StaticMesh);
}

void FGetStaticMeshDataflowNode_v2::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&StaticMesh))
	{
		SetValue(Context, StaticMesh, &StaticMesh);
	}
}

bool FGetStaticMeshDataflowNode_v2::SupportsAssetProperty(UObject* Asset) const
{
	return (Cast<UStaticMesh>(Asset) != nullptr);
}

void FGetStaticMeshDataflowNode_v2::SetAssetProperty(UObject* Asset)
{
	if (UStaticMesh* StaticMeshAsset = Cast<UStaticMesh>(Asset))
	{
		StaticMesh = StaticMeshAsset;
	}
}

//////////////////////////////////////////////////////////////////////////////////////////////

FGetStaticMeshDataflowNode::FGetStaticMeshDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterOutputConnection(&StaticMesh);
}

void FGetStaticMeshDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	typedef TObjectPtr<const UStaticMesh> DataType;
	if (Out->IsA<DataType>(&StaticMesh))
	{
		SetValue(Context, StaticMesh, &StaticMesh); // prime to avoid ensure

		if (StaticMesh)
		{
			SetValue(Context, StaticMesh, &StaticMesh);
		}
		else if (const UE::Dataflow::FEngineContext* EngineContext = Context.AsType<UE::Dataflow::FEngineContext>())
		{
			if (const UStaticMesh* StaticMeshFromOwner = UE::Dataflow::Reflection::FindObjectPtrProperty<UStaticMesh>(
				EngineContext->Owner, PropertyName))
			{
				SetValue(Context, DataType(StaticMeshFromOwner), &StaticMesh);

			}
		}
	}
}

bool FGetStaticMeshDataflowNode::SupportsAssetProperty(UObject* Asset) const
{
	return (Cast<UStaticMesh>(Asset) != nullptr);
}

void FGetStaticMeshDataflowNode::SetAssetProperty(UObject* Asset)
{
	if (UStaticMesh* StaticMeshAsset = Cast<UStaticMesh>(Asset))
	{
		StaticMesh = StaticMeshAsset;
	}
}

//////////////////////////////////////////////////////////////////////////////////////////////

FGetStaticMeshBoundingBoxDataflowNode::FGetStaticMeshBoundingBoxDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&StaticMesh);

	RegisterOutputConnection(&BoundingBox);
	RegisterOutputConnection(&Center);
	RegisterOutputConnection(&Dimensions);
}

void FGetStaticMeshBoundingBoxDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&BoundingBox) || Out->IsA(&Center) || Out->IsA(&Dimensions))
	{
		FBox OutBoundingBox(ForceInit);

		if (const TObjectPtr<const UStaticMesh> InStaticMesh = GetValue(Context, &StaticMesh))
		{
			OutBoundingBox = InStaticMesh->GetBounds().GetBox();
		}

		SetValue(Context, OutBoundingBox, &BoundingBox);
		SetValue(Context, OutBoundingBox.GetCenter(), &Center);
		SetValue(Context, OutBoundingBox.GetSize(), &Dimensions);
	}
}

