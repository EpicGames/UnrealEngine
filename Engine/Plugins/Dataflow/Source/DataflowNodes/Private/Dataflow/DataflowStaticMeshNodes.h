// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Dataflow/DataflowCore.h"
#include "Dataflow/DataflowEngine.h"

#include "DataflowStaticMeshNodes.generated.h"

#define UE_API DATAFLOWNODES_API

DEFINE_LOG_CATEGORY_STATIC(LogDataflowStaticMeshNodes, Log, All);

class UStaticMesh;

/*
* Get a existing static mesh asset
*/
USTRUCT()
struct FGetStaticMeshDataflowNode_v2 : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FGetStaticMeshDataflowNode_v2, "StaticMesh", "General", "Static Mesh")

public:
	FGetStaticMeshDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	UPROPERTY(EditAnywhere, Category = "Dataflow", meta = (DataflowOutput, DisplayName = "StaticMesh"))
	TObjectPtr<const UStaticMesh> StaticMesh = nullptr;

	UE_API virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

	UE_API virtual bool SupportsAssetProperty(UObject* Asset) const override;
	UE_API virtual void SetAssetProperty(UObject* Asset) override;
};

/**
 * Get the local geometric bounding box a static mesh
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FGetStaticMeshBoundingBoxDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FGetStaticMeshBoundingBoxDataflowNode, "GetStaticMeshBoundingBox", "Static Mesh", "Bounds Size Dimensions Extents Center")

private:
	/** Static Mesh to compute the bouning box from */
	UPROPERTY(EditAnywhere, Category = "Dataflow", meta = (DataflowInput))
	TObjectPtr<const UStaticMesh> StaticMesh = nullptr;

	/** Geometric bounding box of the input Static Mesh */
	UPROPERTY(meta = (DataflowOutput))
	FBox BoundingBox = FBox(ForceInit);

	/** Center of the resulting bounding box */
	UPROPERTY(meta = (DataflowOutput))
	FVector Center = FVector::ZeroVector;

	/** Dimensions of the resulting bounding box in centimers */
	UPROPERTY(meta = (DataflowOutput))
	FVector Dimensions = FVector::ZeroVector;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FGetStaticMeshBoundingBoxDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

};

/// DEPRECATED NODE SECTION

/*
* Get a existing static mesh asset
* Note that this version of the node is deprecated, please use non deprecated version of the StaticMesh node
*/
USTRUCT(meta = (Deprecated = 5.8))
struct FGetStaticMeshDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FGetStaticMeshDataflowNode, "StaticMesh", "General", "Static Mesh")

public:

	UPROPERTY(EditAnywhere, Category = "Dataflow", meta = (DataflowOutput, DisplayName = "StaticMesh"))
	TObjectPtr<const UStaticMesh> StaticMesh = nullptr;

	UPROPERTY(EditAnywhere, Category = "Dataflow")
	FName PropertyName = "StaticMesh";

	FGetStaticMeshDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

	UE_API virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

	UE_API virtual bool SupportsAssetProperty(UObject* Asset) const override;
	UE_API virtual void SetAssetProperty(UObject* Asset) override;
};


namespace UE::Dataflow
{
	void RegisterStaticMeshNodes();
}

#undef UE_API
