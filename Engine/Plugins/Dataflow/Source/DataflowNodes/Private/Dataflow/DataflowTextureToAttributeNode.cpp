// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowTextureToAttributeNode.h"

#include "Components/DynamicMeshComponent.h"
#include "Dataflow/DataflowNodeFactory.h"
#include "Engine/Texture2D.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/Facades/CollectionMeshFacade.h"
#include "ImageCore.h"
#include "Async/ParallelFor.h"


#if WITH_EDITOR
#include "UObject/UObjectGlobals.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowTextureToAttributeNode)

#define LOCTEXT_NAMESPACE "DataflowTextureToAttributeNode"

namespace UE::Dataflow
{
	void RegisterTextureToAttributeNodes()
	{
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowTextureToAttributeNode);
	}

	namespace Private
	{
		static bool GetImageFromTexture2D(UTexture2D& InTexture, FImage& OutImage)
		{
			if (const FSharedImageConstRef CpuCopy = InTexture.GetCPUCopy())
			{
				CpuCopy->CopyTo(OutImage, ERawImageFormat::RGBA32F, EGammaSpace::Linear);
				return true;
			}
#if WITH_EDITORONLY_DATA
			if (InTexture.Source.IsValid())
			{
				FImage MipImage;
				if (InTexture.Source.GetMipImage(MipImage, 0))
				{
					MipImage.CopyTo(OutImage, ERawImageFormat::RGBA32F, EGammaSpace::Linear);
					return true;
				}
			}
#endif
			return false;
		}
	}
};



FDataflowTextureToAttributeNode::FDataflowTextureToAttributeNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&Texture);
	RegisterInputConnection(&UVChannel);
	RegisterInputConnection(&AttributeName);
	RegisterOutputConnection(&Collection, &Collection);
	RegisterOutputConnection(&AttributeName, &AttributeName);
}

FDataflowTextureToAttributeNode::~FDataflowTextureToAttributeNode()
{
	StopListeningToTextureChanges();
}

void FDataflowTextureToAttributeNode::PostSerialize(const FArchive& Ar)
{
	FDataflowNode::PostSerialize(Ar);
	StopListeningToTextureChanges();
	StartListeningToTextureChanges();
}

void FDataflowTextureToAttributeNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&AttributeName))
	{
		SafeForwardInput(Context, &AttributeName, &AttributeName);
	}
	else if (Out->IsA(&Collection))
	{
		const FName InAttributeName = FName(GetValue(Context, &AttributeName, AttributeName));
		if (InAttributeName.IsNone())
		{
			Context.Error(LOCTEXT("NoAttributeName_Msg", "Attribute name is empty, data transfer from texture will be skipped"), this, Out);
			SafeForwardInput(Context, &Collection, &Collection);
			return;
		}

		FManagedArrayCollection InCollection = GetValue(Context, &Collection, Collection);

		const FName InVertexGroupName = VertexGroup.Name;
		if (!InCollection.HasGroup(InVertexGroupName))
		{
			Context.Error(LOCTEXT("InvalidVertexGroup_Msg", "Vertex group does not exists, data transfer from texture will be skipped"), this, Out);
			SafeForwardInput(Context, &Collection, &Collection);
			return;
		}

		const TManagedArray<float>* ExistingAttribute = InCollection.FindAttributeTyped<float>(InAttributeName, InVertexGroupName);
		if (ExistingAttribute == nullptr && InCollection.HasAttribute(InAttributeName, InVertexGroupName))
		{
			Context.Warning(LOCTEXT("ExistingAttributeWrongType_Msg", "Attribute already exist but is not a float type, transfer will be skipped"), this, Out);
			SafeForwardInput(Context, &Collection, &Collection);
			return;
		}

		const TObjectPtr<UTexture2D> InTexture = GetValue(Context, &Texture, Texture);
		if (!InTexture)
		{
			Context.Warning(LOCTEXT("NoTexture_Msg", "Input texture is null, no data will be transfered and the attribute will be inititialized to zeros"), this, Out);
		}

		// Create or get the new attribute 
		TManagedArray<float>& TargetAttribute = InCollection.AddAttribute<float>(InAttributeName, InVertexGroupName);
		TargetAttribute.Fill(0);

		// get the Uv attribute ( could be single or array of Uvs depending on the type of collection )
		const int32 InUVChannel = GetValue(Context, &UVChannel, UVChannel);
		GeometryCollection::Facades::FCollectionMeshFacade MeshFacade(InCollection);
		const TManagedArray<FVector2f>* UvAttribute = MeshFacade.FindUVLayer(InUVChannel);

		const TManagedArray<TArray<int32>>* UvLookup = nullptr;
		if (!UvAttribute)
		{
			UvLookup = InCollection.FindAttributeTyped<TArray<int32>>("SimVertex2DLookup", InVertexGroupName);
			if (UvLookup)
			{
				UvAttribute = InCollection.FindAttributeTyped<FVector2f>("SimPosition2D", "SimVertices2D");
			}
		}

		// cannot really include cloth here so we need to directlky serach for the UV attribute in the render vertices ( eventually we may converge to use only geometry collections )
		const TManagedArray<TArray<FVector2f>>* UvArrayAttribute = InCollection.FindAttributeTyped<TArray<FVector2f>>("RenderUVs", InVertexGroupName);
		const int32 UvArrayNumChannels = (UvArrayAttribute && (UvArrayAttribute->Num() > 0)) ? (*UvArrayAttribute)[0].Num() : 0;

		if (!UvAttribute && (InUVChannel<0 || InUVChannel >= UvArrayNumChannels))
		{
			Context.Warning(LOCTEXT("InvalidUVChannel_Msg", "Invalid UV channel, no data will be transfered and the attribute will be inititialized to zeros"), this, Out);
		}
		else if (InTexture)
		{
			FImage Image;
			const bool bGotImage = UE::Dataflow::Private::GetImageFromTexture2D(*InTexture, Image);
			if (!bGotImage || Image.SizeX == 0 || Image.SizeY == 0)
			{
				Context.Warning(LOCTEXT("TextureError_Msg", "Failed to get the texture data or empty image, no data will be transfered and the attribute will be inititialized to zeros"), this, Out);
			}
			else
			{
				TArrayView64<FLinearColor> Pixels = Image.AsRGBA32F();

				auto GetUVs = [&UvAttribute, &UvArrayAttribute, &InUVChannel, &UvLookup](int32 Index)
					{
						if (UvAttribute)
						{
							if (UvLookup && UvLookup->IsValidIndex(Index))
							{
								if ((*UvLookup)[Index].Num())
								{
									const int32 UvIndex = (*UvLookup)[Index][0];
									if (UvAttribute->IsValidIndex(UvIndex))
									{
										// SimPosition2D = (1 - UV) * UVScale (see ClothGeometryTools::BuildIslandsFromDynamicMeshUVs).
										// For default UVScale=(1,1): UV = 1 - SimPosition2D, recovering both U and V correctly.
										return FVector2f(1.0f) - (*UvAttribute)[UvIndex];
									}
								}
							}
							else if (UvAttribute->IsValidIndex(Index))
							{
								return (*UvAttribute)[Index];
							}
						}
						else if (UvArrayAttribute && UvArrayAttribute->IsValidIndex(Index))
						{
							const TArray<FVector2f>& UVs = (*UvArrayAttribute)[Index];
							return UVs.IsValidIndex(InUVChannel)? UVs[InUVChannel]: FVector2f::ZeroVector;
						}
						return FVector2f::ZeroVector;
					};

				const int32 MaxX = Image.SizeX - 1;
				const int32 MaxY = Image.SizeY - 1;

				const int32 NumVertices = TargetAttribute.Num();
				const int32 MinBatchSize = 1000;
				ParallelFor(TEXT("DataflowTextureToAttribute"), NumVertices, MinBatchSize,
					[&GetUVs, &Image, &MaxX, &MaxY, &Pixels, &TargetAttribute](const int32 Index) {
						const FVector2f UvCoord = GetUVs(Index);
						const FVector2f NormUvCoord
						{
							FMath::Fractional(FMath::Fractional(UvCoord.X) + 1.0f), // [0,1[
							FMath::Fractional(FMath::Fractional(UvCoord.Y) + 1.0f), // [0,1[
						};
						const FVector2f PixelCoord
						{
							NormUvCoord.X * (float)(Image.SizeX) - 0.5f,
							NormUvCoord.Y * (float)(Image.SizeY) - 0.5f,
						};
						const FVector2f PixelFraction
						{
							FMath::Fractional(PixelCoord.X),
							FMath::Fractional(PixelCoord.Y),
						};

						const int32 PixelX0 = FMath::Clamp(int32(PixelCoord.X), 0, MaxX);
						const int32 PixelX1 = FMath::Clamp(PixelX0 + 1, 0, MaxX);
						const int32 PixelY0 = FMath::Clamp(int32(PixelCoord.Y), 0, MaxY);
						const int32 PixelY1 = FMath::Clamp(PixelY0 + 1, 0, MaxY);
						const FLinearColor Pixel00 = Pixels[PixelX0 + PixelY0 * Image.SizeX];
						const FLinearColor Pixel10 = Pixels[PixelX1 + PixelY0 * Image.SizeX];
						const FLinearColor Pixel01 = Pixels[PixelX0 + PixelY1 * Image.SizeX];
						const FLinearColor Pixel11 = Pixels[PixelX1 + PixelY1 * Image.SizeX];

						const FLinearColor Lerp0 = FMath::Lerp(Pixel00, Pixel10, PixelFraction.X);
						const FLinearColor Lerp1 = FMath::Lerp(Pixel01, Pixel11, PixelFraction.X);
						TargetAttribute[Index] = FMath::Lerp(Lerp0, Lerp1, PixelFraction.Y).R;
					},
					bMultithreadProcessing? EParallelForFlags::None: EParallelForFlags::ForceSingleThread);
			}
		}

		SetValue(Context, MoveTemp(InCollection), &Collection);
	}
}

FDataflowNode::FAttributeKey FDataflowTextureToAttributeNode::GetVertexAttributeToVisualize(UE::Dataflow::FContext& Context, const FName OutputName, const FName RenderGroup) const
{
	return FAttributeKey
	{
		.AttributeName = FName(GetValue(Context, &AttributeName, AttributeName)),
		.GroupName = VertexGroup.Name,
	};
}

void FDataflowTextureToAttributeNode::OnPropertyChanged(UE::Dataflow::FContext& Context, const FPropertyChangedEvent& InPropertyChangedEvent)
{
	if (InPropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(FDataflowTextureToAttributeNode, Texture))
	{
		StopListeningToTextureChanges();
		StartListeningToTextureChanges();
	}
}

void FDataflowTextureToAttributeNode::StartListeningToTextureChanges()
{
#if WITH_EDITOR
	check(!TextureChangeDelegateHandle.IsValid());
	TextureChangeDelegateHandle = FCoreUObjectDelegates::OnObjectPropertyChanged.AddSP(this, &FDataflowTextureToAttributeNode::OnTextureChanged);
#endif
}

void FDataflowTextureToAttributeNode::StopListeningToTextureChanges()
{
#if WITH_EDITOR
	if (TextureChangeDelegateHandle.IsValid())
	{
		FCoreUObjectDelegates::OnObjectPropertyChanged.RemoveAll(this);
		TextureChangeDelegateHandle.Reset();
	}
#endif
}

void FDataflowTextureToAttributeNode::OnTextureChanged(UObject* Object, FPropertyChangedEvent& Event)
{
	if (IsValid(Texture) && IsValid(Object) && Texture == Object)
	{
		TWeakPtr<FDataflowNode> LocalWeakThis = this->AsWeak();
		ExecuteOnGameThread(TEXT("FDataflowTextureToAttributeNode::OnTextureChanged"),
			[LocalWeakThis]()
			{
				if (TSharedPtr<FDataflowNode> SharedThis = LocalWeakThis.Pin())
				{
					SharedThis->Invalidate();
				}
			}
		);
	}
}

#undef LOCTEXT_NAMESPACE 

