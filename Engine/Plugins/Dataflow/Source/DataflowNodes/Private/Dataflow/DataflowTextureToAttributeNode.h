// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowNode.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "Dataflow/DataflowCollectionAddScalarVertexPropertyNode.h"
#include "Delegates/IDelegateInstance.h"

#include "DataflowTextureToAttributeNode.generated.h"

class UTexture2D;

/**
* Transfer data from a texture to a an attribute in a collection
* The texture is sampled at the specific UVs (UVChannel parameter) and is transfered to an float attribute in the selected vertex group 
* Sampling is done using bilinear filtering
*/
USTRUCT()
struct FDataflowTextureToAttributeNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowTextureToAttributeNode, "TextureToAttribute", "Dataflow", "Transfer UVs Projection")

public:
	FDataflowTextureToAttributeNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
	virtual ~FDataflowTextureToAttributeNode();

private:
	/** Collection to transfer the attribute data to */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = Collection))
	FManagedArrayCollection Collection;

	/** Texture to transfer the attribute from */
	UPROPERTY(EditAnywhere, Category = Source, meta = (DataflowInput))
	TObjectPtr<UTexture2D> Texture = nullptr;

	/** UV channel to use to sample the texture */
	UPROPERTY(EditAnywhere, Category = Source, meta = (DataflowInput))
	int32 UVChannel = 0;

	/** Name of the attribute to to transfer the texture color to */
	UPROPERTY(EditAnywhere, Category = Target, meta = (DataflowInput, DataflowOutput, DataflowPassthrough = AttributeName))
	FString AttributeName;

	/** Vertex group to use */
	UPROPERTY(EditAnywhere, Category = Target)
	FScalarVertexPropertyGroup VertexGroup;

	/** transfers texture data to vertices using Parallelfor */
	UPROPERTY(EditAnywhere, Category = Performance)
	bool bMultithreadProcessing = true;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	virtual FAttributeKey GetVertexAttributeToVisualize(UE::Dataflow::FContext& Context, const FName OutputName, const FName RenderGroup) const override;

	virtual void PostSerialize(const FArchive& Ar) override;
	virtual void OnPropertyChanged(UE::Dataflow::FContext& Context, const FPropertyChangedEvent& InPropertyChangedEvent) override;
	void StartListeningToTextureChanges();
	void StopListeningToTextureChanges();
	void OnTextureChanged(UObject* Object, FPropertyChangedEvent& Event);
	FDelegateHandle TextureChangeDelegateHandle;
};

namespace UE::Dataflow
{
	void RegisterTextureToAttributeNodes();
}

