// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowToolNode.h"

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

uint64 FDataflowToolNodeSnapshot::GetAllocatedSize() const
{
	return Data.GetAllocatedSize();
}

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

void FDataflowToolNodeSnapshotSet::NotifyChanged()
{
	if (OnSnapshotSetChanged.IsBound())
	{
		OnSnapshotSetChanged.Broadcast(*this);
	}
}

TConstArrayView<FDataflowToolNodeSnapshot> FDataflowToolNodeSnapshotSet::GetSnapshots() const
{
	return Snapshots;
}

const FDataflowToolNodeSnapshot* FDataflowToolNodeSnapshotSet::GetActiveSnapshot() const
{
	if (Snapshots.IsValidIndex(ActiveSnapshot))
	{
		return &Snapshots[ActiveSnapshot];
	}
	return nullptr;
	
}

void FDataflowToolNodeSnapshotSet::SetActiveSnapshot(int32 ActiveIndex, bool bNotify)
{
	if (Snapshots.IsValidIndex(ActiveIndex))
	{
		if (ActiveSnapshot != ActiveIndex)
		{
			ActiveSnapshot = ActiveIndex;
			if (bNotify)
			{
				NotifyChanged();
			}
		}
	}
}

FDataflowToolNodeSnapshot& FDataflowToolNodeSnapshotSet::AddSnapshot(bool bNotify)
{
	// if all snapshots are lock we will exceed this number 
	constexpr int32 MaxDesiredNumberOfSnapshots = 5;
	if (Snapshots.Num() >= MaxDesiredNumberOfSnapshots)
	{
		RemoveSnapshot(FindOldestUnlockedSnapShot(), /*bNotify*/false);
	}
	FDataflowToolNodeSnapshot& NewSnapshot = Snapshots.AddDefaulted_GetRef();
	NewSnapshot.Name = TEXT("Tool Snapshot");
	ActiveSnapshot = (Snapshots.Num() - 1);
	if (bNotify)
	{
		NotifyChanged();
	}
	return NewSnapshot;
}

bool FDataflowToolNodeSnapshotSet::RemoveSnapshot(int32 Index, bool bNotify)
{
	if (Snapshots.IsValidIndex(Index))
	{
		Snapshots.RemoveAt(Index);
		if (ActiveSnapshot == Index)
		{
			ActiveSnapshot = Snapshots.IsEmpty() ? INDEX_NONE : (Snapshots.Num() - 1);
		}
		else if (ActiveSnapshot > Index)
		{
			ActiveSnapshot = ActiveSnapshot - 1;
		}
		if (bNotify)
		{
			NotifyChanged();
		}
		return true;
	}
	return false;
}

int32 FDataflowToolNodeSnapshotSet::FindOldestUnlockedSnapShot()
{
	int32 OldestSnapshotIndex = INDEX_NONE;
	FDateTime OldestSnapshotDate = FDateTime::MaxValue();
	const int32 NumSnapShots = Snapshots.Num();
	for (int32 Index = 0; Index < NumSnapShots; ++Index)
	{
		const FDataflowToolNodeSnapshot& Snapshot = Snapshots[Index];
		if (Snapshot.GetDate() < OldestSnapshotDate && !Snapshot.IsLocked())
		{
			OldestSnapshotIndex = Index;
			OldestSnapshotDate = Snapshot.GetDate();
		}
	}
	return OldestSnapshotIndex;
}

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FDataflowToolNode::FDataflowToolNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterNotifyChanged();
}

FDataflowToolNode::~FDataflowToolNode()
{
	UnregisterNotifyChanged();
}

void FDataflowToolNode::RegisterNotifyChanged()
{
	UnregisterNotifyChanged();
	OnNotifyChangedHandle = Snapshots.OnSnapshotSetChanged.AddRaw(this, &FDataflowToolNode::OnSnapshotSetChanged);
}

void FDataflowToolNode::UnregisterNotifyChanged()
{
	if (OnNotifyChangedHandle.IsValid())
	{
		Snapshots.OnSnapshotSetChanged.Remove(OnNotifyChangedHandle);
	}
}

void FDataflowToolNode::OnSnapshotSetChanged(FDataflowToolNodeSnapshotSet& Set)
{
	Invalidate();
}

FDataflowToolNodeSnapshot& FDataflowToolNode::AddSnapshot()
{
	return Snapshots.AddSnapshot();
}

bool FDataflowToolNode::RemoveSnapshot(int32 Index)
{
	return Snapshots.RemoveSnapshot(Index);
}

void FDataflowToolNode::SwapSnapshots(FDataflowToolNodeSnapshotSet& InOutSnapshots)
{
	UnregisterNotifyChanged();
	Swap(Snapshots, InOutSnapshots);
	RegisterNotifyChanged();
}

TConstArrayView<FDataflowToolNodeSnapshot> FDataflowToolNode::GetSnapshots() const
{
	return Snapshots.GetSnapshots();
}

const FDataflowToolNodeSnapshot* FDataflowToolNode::GetActiveSnapshot() const
{
	return Snapshots.GetActiveSnapshot();
}

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FDataflowToolNode::FSnapshotToolChange::FSnapshotToolChange::FSnapshotToolChange(FDataflowToolNode& Node)
	: WeakNode(StaticCastWeakPtr<FDataflowToolNode>(Node.AsWeak()))
	, SavedSnapshots(Node.Snapshots)
{}

FString FDataflowToolNode::FSnapshotToolChange::ToString() const
{
	return TEXT("FDataflowToolNode::FSnapshotToolChange");
}

void FDataflowToolNode::FSnapshotToolChange::Apply(UObject* Object)
{
	if (TSharedPtr<FDataflowToolNode> Node = WeakNode.Pin())
	{
		SwapApplyRevert(Object, *Node);
	}
}

void FDataflowToolNode::FSnapshotToolChange::Revert(UObject* Object)
{
	if (TSharedPtr<FDataflowToolNode> Node = WeakNode.Pin())
	{
		SwapApplyRevert(Object, *Node);
	}
}

void FDataflowToolNode::FSnapshotToolChange::SwapApplyRevert(UObject* Object, FDataflowToolNode& Node)
{
	Node.SwapSnapshots(SavedSnapshots);
	Node.Invalidate();
}