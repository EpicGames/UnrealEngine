// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowTools.h"
#include "Dataflow/DataflowNode.h"
//#include "Widgets/Notifications/SNotificationList.h"
//#include "Framework/Notifications/NotificationManager.h"

DEFINE_LOG_CATEGORY(LogDataflowNodes);

namespace UE::Dataflow
{
	void FDataflowTools::LogAndToastWarning(const FDataflowNode& DataflowNode, const FText& Headline, const FText& Details)
	{
		static const FTextFormat TextFormat = FTextFormat::FromString(TEXT("{0}: {1}\n{2}"));
		const FText NodeName = FText::FromName(DataflowNode.GetName());
		const FText Text = FText::Format(TextFormat, NodeName, Headline, Details);

		// @todo(Dataflow) : Add error suppurt during evaluation
		//FNotificationInfo NotificationInfo(Text);
		//NotificationInfo.ExpireDuration = 5.0f;
		//FSlateNotificationManager::Get().AddNotification(NotificationInfo);

		UE_LOGF(LogDataflowNodes, Display, "%ls", *Text.ToString());
	}

	void FDataflowTools::MakeCollectionName(FString& InOutString)
	{
		InOutString = SlugStringForValidName(InOutString, TEXT("_")).Replace(TEXT("\\"), TEXT("_"));
		bool bCharsWereRemoved;
		do { InOutString.TrimCharInline(TEXT('_'), &bCharsWereRemoved); } while (bCharsWereRemoved);
	}

}  // End namespace UE::Dataflow
