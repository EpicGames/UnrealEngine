// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Logging/LogMacros.h"

struct FManagedArrayCollection;
struct FDataflowNode;
class FString;

namespace UE::Dataflow
{
	/**
	* Dataflow Node Tools
	*/
	struct FDataflowTools
	{
		static void LogAndToastWarning(const FDataflowNode& DataflowNode, const FText& Headline, const FText& Details);

		/**
		 * Turn a string into a valid collection group or attribute name.
		 * The resulting name won't contains spaces and any other special characters as listed in
		 * INVALID_OBJECTNAME_CHARACTERS (currently "',/.:|&!~\n\r\t@#(){}[]=;^%$`).
		 * It will also have all leading underscore removed, as these names are reserved for internal use.
		 */
		static void MakeCollectionName(FString& InOutString);
	};
}  // End namespace UE::Dataflow

DECLARE_LOG_CATEGORY_EXTERN(LogDataflowNodes, Log, All);
