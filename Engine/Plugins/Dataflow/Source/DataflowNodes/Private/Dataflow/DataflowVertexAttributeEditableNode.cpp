// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowVertexAttributeEditableNode.h"

#include "Dataflow/DataflowInputOutput.h"
#include "Dataflow/DataflowTools.h"
#include "InteractiveToolChange.h"
#include "Misc/LazySingleton.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowVertexAttributeEditableNode)

FDataflowVertexAttributeEditableNode::FDataflowVertexAttributeEditableNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowToolNode(InParam, InGuid)
{}

void FDataflowVertexAttributeEditableNode::GetExtraVertexMapping(UE::Dataflow::FContext& Context, FName SelectedViewMode, TArray<int32>& OutMappingToWeight, TArray<TArray<int32>>& OutMappingFromWeight) const
{
	OutMappingToWeight.Reset();
	OutMappingFromWeight.Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// 
// Object encapsulating a change to the vertex attribute node's values. Used for Undo/Redo.
class FDataflowVertexAttributeEditableNode::FEditNodeToolChange final : public FDataflowToolNode::FSnapshotToolChange
{
public:
	FEditNodeToolChange(FDataflowVertexAttributeEditableNode& Node)
		: FDataflowToolNode::FSnapshotToolChange(Node)
	{
	}
};

TUniquePtr<FToolCommandChange> FDataflowVertexAttributeEditableNode::MakeEditNodeToolChange()
{
	return MakeUnique<FDataflowVertexAttributeEditableNode::FEditNodeToolChange>(*this);
}
