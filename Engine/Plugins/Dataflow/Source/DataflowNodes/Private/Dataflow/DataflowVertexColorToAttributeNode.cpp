// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowVertexColorToAttributeNode.h"

#include "Dataflow/DataflowNodeFactory.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/Facades/CollectionMeshFacade.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowVertexColorToAttributeNode)

#define LOCTEXT_NAMESPACE "DataflowVertexColorToAttributeNode"

namespace UE::Dataflow
{
	void RegisterVertexColorToAttributeNodes()
	{
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowVertexColorToAttributeNode);
	}
};

FDataflowVertexColorToAttributeNode::FDataflowVertexColorToAttributeNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&ColorChannel);
	RegisterInputConnection(&AttributeName);
	RegisterInputConnection(&ScalingFactor);
	RegisterOutputConnection(&Collection, &Collection);
}

void FDataflowVertexColorToAttributeNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Collection))
	{
		const FName InAttributeName = FName(GetValue(Context, &AttributeName, AttributeName));
		if (InAttributeName.IsNone())
		{
			Context.Error(LOCTEXT("NoAttributeName_Msg", "Attribute name is empty, data transfer from vertex color will be skipped"), this, Out);
			SafeForwardInput(Context, &Collection, &Collection);
			return;
		}

		FManagedArrayCollection InCollection = GetValue(Context, &Collection, Collection);

		const FName InVertexGroupName = VertexGroup.Name;
		if (!InCollection.HasGroup(InVertexGroupName))
		{
			Context.Error(LOCTEXT("InvalidVertexGroup_Msg", "Vertex group does not exists, data transfer from vertex color will be skipped"), this, Out);
			SafeForwardInput(Context, &Collection, &Collection);
			return;
		}

		// Create or get the new attribute 
		TManagedArray<float>& TargetAttribute = InCollection.AddAttribute<float>(InAttributeName, InVertexGroupName);
		TargetAttribute.Fill(0);

		// Get the vertex color attribute (we have geometry collection or cloth collection that store it differently)
		const TManagedArray<FLinearColor>* VertexColorAttribute = InCollection.FindAttribute<FLinearColor>(FGeometryCollection::ColorAttribute, InVertexGroupName);
		if (!VertexColorAttribute)
		{
			// cannot really include cloth here so we need to directly serach for the vertex color attribute in the render vertices ( eventually we may converge to use only geometry collections )
			VertexColorAttribute = InCollection.FindAttribute<FLinearColor>("RenderColor", InVertexGroupName);
		}
		if (!VertexColorAttribute)
		{
			Context.Warning(LOCTEXT("NoVertexColor_Msg", "No vertex color could be found in this collection, no data will be transfered and the attribute will be inititialized to zeros"), this, Out);
		}
		else
		{
			const EDataflowImageChannel InColorChannel = GetValue(Context, &ColorChannel, ColorChannel);
			const float InScalingFactor = FMath::Clamp(GetValue(Context, &ScalingFactor, ScalingFactor), 0.f, 1.f);

			const int32 NumVertices = FMath::Min(TargetAttribute.Num(), VertexColorAttribute->Num());
			for (int32 Index = 0; Index < NumVertices; ++Index)
			{
				const FLinearColor Color = (*VertexColorAttribute)[Index];
				float Value = 0;
				switch (InColorChannel)
				{
				case EDataflowImageChannel::Red: 
					Value = Color.R;
					break;
				case EDataflowImageChannel::Green:
					Value = Color.G;
					break;
				case EDataflowImageChannel::Blue:
					Value = Color.B;
					break;
				case EDataflowImageChannel::Alpha:
					Value = Color.A;
					break;
				}
				TargetAttribute[Index] = Value * InScalingFactor;
			}
		}

		SetValue(Context, MoveTemp(InCollection), &Collection);
	}
}

#undef LOCTEXT_NAMESPACE 

