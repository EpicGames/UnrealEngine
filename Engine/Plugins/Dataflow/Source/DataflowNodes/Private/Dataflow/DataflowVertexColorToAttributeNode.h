// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowNode.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "Dataflow/DataflowCollectionAddScalarVertexPropertyNode.h"

#include "Dataflow/DataflowImage.h"

#include "DataflowVertexColorToAttributeNode.generated.h"

/**
* Transfer data from a vertex color to a an attribute in a collection
*/
USTRUCT()
struct FDataflowVertexColorToAttributeNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowVertexColorToAttributeNode, "VertexColorToAttribute", "Dataflow", "Transfer")

public:
	FDataflowVertexColorToAttributeNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	/** Collection to transfer the attribute data to */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = Collection))
	FManagedArrayCollection Collection;

	/** Color channel to use as attribute value */
	UPROPERTY(EditAnywhere, Category = Source, meta = (DataflowInput))
	EDataflowImageChannel ColorChannel = EDataflowImageChannel::Red;

	/** Name of the attribute to to transfer the vertex color to */
	UPROPERTY(EditAnywhere, Category = Target, meta = (DataflowInput))
	FString AttributeName;

	/** Scaling factor applied to vertex colours (in range 0-1)  */
	UPROPERTY(EditAnywhere, Category = Target, meta = (DataflowInput, UIMin = 0.0, UIMax = 1.0, ClampMin = 0.0, ClampMax = 1.0))
	float ScalingFactor = 1.0f;

	/** Vertex group to use */
	UPROPERTY(EditAnywhere, Category = Target)
	FScalarVertexPropertyGroup VertexGroup;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

namespace UE::Dataflow
{
	void RegisterVertexColorToAttributeNodes();
}

