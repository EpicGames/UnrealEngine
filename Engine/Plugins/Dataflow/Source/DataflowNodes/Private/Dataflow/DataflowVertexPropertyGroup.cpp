// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowVertexPropertyGroup.h"
#include "Misc/LazySingleton.h"

#define LOCTEXT_NAMESPACE "DataflowVertexPropertyGroup"

FDataflowAddScalarVertexPropertyCallbackRegistry& FDataflowAddScalarVertexPropertyCallbackRegistry::Get()
{
	return TLazySingleton<FDataflowAddScalarVertexPropertyCallbackRegistry>::Get();
}

void FDataflowAddScalarVertexPropertyCallbackRegistry::RegisterCallbacks(TUniquePtr<IDataflowAddScalarVertexPropertyCallbacks>&& Callbacks)
{
	AllCallbacks.Add(Callbacks->GetName(), MoveTemp(Callbacks));
}

void FDataflowAddScalarVertexPropertyCallbackRegistry::DeregisterCallbacks(const FName& CallbacksName)
{
	AllCallbacks.Remove(CallbacksName);
}

TArray<FName> FDataflowAddScalarVertexPropertyCallbackRegistry::GetTargetGroupNames() const
{
	TArray<FName> UniqueNames;

	for (const TPair<FName, TUniquePtr<IDataflowAddScalarVertexPropertyCallbacks>>& CallbacksEntry : AllCallbacks)
	{
		for (const FName& GroupName : CallbacksEntry.Value->GetTargetGroupNames())
		{
			UniqueNames.AddUnique(GroupName);
		}
	}
	return UniqueNames;
}

FDataflowAddScalarVertexPropertyCallbackRegistry::FTargetGroupInfo FDataflowAddScalarVertexPropertyCallbackRegistry::GetTargetGroupInfo(FName TargetGroup) const
{
	TArray<FTargetGroupInfo> OutInfos;
	for (const TPair<FName, TUniquePtr<IDataflowAddScalarVertexPropertyCallbacks>>& CallbacksEntry : AllCallbacks)
	{
		CallbacksEntry.Value->GetTargetGroupInfos(OutInfos);
	}
	for (const FTargetGroupInfo& Info : OutInfos)
	{
		if (Info.TargetGroup == TargetGroup)
		{
			return Info;
		}
	}
	return {};
}

TArray<UE::Dataflow::FRenderingParameter> FDataflowAddScalarVertexPropertyCallbackRegistry::GetRenderingParameters() const
{
	TArray<UE::Dataflow::FRenderingParameter> UniqueParameters;

	for (const TPair<FName, TUniquePtr<IDataflowAddScalarVertexPropertyCallbacks>>& CallbacksEntry : AllCallbacks)
	{
		for (const UE::Dataflow::FRenderingParameter& RenderingParameter : CallbacksEntry.Value->GetRenderingParameters())
		{
			UniqueParameters.AddUnique(RenderingParameter);
		}
	}
	return UniqueParameters;
}

TArray<UE::Dataflow::FRenderingParameter> FDataflowAddScalarVertexPropertyCallbackRegistry::GetRenderingParameters(const FName& TargetGroup) const
{
	TArray<UE::Dataflow::FRenderingParameter> UniqueParameters;

	for (const TPair<FName, TUniquePtr<IDataflowAddScalarVertexPropertyCallbacks>>& CallbacksEntry : AllCallbacks)
	{
		const TArray<UE::Dataflow::FRenderingParameter> RenderingParameters = CallbacksEntry.Value->GetRenderingParameters();
		const TArray<FName> TargetGroups = CallbacksEntry.Value->GetTargetGroupNames();
		if (RenderingParameters.Num() == TargetGroups.Num())
		{
			for (int32 TargetIndex = 0, NumTargets = TargetGroups.Num(); TargetIndex < NumTargets; ++TargetIndex)
			{
				if (TargetGroups[TargetIndex] == TargetGroup)
				{
					UniqueParameters.AddUnique(RenderingParameters[TargetIndex]);
				}
			}
		}
		else if (TargetGroups.Find(TargetGroup) != INDEX_NONE)
		{
			for (const UE::Dataflow::FRenderingParameter& RenderingParameter : CallbacksEntry.Value->GetRenderingParameters())
			{
				UniqueParameters.AddUnique(RenderingParameter);
			}
		}
	}
	return UniqueParameters;
}


#undef LOCTEXT_NAMESPACE 

