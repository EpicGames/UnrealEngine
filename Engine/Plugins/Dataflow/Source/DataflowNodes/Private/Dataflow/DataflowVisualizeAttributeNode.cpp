// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowVisualizeAttributeNode.h"

#include "Dataflow/DataflowNodeFactory.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/Facades/CollectionMeshFacade.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowVisualizeAttributeNode)

#define LOCTEXT_NAMESPACE "DataflowVisualizeAttributeNode"

namespace UE::Dataflow
{
	void RegisterVisualizeAttributeNodes()
	{
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowVisualizeAttributeNode);
	}
};

FDataflowVisualizeAttributeNode::FDataflowVisualizeAttributeNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&AttributeName);
	RegisterOutputConnection(&Collection, &Collection);
}

void FDataflowVisualizeAttributeNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Collection))
	{
		// All the checks below are just for validation and informing the user about inputs
		const FName InAttributeName = FName(GetValue(Context, &AttributeName, AttributeName));
		if (InAttributeName.IsNone())
		{
			Context.Error(LOCTEXT("NoAttributeName_Msg", "Attribute name is empty, no data will be displayed"), this, Out);
			SafeForwardInput(Context, &Collection, &Collection);
			return;
		}

		FManagedArrayCollection InCollection = GetValue(Context, &Collection, Collection);

		const FName InVertexGroupName = VertexGroup.Name;
		if (!InCollection.HasGroup(InVertexGroupName))
		{
			Context.Error(LOCTEXT("InvalidVertexGroup_Msg", "Vertex group does not exists, no data will be displayed"), this, Out);
			SafeForwardInput(Context, &Collection, &Collection);
			return;
		}

		if (!InCollection.HasAttribute(InAttributeName, InVertexGroupName))
		{
			Context.Error(LOCTEXT("NoAttribute_Msg", "Attribute could not be found, no data will be displayed"), this, Out);
			SafeForwardInput(Context, &Collection, &Collection);
			return;
		}

		SafeForwardInput(Context, &Collection, &Collection);
	}
}

FDataflowNode::FAttributeKey FDataflowVisualizeAttributeNode::GetVertexAttributeToVisualize(UE::Dataflow::FContext& Context, const FName OutputName, const FName RenderGroup) const
{
	return FAttributeKey
	{
		.AttributeName = FName(GetValue(Context, &AttributeName, AttributeName)),
		.GroupName = VertexGroup.Name,
	};
}

#undef LOCTEXT_NAMESPACE 

