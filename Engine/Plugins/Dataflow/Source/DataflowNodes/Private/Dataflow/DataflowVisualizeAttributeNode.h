// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowNode.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "Dataflow/DataflowCollectionAddScalarVertexPropertyNode.h"

#include "DataflowVisualizeAttributeNode.generated.h"

/**
* Node to visualize a specific attribute in the construction viewport as vertex color
* it supports a number of attribute type ( decimal, vector or boolean )
*/
USTRUCT()
struct FDataflowVisualizeAttributeNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowVisualizeAttributeNode, "VisualizeAttribute", "Dataflow", "Debug Render Vertex Color View")

public:
	FDataflowVisualizeAttributeNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	/** Collection to read the data from  */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = Collection))
	FManagedArrayCollection Collection;

	/** Name of the attribute to visualize */
	UPROPERTY(EditAnywhere, Category = Target, meta = (DataflowInput))
	FString AttributeName;

	/** Vertex group matching the attribute */
	UPROPERTY(EditAnywhere, Category = Target)
	FScalarVertexPropertyGroup VertexGroup;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	virtual FAttributeKey GetVertexAttributeToVisualize(UE::Dataflow::FContext& Context, const FName OutputName, const FName RenderGroup) const override;
};

namespace UE::Dataflow
{
	void RegisterVisualizeAttributeNodes();
}

