// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/SamplerNodes/DataflowAbsSamplerNode.h"
#include "Dataflow/SamplerNodes/DataflowSamplerFunctions.h"
#include "Dataflow/DataflowNodeFactory.h"

FBox FDataflowAbsFloatSampler::GetRenderBounds() const
{
	if (Sampler.IsValid())
	{
		return Sampler->GetRenderBounds();
	}

	return FBox(FVector(-50.0), FVector(50.0));
}

void FDataflowAbsFloatSampler::Sample(TArrayView<const FVector3f> Positions, TArrayView<float> OutValues) const
{
	if (Sampler.IsValid() && Positions.Num() == OutValues.Num())
	{
		TArray<float> OutValueFromInputSampler;
		OutValueFromInputSampler.SetNumUninitialized(OutValues.Num());
	
		Sampler->Sample(Positions, OutValueFromInputSampler);
	
		for (int32 Idx = 0; Idx < OutValues.Num(); ++Idx)
		{
			OutValues[Idx] = FMath::Abs(OutValueFromInputSampler[Idx]);
		}
	}
	else
	{
		for (int32 Idx = 0; Idx < OutValues.Num(); ++Idx)
		{
			OutValues[Idx] = 0.f;
		}
	}
}

FBox FDataflowAbsVectorSampler::GetRenderBounds() const
{
	if (Sampler.IsValid())
	{
		return Sampler->GetRenderBounds();
	}

	return FBox(FVector(-50.0), FVector(50.0));
}

void FDataflowAbsVectorSampler::Sample(TArrayView<const FVector3f> Positions, TArrayView<FVector3f> OutValues) const
{
	if (Sampler.IsValid() && Positions.Num() == OutValues.Num())
	{
		TArray<FVector3f> OutValueFromInputSampler;
		OutValueFromInputSampler.SetNumUninitialized(OutValues.Num());
	
		Sampler->Sample(Positions, OutValueFromInputSampler);
	
		for (int32 Idx = 0; Idx < OutValues.Num(); ++Idx)
		{
			OutValues[Idx] = FVector3f(FMath::Abs(OutValueFromInputSampler[Idx].X), 
				FMath::Abs(OutValueFromInputSampler[Idx].Y), 
				FMath::Abs(OutValueFromInputSampler[Idx].Z));
		}
	}
	else
	{
		for (int32 Idx = 0; Idx < OutValues.Num(); ++Idx)
		{
			OutValues[Idx] = FVector3f::ZeroVector;
		}
	}
}

FDataflowAbsSamplerNode::FDataflowAbsSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	static const FName MainTypeGroup("Main");
	
	RegisterInputConnection(&Sampler)
		.SetTypeDependencyGroup(MainTypeGroup);
	RegisterOutputConnection(&Sampler)
		.SetPassthroughInput(&Sampler)
		.SetTypeDependencyGroup(MainTypeGroup);
}

void FDataflowAbsSamplerNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Sampler))
	{
		if (const FDataflowFloatSampler* FloatSampler = GetValue(Context, &Sampler).TryGet <FDataflowFloatSampler>())
		{
			TSharedRef<FDataflowAbsFloatSampler> Impl = MakeShared<FDataflowAbsFloatSampler>();

			Impl->Sampler = FloatSampler->GetImpl();

			FDataflowFloatSampler OutSampler(Impl.ToSharedPtr());

			SetValue(Context, OutSampler, &Sampler);
			return;
		}
		else if (const FDataflowVectorSampler* VectorSampler = GetValue(Context, &Sampler).TryGet<FDataflowVectorSampler>())
		{
			TSharedRef<FDataflowAbsVectorSampler> Impl = MakeShared<FDataflowAbsVectorSampler>();
		
			Impl->Sampler = VectorSampler->GetImpl();
		
			FDataflowVectorSampler OutSampler(Impl.ToSharedPtr());

			SetValue(Context, OutSampler, &Sampler);
			return;
		}

		SafeForwardInput(Context, &Sampler, &Sampler);
	}
}


