// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowNode.h"
#include "Dataflow/SamplerNodes/DataflowSamplerTypes.h"

#include "DataflowAbsSamplerNode.generated.h"

/***********************************************************************************************************************
 * No Input Sampler nodes
 ***********************************************************************************************************************/
USTRUCT()
struct FDataflowAbsFloatSampler : public FDataflowFloatSamplerBase
{
	GENERATED_BODY();

	virtual ~FDataflowAbsFloatSampler() override = default;
	virtual void Sample(TArrayView<const FVector3f> Positions, TArrayView<float> OutValues) const override;
	virtual FBox GetRenderBounds() const override;

public:
	/**  */
	TSharedPtr<const FDataflowFloatSamplerBase> Sampler;
};

USTRUCT()
struct FDataflowAbsVectorSampler : public FDataflowVectorSamplerBase
{
	GENERATED_BODY();

	virtual ~FDataflowAbsVectorSampler() override = default;
	virtual void Sample(TArrayView<const FVector3f> Positions, TArrayView<FVector3f> OutValues) const override;
	virtual FBox GetRenderBounds() const override;

public:
	/**  */
	TSharedPtr<const FDataflowVectorSamplerBase> Sampler;
};

/**
 * Output an abs float sampler
 */
USTRUCT(meta = (Icon = "Icons.EyeDropper"))
struct FDataflowAbsSamplerNode : public FDataflowNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowAbsSamplerNode, "Abs Sampler", "Samplers", "Absolute")

public:
	FDataflowAbsSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = Sampler))
	FDataflowSamplerTypes Sampler;
	
	void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};
