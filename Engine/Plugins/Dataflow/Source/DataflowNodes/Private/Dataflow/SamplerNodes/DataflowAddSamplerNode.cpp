// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/SamplerNodes/DataflowAddSamplerNode.h"
#include "Dataflow/DataflowNodeFactory.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowAddSamplerNode)

#define LOCTEXT_NAMESPACE "DataflowAddSamplerNode"

FBox FDataflowAddFloatSampler::GetRenderBounds() const
{
	FBox RenderBounds = FBox(ForceInit);

	int32 NumAddedRenderBounds = 0;

	for (const TSharedPtr<const FDataflowFloatSamplerBase>& Sampler : Samplers)
	{
		if (Sampler.IsValid())
		{
			RenderBounds += Sampler->GetRenderBounds();
		}

		NumAddedRenderBounds++;
	}

	if (NumAddedRenderBounds)
	{
		return RenderBounds;
	}
	else
	{
		return FBox(FVector(-50.0), FVector(50.0));
	}
}

void FDataflowAddFloatSampler::Sample(TArrayView<const FVector3f> Positions, TArrayView<float> OutValues) const
{
	for (int32 Index = 0; Index < OutValues.Num(); ++Index)
	{
		OutValues[Index] = 0;
	}

	if (Positions.Num() == OutValues.Num())
	{
		TArray<float> OneSamplerValues;

		for (const TSharedPtr<const FDataflowFloatSamplerBase>& Sampler : Samplers)
		{
			OneSamplerValues.Init(0, OutValues.Num());
			Sampler->Sample(Positions, OneSamplerValues);
			for (int32 Index = 0; Index < OutValues.Num(); ++Index)
			{
				OutValues[Index] += OneSamplerValues[Index];
			}
		}
	}
}

FBox FDataflowAddVectorSampler::GetRenderBounds() const
{
	FBox RenderBounds = FBox(ForceInit);

	int32 NumAddedRenderBounds = 0;

	for (const TSharedPtr<const FDataflowVectorSamplerBase>& Sampler : Samplers)
	{
		if (Sampler.IsValid())
		{
			RenderBounds += Sampler->GetRenderBounds();
		}

		NumAddedRenderBounds++;
	}

	if (NumAddedRenderBounds)
	{
		return RenderBounds;
	}
	else
	{
		return FBox(FVector(-50.0), FVector(50.0));
	}
}

void FDataflowAddVectorSampler::Sample(TArrayView<const FVector3f> Positions, TArrayView<FVector3f> OutValues) const
{
	for (int32 Index = 0; Index < OutValues.Num(); ++Index)
	{
		OutValues[Index] = FVector3f::ZeroVector;
	}

	if (Positions.Num() == OutValues.Num())
	{
		TArray<FVector3f> OneSamplerValues;

		for (const TSharedPtr<const FDataflowVectorSamplerBase>& Sampler : Samplers)
		{
			OneSamplerValues.Init(FVector3f::ZeroVector, OutValues.Num());
			Sampler->Sample(Positions, OneSamplerValues);
			for (int32 Index = 0; Index < OutValues.Num(); ++Index)
			{
				OutValues[Index] += OneSamplerValues[Index];
			}
		}
	}
}

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FDataflowAddSamplerNode::FDataflowAddSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowMultiInputSamplerNodeBase(InParam, InGuid)
{
	RegisterInitialConnections();
}

void FDataflowAddSamplerNode::EvaluateSampler(UE::Dataflow::FContext& Context, FDataflowSamplerTypes::FStorageType& OutSampler) const
{
	// use the first sampler as a reference
	const int32 NumSamplers = GetNumSamplerInputs();
	if (NumSamplers == 0)
	{
		return;
	}

	const FDataflowSamplerTypes::FStorageType& FirstSampler = GetSamplerInput(Context, 0);
	if (FirstSampler.TryGet<FDataflowFloatSampler>())
	{
		TSharedPtr<FDataflowAddFloatSampler> OutImpl = MakeShared<FDataflowAddFloatSampler>();
		for (int32 Index = 0; Index < NumSamplers; ++Index)
		{
			if (const FDataflowFloatSampler* FloatSampler = GetSamplerInput(Context, Index).TryGet<FDataflowFloatSampler>())
			{
				if (TSharedPtr<const FDataflowFloatSamplerBase> SamplerImpl = FloatSampler->GetImpl())
				{
					OutImpl->Samplers.Add(SamplerImpl);
				}
			}
		}
		OutSampler = FDataflowFloatSampler(OutImpl);
	}
	else if (FirstSampler.TryGet<FDataflowVectorSampler>())
	{
		TSharedPtr<FDataflowAddVectorSampler> OutImpl = MakeShared<FDataflowAddVectorSampler>();
		for (int32 Index = 0; Index < NumSamplers; ++Index)
		{
			if (const FDataflowVectorSampler* VectorSampler = GetSamplerInput(Context, Index).TryGet<FDataflowVectorSampler>())
			{
				if (TSharedPtr<const FDataflowVectorSamplerBase> SamplerImpl = VectorSampler->GetImpl())
				{
					OutImpl->Samplers.Add(SamplerImpl);
				}
			}
		}
		OutSampler = FDataflowVectorSampler(OutImpl);
	}

}

#undef LOCTEXT_NAMESPACE 
