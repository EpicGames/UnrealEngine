// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 
#include "Dataflow/SamplerNodes/DataflowSamplerTypes.h"
#include "Dataflow/SamplerNodes/DataflowSamplerMultiInput.h"

#include "DataflowAddSamplerNode.generated.h"

USTRUCT()
struct FDataflowAddFloatSampler : public FDataflowFloatSamplerBase
{
	GENERATED_BODY();
	virtual ~FDataflowAddFloatSampler() override = default;
	virtual void Sample(TArrayView<const FVector3f> Positions, TArrayView<float> OutValues) const override;
	virtual FBox GetRenderBounds() const override;

public:
	/** list of samplers to add results from */
	TArray<TSharedPtr<const FDataflowFloatSamplerBase>> Samplers;
};

USTRUCT()
struct FDataflowAddVectorSampler : public FDataflowVectorSamplerBase
{
	GENERATED_BODY();
	virtual ~FDataflowAddVectorSampler() override = default;
	virtual void Sample(TArrayView<const FVector3f> Positions, TArrayView<FVector3f> OutValues) const override;
	virtual FBox GetRenderBounds() const override;

public:
	/** list of samplers to add results from */
	TArray<TSharedPtr<const FDataflowVectorSamplerBase>> Samplers;
};

/**
* add sampler results together
*/
USTRUCT(meta = (Icon = "Icons.EyeDropper"))
struct FDataflowAddSamplerNode : public FDataflowMultiInputSamplerNodeBase
{
	GENERATED_USTRUCT_BODY()

	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowAddSamplerNode, "Add Sampler", "Samplers", "+ Addition Sum")

public:
	FDataflowAddSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
	virtual void EvaluateSampler(UE::Dataflow::FContext& Context, FDataflowSamplerTypes::FStorageType& OutSampler) const override;
};

