// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/SamplerNodes/DataflowClampSamplerNode.h"
#include "Dataflow/SamplerNodes/DataflowSamplerFunctions.h"
#include "Dataflow/DataflowNodeFactory.h"

FBox FDataflowClampFloatSampler::GetRenderBounds() const
{
	if (Sampler.IsValid())
	{
		return Sampler->GetRenderBounds();
	}

	return FBox(FVector(-50.0), FVector(50.0));
}

void FDataflowClampFloatSampler::Sample(TArrayView<const FVector3f> Positions, TArrayView<float> OutValues) const
{
	if (Sampler.IsValid() && Positions.Num() == OutValues.Num())
	{
		TArray<float> OutValueFromInputSampler;
		OutValueFromInputSampler.SetNumUninitialized(OutValues.Num());
	
		Sampler->Sample(Positions, OutValueFromInputSampler);
	
		for (int32 Idx = 0; Idx < OutValues.Num(); ++Idx)
		{
			OutValues[Idx] = FMath::Clamp(OutValueFromInputSampler[Idx], Min, Max);
		}
	}
	else
	{
		for (int32 Idx = 0; Idx < OutValues.Num(); ++Idx)
		{
			OutValues[Idx] = 0.f;
		}
	}
}

FDataflowClampFloatSamplerNode::FDataflowClampFloatSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Sampler);
	RegisterInputConnection(&ClampSampler.Min, GET_MEMBER_NAME_CHECKED(FDataflowClampFloatSampler, Min));
	RegisterInputConnection(&ClampSampler.Max, GET_MEMBER_NAME_CHECKED(FDataflowClampFloatSampler, Max));
	RegisterOutputConnection(&Sampler, &Sampler);
}

void FDataflowClampFloatSamplerNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Sampler))
	{
		const FDataflowFloatSampler& FloatSampler = GetValue(Context, &Sampler);

		TSharedRef<FDataflowClampFloatSampler> Impl = MakeShared<FDataflowClampFloatSampler>();

		Impl->Sampler = FloatSampler.GetImpl();
		Impl->Min = GetValue(Context, &ClampSampler.Min);
		Impl->Max = GetValue(Context, &ClampSampler.Max);

		FDataflowFloatSampler OutSampler(Impl.ToSharedPtr());

		SetValue(Context, OutSampler, &Sampler);
	}
}


