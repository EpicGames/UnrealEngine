// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowNode.h"
#include "Dataflow/SamplerNodes/DataflowSamplerTypes.h"

#include "DataflowClampSamplerNode.generated.h"

USTRUCT()
struct FDataflowClampFloatSampler : public FDataflowFloatSamplerBase
{
	GENERATED_BODY();

	virtual ~FDataflowClampFloatSampler() override = default;
	virtual void Sample(TArrayView<const FVector3f> Positions, TArrayView<float> OutValues) const override;
	virtual FBox GetRenderBounds() const override;

public:
	/**  */
	UPROPERTY(EditAnywhere, Category = Clamp, meta = (DataflowInput))
	float Min = 0.f;
	
	/**  */
	UPROPERTY(EditAnywhere, Category = Clamp, meta = (DataflowInput))
	float Max = 1.f;
		
	/**  */
	TSharedPtr<const FDataflowFloatSamplerBase> Sampler;
};

/**
 * Output an Clamp float sampler
 */
USTRUCT(meta = (Icon = "Icons.EyeDropper"))
struct FDataflowClampFloatSamplerNode : public FDataflowNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowClampFloatSamplerNode, "Clamp Float Sampler", "Samplers", "")

public:
	FDataflowClampFloatSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = Sampler))
	FDataflowFloatSampler Sampler;
	
	UPROPERTY(EditAnywhere, Category = Sampler, meta = (ShowOnlyInnerProperties, SkipInDisplayNameChain))
	FDataflowClampFloatSampler ClampSampler;

	void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};
