// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/SamplerNodes/DataflowColorRampSamplerNode.h"
#include "Dataflow/SamplerNodes/DataflowSamplerFunctions.h"
#include "Dataflow/DataflowNodeFactory.h"

FBox FDataflowColorRampVectorSampler::GetRenderBounds() const
{
	if (Sampler.IsValid())
	{
		return Sampler->GetRenderBounds();
	}

	return FBox(FVector(-50.0), FVector(50.0));
}

void FDataflowColorRampVectorSampler::Sample(TArrayView<const FVector3f> Positions, TArrayView<FVector3f> OutValues) const
{
	if (Sampler.IsValid() && Positions.Num() == OutValues.Num())
	{
		TArray<float> OutValueFromInputSampler;
		OutValueFromInputSampler.SetNumUninitialized(OutValues.Num());

		Sampler->Sample(Positions, OutValueFromInputSampler);

		if (Max - Min < SMALL_NUMBER)
		{
			for (int32 Idx = 0; Idx < OutValues.Num(); ++Idx)
			{
				OutValues[Idx] = FVector3f::ZeroVector;
			}

			return;
		}

		for (int32 Idx = 0; Idx < OutValues.Num(); ++Idx)
		{

			float NormalizedValue = (OutValueFromInputSampler[Idx] - Min) / (Max - Min);
			NormalizedValue = FMath::Clamp(NormalizedValue, 0.f, 1.f);

			FLinearColor Color = ColorRamp.GetLinearColorValue(NormalizedValue);

			OutValues[Idx] = FVector3f(Color.R, Color.G, Color.B);
		}
	}
	else
	{
		for (int32 Idx = 0; Idx < OutValues.Num(); ++Idx)
		{
			OutValues[Idx] = FVector3f::ZeroVector;
		}
	}
}

FDataflowColorRampSamplerNode::FDataflowColorRampSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Sampler);
	RegisterOutputConnection(&ColorSampler);

	constexpr bool bOnlyRGB = true;
	ColorRampSampler.ColorRamp.SetColorAtTime(0.f, FColor::Purple, bOnlyRGB);
	ColorRampSampler.ColorRamp.SetColorAtTime(1.0f, FLinearColor::Yellow, bOnlyRGB);
}

void FDataflowColorRampSamplerNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&ColorSampler))
	{
		const FDataflowFloatSampler& InFloatSampler = GetValue(Context, &Sampler);

		TSharedRef<FDataflowColorRampVectorSampler> Impl = MakeShared<FDataflowColorRampVectorSampler>(ColorRampSampler);

		Impl->Sampler = InFloatSampler.GetImpl();

		FDataflowVectorSampler OutSampler(Impl.ToSharedPtr());

		SetValue(Context, OutSampler, &ColorSampler);
		return;
	}
}
