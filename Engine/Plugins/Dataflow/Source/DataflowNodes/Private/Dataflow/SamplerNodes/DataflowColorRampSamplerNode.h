// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowNode.h"
#include "Dataflow/SamplerNodes/DataflowSamplerTypes.h"
#include "Dataflow/DataflowImage.h"
#include "Curves/LinearColorRamp.h"

#include "DataflowColorRampSamplerNode.generated.h"

USTRUCT()
struct FDataflowColorRampVectorSampler : public FDataflowVectorSamplerBase
{
	GENERATED_BODY();

	virtual ~FDataflowColorRampVectorSampler() override = default;
	virtual void Sample(TArrayView<const FVector3f> Positions, TArrayView<FVector3f> OutValues) const override;
	virtual FBox GetRenderBounds() const override;

public:
	/** Attribute minimum value for coloring. This value represents the minimum color on the color ramp. */
	UPROPERTY(EditAnywhere, Category = "Color")
	float Min = 0.f;

	/** Attribute maximum value for coloring. This value represents the maximum color on the color ramp. */
	UPROPERTY(EditAnywhere, Category = "Color")
	float Max = 1.f;

	/** Color ramp specifiyng the color for displayed points using a single value attribute. Only used with FloatSampler input. */
	UPROPERTY(EditAnywhere, Category = "Color")
	FLinearColorRamp ColorRamp;

	TSharedPtr<const FDataflowFloatSamplerBase> Sampler;
};

/**
 *
 * ColorRamp Sampler
 * Input(s) : Float Sampler
 * Output(s): Vector sampler representing colors generated by the color ramp using the sampled float values
 *
 */
USTRUCT(meta = (Icon = "Icons.EyeDropper"))
struct FDataflowColorRampSamplerNode : public FDataflowNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowColorRampSamplerNode, "ColorRamp Sampler", "Samplers", "")

public:
	FDataflowColorRampSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	/** Sampler input */
	UPROPERTY(meta = (DataflowInput))
	FDataflowFloatSampler Sampler;

	/** Sampler output */
	UPROPERTY(meta = (DataflowOutput))
	FDataflowVectorSampler ColorSampler;

	UPROPERTY(EditAnywhere, Category = Sampler, meta = (ShowOnlyInnerProperties, SkipInDisplayNameChain))
	FDataflowColorRampVectorSampler ColorRampSampler;


	void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};
