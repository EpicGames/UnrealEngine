// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/SamplerNodes/DataflowCombineSamplerNode.h"
#include "Dataflow/SamplerNodes/DataflowSamplerFunctions.h"
#include "Dataflow/DataflowNodeFactory.h"

FBox FDataflowCombineVectorSampler::GetRenderBounds() const
{
	FBox RenderBounds = FBox(ForceInit);

	if (FloatSamplerX.IsValid() || FloatSamplerY.IsValid() || FloatSamplerZ.IsValid())
	{
		if (FloatSamplerX.IsValid())
		{
			RenderBounds += FloatSamplerX->GetRenderBounds();
		}

		if (FloatSamplerY.IsValid())
		{
			RenderBounds += FloatSamplerY->GetRenderBounds();
		}

		if (FloatSamplerZ.IsValid())
		{
			RenderBounds += FloatSamplerZ->GetRenderBounds();
		}

		return RenderBounds;
	}

	return FBox(FVector(-50.0), FVector(50.0));
}

void FDataflowCombineVectorSampler::Sample(TArrayView<const FVector3f> Positions, TArrayView<FVector3f> OutValues) const
{
	if (Positions.Num() == OutValues.Num())
	{
		TArray<float> OutValueFromFloatSamplerX, OutValueFromFloatSamplerY, OutValueFromFloatSamplerZ;

		OutValueFromFloatSamplerX.Init(0.f, OutValues.Num());
		OutValueFromFloatSamplerY.Init(0.f, OutValues.Num());
		OutValueFromFloatSamplerZ.Init(0.f, OutValues.Num());

		if (FloatSamplerX.IsValid())
		{
			FloatSamplerX->Sample(Positions, OutValueFromFloatSamplerX);
		}

		if (FloatSamplerY.IsValid())
		{
			FloatSamplerY->Sample(Positions, OutValueFromFloatSamplerY);
		}

		if (FloatSamplerZ.IsValid())
		{
			FloatSamplerZ->Sample(Positions, OutValueFromFloatSamplerZ);
		}

		for (int32 Idx = 0; Idx < OutValues.Num(); ++Idx)
		{
			OutValues[Idx] = FVector3f(OutValueFromFloatSamplerX[Idx], OutValueFromFloatSamplerY[Idx], OutValueFromFloatSamplerZ[Idx]);
		}
	}
	else
	{
		for (int32 Idx = 0; Idx < OutValues.Num(); ++Idx)
		{
			OutValues[Idx] = FVector3f::ZeroVector;
		}
	}
}

FDataflowCombineFloatSamplerNode::FDataflowCombineFloatSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&FloatSamplerX);
	RegisterInputConnection(&FloatSamplerY);
	RegisterInputConnection(&FloatSamplerZ);
	RegisterOutputConnection(&VectorSampler);
}

void FDataflowCombineFloatSamplerNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&VectorSampler))
	{
		const FDataflowFloatSampler& InFloatSamplerX = GetValue(Context, &FloatSamplerX);
		const FDataflowFloatSampler& InFloatSamplerY = GetValue(Context, &FloatSamplerY);
		const FDataflowFloatSampler& InFloatSamplerZ = GetValue(Context, &FloatSamplerZ);

		TSharedRef<FDataflowCombineVectorSampler> Impl = MakeShared<FDataflowCombineVectorSampler>();

		Impl->FloatSamplerX = InFloatSamplerX.GetImpl();
		Impl->FloatSamplerY = InFloatSamplerY.GetImpl();
		Impl->FloatSamplerZ = InFloatSamplerZ.GetImpl();

		FDataflowVectorSampler OutSampler(Impl.ToSharedPtr());

		SetValue(Context, OutSampler, &VectorSampler);
	}
}
