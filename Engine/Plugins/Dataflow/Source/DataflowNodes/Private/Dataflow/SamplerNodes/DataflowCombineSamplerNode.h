// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowNode.h"
#include "Dataflow/SamplerNodes/DataflowSamplerTypes.h"

#include "DataflowCombineSamplerNode.generated.h"

USTRUCT()
struct FDataflowCombineVectorSampler : public FDataflowVectorSamplerBase
{
	GENERATED_BODY();

	virtual ~FDataflowCombineVectorSampler() override = default;
	virtual void Sample(TArrayView<const FVector3f> Positions, TArrayView<FVector3f> OutValues) const override;
	virtual FBox GetRenderBounds() const override;

public:
	TSharedPtr<const FDataflowFloatSamplerBase> FloatSamplerX;

	TSharedPtr<const FDataflowFloatSamplerBase> FloatSamplerY;

	TSharedPtr<const FDataflowFloatSamplerBase> FloatSamplerZ;
};

/**
 *
 * Combine float sampler
 * Input(s):  3 float samplers (X, Y, Z) to combine into a single vector sampler
 * Output(s): vector sampler
 *
 */
USTRUCT(meta = (Icon = "Icons.EyeDropper"))
struct FDataflowCombineFloatSamplerNode : public FDataflowNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowCombineFloatSamplerNode, "Combine Float Sampler", "Samplers", "")

public:
	FDataflowCombineFloatSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	FDataflowFloatSampler FloatSamplerX;

	UPROPERTY(meta = (DataflowInput))
	FDataflowFloatSampler FloatSamplerY;

	UPROPERTY(meta = (DataflowInput))
	FDataflowFloatSampler FloatSamplerZ;

	UPROPERTY(meta = (DataflowOutput))
	FDataflowVectorSampler VectorSampler;

	void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};
