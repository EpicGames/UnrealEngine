// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/SamplerNodes/DataflowDistanceFromBoxSamplerNode.h"
#include "Dataflow/SamplerNodes/DataflowSamplerFunctions.h"
#include "Dataflow/DataflowNodeFactory.h"

FBox FDataflowDistanceFromBoxFloatSampler::GetRenderBounds() const
{
	return FBox(FVector(-50.0), FVector(50.0));
}

void FDataflowDistanceFromBoxFloatSampler::Sample(TArrayView<const FVector3f> Positions, TArrayView<float> OutValues) const
{
	if (Positions.Num() == OutValues.Num())
	{
		for (int32 Idx = 0; Idx < Positions.Num(); ++Idx)
		{
			OutValues[Idx] = FMath::Sqrt(
			FMath::Square(FMath::Max3(0.0, Box.Min.X - Positions[Idx].X, Positions[Idx].X - Box.Max.X)) +
				FMath::Square(FMath::Max3(0.0, Box.Min.Y - Positions[Idx].Y, Positions[Idx].Y - Box.Max.Y)) +
				FMath::Square(FMath::Max3(0.0, Box.Min.Z - Positions[Idx].Z, Positions[Idx].Z - Box.Max.Z)));
		}
	}
	else
	{
		for (int32 Idx = 0; Idx < OutValues.Num(); ++Idx)
		{
			OutValues[Idx] = 0.f;
		}
	}
}

FDataflowDistanceFromBoxFloatSamplerNode::FDataflowDistanceFromBoxFloatSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&DistanceFromBoxSampler.Box, GET_MEMBER_NAME_CHECKED(FDataflowDistanceFromBoxFloatSampler, Box));
	RegisterOutputConnection(&Sampler);
}

void FDataflowDistanceFromBoxFloatSamplerNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Sampler))
	{
		TSharedRef<FDataflowDistanceFromBoxFloatSampler> Impl = MakeShared<FDataflowDistanceFromBoxFloatSampler>();

		Impl->Box = GetValue(Context, &DistanceFromBoxSampler.Box);

		FDataflowFloatSampler OutSampler(Impl.ToSharedPtr());

		SetValue(Context, OutSampler, &Sampler);
	}
}


