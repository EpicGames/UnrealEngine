// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowNode.h"
#include "Dataflow/SamplerNodes/DataflowSamplerTypes.h"

#include "DataflowDistanceFromBoxSamplerNode.generated.h"

USTRUCT()
struct FDataflowDistanceFromBoxFloatSampler : public FDataflowFloatSamplerBase
{
	GENERATED_BODY();

	virtual ~FDataflowDistanceFromBoxFloatSampler() override = default;
	virtual void Sample(TArrayView<const FVector3f> Positions, TArrayView<float> OutValues) const override;
	virtual FBox GetRenderBounds() const override;

public:
	/** Box to compute the distance from */
	UPROPERTY(meta = (DataflowInput))
	FBox Box;
};

/**
 * 
 * Distance from a box float sampler
 * Input(s) : an AABB
 * Output(s): for every sampled point the distance from the input box
 * 
 */
USTRUCT(meta = (Icon = "Icons.EyeDropper"))
struct FDataflowDistanceFromBoxFloatSamplerNode : public FDataflowNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowDistanceFromBoxFloatSamplerNode, "Distance From Box Float Sampler", "Samplers", "")

public:
	FDataflowDistanceFromBoxFloatSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	UPROPERTY(meta = (DataflowOutput))
	FDataflowFloatSampler Sampler;

	UPROPERTY(EditAnywhere, Category = Sampler, meta = (ShowOnlyInnerProperties, SkipInDisplayNameChain))
	FDataflowDistanceFromBoxFloatSampler DistanceFromBoxSampler;

	void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};
