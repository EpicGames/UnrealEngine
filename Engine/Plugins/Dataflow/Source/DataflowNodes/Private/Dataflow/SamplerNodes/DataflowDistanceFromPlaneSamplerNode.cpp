// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/SamplerNodes/DataflowDistanceFromPlaneSamplerNode.h"
#include "Dataflow/SamplerNodes/DataflowSamplerFunctions.h"
#include "Dataflow/DataflowNodeFactory.h"

FBox FDataflowDistanceFromPlaneFloatSampler::GetRenderBounds() const
{
	return FBox(FVector(-50.0), FVector(50.0));
}

void FDataflowDistanceFromPlaneFloatSampler::Sample(TArrayView<const FVector3f> Positions, TArrayView<float> OutValues) const
{
	if (Positions.Num() == OutValues.Num())
	{
		for (int32 Idx = 0; Idx < Positions.Num(); ++Idx)
		{
			OutValues[Idx] = FMath::Abs(Plane.DistanceFromPoint((FVector)Positions[Idx]));
		}
	}
	else
	{
		for (int32 Idx = 0; Idx < OutValues.Num(); ++Idx)
		{
			OutValues[Idx] = 0.f;
		}
	}
}

FDataflowDistanceFromPlaneFloatSamplerNode::FDataflowDistanceFromPlaneFloatSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&DistanceFromPlaneSampler.Plane, GET_MEMBER_NAME_CHECKED(FDataflowDistanceFromPlaneFloatSampler, Plane));
	RegisterOutputConnection(&Sampler);
}

void FDataflowDistanceFromPlaneFloatSamplerNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Sampler))
	{
		TSharedRef<FDataflowDistanceFromPlaneFloatSampler> Impl = MakeShared<FDataflowDistanceFromPlaneFloatSampler>();

		Impl->Plane = GetValue(Context, &DistanceFromPlaneSampler.Plane);

		FDataflowFloatSampler OutSampler(Impl.ToSharedPtr());

		SetValue(Context, OutSampler, &Sampler);
	}
}


