// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowNode.h"
#include "Dataflow/SamplerNodes/DataflowSamplerTypes.h"
#include "Dataflow/DataflowPlane.h"

#include "DataflowDistanceFromPlaneSamplerNode.generated.h"

USTRUCT()
struct FDataflowDistanceFromPlaneFloatSampler : public FDataflowFloatSamplerBase
{
	GENERATED_BODY();

	virtual ~FDataflowDistanceFromPlaneFloatSampler() override = default;
	virtual void Sample(TArrayView<const FVector3f> Positions, TArrayView<float> OutValues) const override;
	virtual FBox GetRenderBounds() const override;

public:
	/** Plane to compute the distance from */
	UPROPERTY(meta = (DataflowInput))
	FDataflowPlane Plane;
};

/**
 * 
 * Distance from a plane float sampler
 * Input(s) : a plane
 * Output(s): for every sampled point the distance from the input plane
 * 
 */
USTRUCT(meta = (Icon = "Icons.EyeDropper"))
struct FDataflowDistanceFromPlaneFloatSamplerNode : public FDataflowNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowDistanceFromPlaneFloatSamplerNode, "Distance From Plane Float Sampler", "Samplers", "")

public:
	FDataflowDistanceFromPlaneFloatSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	UPROPERTY(meta = (DataflowOutput))
	FDataflowFloatSampler Sampler;

	UPROPERTY(EditAnywhere, Category = Sampler, meta = (ShowOnlyInnerProperties, SkipInDisplayNameChain))
	FDataflowDistanceFromPlaneFloatSampler DistanceFromPlaneSampler;

	void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};
