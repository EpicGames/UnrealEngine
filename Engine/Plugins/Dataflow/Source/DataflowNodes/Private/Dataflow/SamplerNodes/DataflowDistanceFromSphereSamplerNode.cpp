// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/SamplerNodes/DataflowDistanceFromSphereSamplerNode.h"
#include "Dataflow/SamplerNodes/DataflowSamplerFunctions.h"
#include "Dataflow/DataflowNodeFactory.h"

FBox FDataflowDistanceFromSphereFloatSampler::GetRenderBounds() const
{
	return FBox(FVector(-50.0), FVector(50.0));
}

void FDataflowDistanceFromSphereFloatSampler::Sample(TArrayView<const FVector3f> Positions, TArrayView<float> OutValues) const
{
	if (Positions.Num() == OutValues.Num())
	{
		for (int32 Idx = 0; Idx < Positions.Num(); ++Idx)
		{
			OutValues[Idx] = FMath::Abs((FVector(Positions[Idx]) - Sphere.Center).Length() - Sphere.W);
		}
	}
	else
	{
		for (int32 Idx = 0; Idx < OutValues.Num(); ++Idx)
		{
			OutValues[Idx] = 0.f;
		}
	}
}

FDataflowDistanceFromSphereFloatSamplerNode::FDataflowDistanceFromSphereFloatSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&DistanceFromSphereSampler.Sphere, GET_MEMBER_NAME_CHECKED(FDataflowDistanceFromSphereFloatSampler, Sphere));
	RegisterOutputConnection(&Sampler);
}

void FDataflowDistanceFromSphereFloatSamplerNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Sampler))
	{
		TSharedRef<FDataflowDistanceFromSphereFloatSampler> Impl = MakeShared<FDataflowDistanceFromSphereFloatSampler>();

		Impl->Sphere = GetValue(Context, &DistanceFromSphereSampler.Sphere);

		FDataflowFloatSampler OutSampler(Impl.ToSharedPtr());

		SetValue(Context, OutSampler, &Sampler);
	}
}


