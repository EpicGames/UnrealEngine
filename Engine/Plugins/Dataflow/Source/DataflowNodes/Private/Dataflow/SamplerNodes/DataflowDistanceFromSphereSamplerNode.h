// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowNode.h"
#include "Dataflow/SamplerNodes/DataflowSamplerTypes.h"

#include "DataflowDistanceFromSphereSamplerNode.generated.h"

USTRUCT()
struct FDataflowDistanceFromSphereFloatSampler : public FDataflowFloatSamplerBase
{
	GENERATED_BODY();

	virtual ~FDataflowDistanceFromSphereFloatSampler() override = default;
	virtual void Sample(TArrayView<const FVector3f> Positions, TArrayView<float> OutValues) const override;
	virtual FBox GetRenderBounds() const override;

public:
	/** Sphere to compute the distance from */
	UPROPERTY(meta = (DataflowInput))
	FSphere Sphere = FSphere(0.0);
};

/**
 * 
 * Distance from a sphere float sampler
 * Input(s) : a sphere
 * Output(s): for every sampled point the distance from the input sphere
 * 
 */
USTRUCT(meta = (Icon = "Icons.EyeDropper"))
struct FDataflowDistanceFromSphereFloatSamplerNode : public FDataflowNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowDistanceFromSphereFloatSamplerNode, "Distance From Sphere Float Sampler", "Samplers", "")

public:
	FDataflowDistanceFromSphereFloatSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	UPROPERTY(meta = (DataflowOutput))
	FDataflowFloatSampler Sampler;

	UPROPERTY(EditAnywhere, Category = Sampler, meta = (ShowOnlyInnerProperties, SkipInDisplayNameChain))
	FDataflowDistanceFromSphereFloatSampler DistanceFromSphereSampler;

	void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};
