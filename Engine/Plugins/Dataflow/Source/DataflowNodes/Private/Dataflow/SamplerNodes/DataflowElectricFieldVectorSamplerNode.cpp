// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/SamplerNodes/DataflowElectricFieldVectorSamplerNode.h"
#include "Dataflow/SamplerNodes/DataflowSamplerFunctions.h"
#include "Dataflow/DataflowNodeFactory.h"
#include "GeometryCollection/Facades/PointsFacade.h"

FBox FDataflowElectricFieldVectorSampler::GetRenderBounds() const
{
	if (!RenderBounds.IsValid)
	{
		FBox Bounds(ForceInit);

		for (int32 IdxCharge = 0; IdxCharge < Charges.Num(); ++IdxCharge)
		{
			Bounds += Charges[IdxCharge].GetPosition();
		}

		FVector Padding(20.0);
		return Bounds.ExpandBy(Padding, Padding);
	}
	else
	{
		return RenderBounds;
	}
}

void FDataflowElectricFieldVectorSampler::Sample(TArrayView<const FVector3f> Positions, TArrayView<FVector3f> OutValues) const
{
	// E = K * q / r^2
	constexpr double K = 8.99e9;
	constexpr double Correction = 1e-4; // The Values are way to big

	if (Positions.Num() == OutValues.Num())
	{
		TArray<FDataflowCharge> ChargesArr = Charges;

		GeometryCollection::Facades::FPointsFacade PointFacade = ChargesFromSampler.GetPointsFacade();
		const int32 NumPoints = PointFacade.GetNumPoints();

		const FName PointAttr = "Point";
		const FName ChargeAttr = "Charge";

		if (NumPoints > 0)
		{
			ChargesArr.Empty();

			for (int32 Idx = 0; Idx < NumPoints; ++Idx)
			{
				ChargesArr.Add(FDataflowCharge(FVector(PointFacade.GetVector3AttributeValue(PointAttr, Idx)),
					PointFacade.GetFloatAttributeValue(ChargeAttr, Idx)));
			}
		}

		for (int32 IdxPosition = 0; IdxPosition < Positions.Num(); ++IdxPosition)
		{
			FVector ESum = FVector::ZeroVector;

			for (int32 IdxCharge = 0; IdxCharge < ChargesArr.Num(); ++IdxCharge)
			{
				FVector ToCharge = FVector(Positions[IdxPosition]) - ChargesArr[IdxCharge].GetPosition();
				double Distance = ToCharge.Length();

				if (Distance > SMALL_NUMBER)
				{
					double Mag = K * ChargesArr[IdxCharge].GetCharge() / FMath::Square(Distance);

					ToCharge.Normalize();
					FVector EAtPosition = ToCharge * Mag;

					ESum += EAtPosition;
				}
			}

			ESum *= Scalar * Correction;

			if (bClamp)
			{
				double ESumMagnitudeClamped = FMath::Clamp(ESum.Length(), 0.0, MaxMagnitude);

				ESum.Normalize();
				ESum *= ESumMagnitudeClamped;
			}

			OutValues[IdxPosition] = FVector3f(ESum);
		}
	}
	else
	{
		for (int32 Index = 0; Index < OutValues.Num(); ++Index)
		{
			OutValues[Index] = FVector3f(0.f);
		}
	}
}

FDataflowElectricFieldVectorSamplerNode::FDataflowElectricFieldVectorSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&ElectricFieldSampler.RenderBounds, GET_MEMBER_NAME_CHECKED(FDataflowElectricFieldVectorSampler, RenderBounds))
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&Charges);

	RegisterOutputConnection(&Sampler);
}

void FDataflowElectricFieldVectorSamplerNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Sampler))
	{
		TSharedRef<FDataflowElectricFieldVectorSampler> Impl = MakeShared<FDataflowElectricFieldVectorSampler>(ElectricFieldSampler);
		Impl->Charges = ElectricFieldSampler.Charges;
		Impl->Scalar = ElectricFieldSampler.Scalar;
		Impl->RenderBounds = GetValue(Context, &ElectricFieldSampler.RenderBounds);
		Impl->ChargesFromSampler = GetValue(Context, &Charges);

		FDataflowVectorSampler OutSampler(Impl.ToSharedPtr());
		SetValue(Context, OutSampler, &Sampler);
	}
}

