// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowNode.h"
#include "Dataflow/SamplerNodes/DataflowSamplerTypes.h"
#include "Dataflow/DataflowPoints.h"

#include "DataflowElectricFieldVectorSamplerNode.generated.h"


/**
* Represents a charge for an electric field
*/
USTRUCT()
struct FDataflowCharge
{
	GENERATED_USTRUCT_BODY()

	FDataflowCharge() :
		Position(FVector::ZeroVector),
		Charge(0.0)
	{
	}

	FDataflowCharge(const FVector& InPosition, const double InCharge) :
		Position(InPosition),
		Charge(InCharge)
	{
	}

	FDataflowCharge(const FDataflowCharge& InCharge)
	{
		Position = InCharge.GetPosition();
		Charge = InCharge.GetCharge();
	}

	FVector GetPosition() const { return Position; }
	double GetCharge() const { return Charge; }

private:
	UPROPERTY(EditAnywhere, Category = "Charge")
	FVector Position;

	UPROPERTY(EditAnywhere, Category = "Charge")
	double Charge;
};

USTRUCT()
struct FDataflowElectricFieldVectorSampler : public FDataflowVectorSamplerBase
{
	GENERATED_BODY();

	virtual ~FDataflowElectricFieldVectorSampler() override = default;
	virtual void Sample(TArrayView<const FVector3f> Positions, TArrayView<FVector3f> OutValues) const override;
	virtual FBox GetRenderBounds() const override;

public:
	UPROPERTY(EditAnywhere, Category = "Charges")
	TArray<FDataflowCharge> Charges;

	UPROPERTY(EditAnywhere, Category = "Charges")
	double Scalar = 1e-3;

	UPROPERTY(EditAnywhere, Category = "Charges")
	bool bClamp = false;

	UPROPERTY(EditAnywhere, Category = "Charges")
	double MaxMagnitude = 10.0;

	UPROPERTY()
	FDataflowPoints ChargesFromSampler;

	UPROPERTY(EditAnywhere, Category = RenderBounds, meta = (DataflowInput))
	FBox RenderBounds = FBox(FVector(-50.0), FVector(50.0));
};

/**
 * Output a ElectricField vector sampler
 */
USTRUCT(meta = (Icon = "Icons.EyeDropper"))
struct FDataflowElectricFieldVectorSamplerNode : public FDataflowNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowElectricFieldVectorSamplerNode, "ElectricField Vector Sampler", "Samplers", "")

public:
	FDataflowElectricFieldVectorSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	UPROPERTY(meta = (DataflowInput))
	FDataflowPoints Charges;

	UPROPERTY(meta = (DataflowOutput))
	FDataflowVectorSampler Sampler;

	UPROPERTY(EditAnywhere, Category = Sampler, meta = (ShowOnlyInnerProperties, SkipInDisplayNameChain))
	FDataflowElectricFieldVectorSampler ElectricFieldSampler;

	void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

