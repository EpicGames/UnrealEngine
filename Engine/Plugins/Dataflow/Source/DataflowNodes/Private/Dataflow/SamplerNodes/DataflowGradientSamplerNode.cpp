// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/SamplerNodes/DataflowGradientSamplerNode.h"
#include "Dataflow/DataflowNodeFactory.h"

namespace UE::Dataflow
{
	void RegisterGradientSamplerNodes()
	{
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowLinearGradientFloatSamplerNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowLinearGradientFromBoxFloatSamplerNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowRadialGradientFloatSamplerNode);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FBox FDataflowLinearGradientFloatSampler::GetRenderBounds() const
{
	FBox Box;

	Box += FVector(StartPoint);
	Box += FVector(EndPoint);

	return Box;
}

void FDataflowLinearGradientFloatSampler::Sample(TArrayView<const FVector3f> Positions, TArrayView<float> OutValues) const
{
	const FVector3f GradientDirection = EndPoint - StartPoint;
	const double SquareLength = GradientDirection.SquaredLength();

	if (SquareLength < UE_SMALL_NUMBER)
	{
		for (float& OutValue : OutValues)
		{
			OutValue = StartValue;
		}
		return;
	}

	const float InvSquareLength = 1.f / SquareLength;

	const int32 NumSamples = FMath::Min(Positions.Num(), OutValues.Num());
	for (int32 Index = 0; Index < NumSamples; ++Index)
	{
		const FVector3f LocalPosition = Positions[Index] - StartPoint;
		const float Alpha = LocalPosition.Dot(GradientDirection) * InvSquareLength;
		const float Value = FMath::Lerp(StartValue, EndValue, Alpha);
		OutValues[Index] = (bClamp)
			? FMath::Clamp(Value, FMath::Min(StartValue, EndValue), FMath::Max(StartValue, EndValue))
			: Value;
	}
}

FDataflowLinearGradientFloatSamplerNode::FDataflowLinearGradientFloatSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Gradient.StartPoint, GET_MEMBER_NAME_CHECKED(FDataflowLinearGradientFloatSampler, StartPoint));
	RegisterInputConnection(&Gradient.StartValue, GET_MEMBER_NAME_CHECKED(FDataflowLinearGradientFloatSampler, StartValue));
	RegisterInputConnection(&Gradient.EndPoint, GET_MEMBER_NAME_CHECKED(FDataflowLinearGradientFloatSampler, EndPoint));
	RegisterInputConnection(&Gradient.EndValue, GET_MEMBER_NAME_CHECKED(FDataflowLinearGradientFloatSampler, EndValue));
	RegisterInputConnection(&Gradient.bClamp, GET_MEMBER_NAME_CHECKED(FDataflowLinearGradientFloatSampler, bClamp));
	RegisterOutputConnection(&Sampler);
}

void FDataflowLinearGradientFloatSamplerNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Sampler))
	{
		TSharedRef<FDataflowLinearGradientFloatSampler> Impl = MakeShared<FDataflowLinearGradientFloatSampler>(Gradient);
		Impl->StartPoint = GetValue(Context, &Gradient.StartPoint);
		Impl->StartValue = GetValue(Context, &Gradient.StartValue);
		Impl->EndPoint = GetValue(Context, &Gradient.EndPoint);
		Impl->EndValue = GetValue(Context, &Gradient.EndValue);
		Impl->bClamp = GetValue(Context, &Gradient.bClamp);

		FDataflowFloatSampler OutSampler(Impl.ToSharedPtr());
		SetValue(Context, OutSampler, &Sampler);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

void FDataflowLinearGradientFromBoxFloatSampler::ComputePoints(FVector& StartPoint, FVector& EndPoint) const
{
	const FVector BoxCenter = Box.GetCenter();
	const FVector BoxHalfExtent = Box.GetExtent();

	FVector GradientHalfExtent{ 0 };
	switch (Axis)
	{
	case EAxis::X: GradientHalfExtent.X = BoxHalfExtent.X; break;
	case EAxis::Y: GradientHalfExtent.Y = BoxHalfExtent.Y; break;
	case EAxis::Z: GradientHalfExtent.Z = BoxHalfExtent.Z; break;
	case EAxis::None: break;
	}

	GradientHalfExtent = Transform.TransformVector(GradientHalfExtent);

	StartPoint = Transform.GetTranslation() + BoxCenter - GradientHalfExtent;
	EndPoint = Transform.GetTranslation() + BoxCenter + GradientHalfExtent;
}

FBox FDataflowLinearGradientFromBoxFloatSampler::GetRenderBounds() const
{
	FVector Min = Box.Min;
	FVector Max = Box.Max;
	FVector Center = 0.5 * (Min + Max);
	FVector Extent = 0.5 * (Max - Min);

	return FBox(Center - 1.2f * Extent, Center + 1.2f * Extent);	
}

void FDataflowLinearGradientFromBoxFloatSampler::Sample(TArrayView<const FVector3f> Positions, TArrayView<float> OutValues) const
{
	for (int32 Idx = 0; Idx < OutValues.Num(); Idx++)
	{
		OutValues[Idx] = 0.f;
	}

	if (Positions.Num() == OutValues.Num())
	{
		FVector StartPoint, EndPoint;

		ComputePoints(StartPoint, EndPoint);

		FDataflowLinearGradientFloatSampler RegularGradientSampler;
		RegularGradientSampler.StartPoint = FVector3f(StartPoint);
		RegularGradientSampler.StartPoint = FVector3f(StartPoint);
		RegularGradientSampler.StartValue = 0.f;
		RegularGradientSampler.EndPoint = FVector3f(EndPoint);
		RegularGradientSampler.EndValue = 1.f;
		RegularGradientSampler.bClamp = bClamp;

		RegularGradientSampler.Sample(Positions, OutValues);

		for (int32 Idx = 0; Idx < OutValues.Num(); Idx++)
		{
			OutValues[Idx] = ColorRamp.GetLinearColorValue(OutValues[Idx]).R;
		}
	}
}

FDataflowLinearGradientFromBoxFloatSamplerNode::FDataflowLinearGradientFromBoxFloatSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Gradient.Box, GET_MEMBER_NAME_CHECKED(FDataflowLinearGradientFromBoxFloatSampler, Box));
	RegisterInputConnection(&Gradient.Axis, GET_MEMBER_NAME_CHECKED(FDataflowLinearGradientFromBoxFloatSampler, Axis));
	RegisterInputConnection(&Gradient.Transform, GET_MEMBER_NAME_CHECKED(FDataflowLinearGradientFromBoxFloatSampler, Transform));
	RegisterOutputConnection(&Sampler);

	constexpr bool bOnlyRGB = true;
	Gradient.ColorRamp.SetColorAtTime(0.0f, FLinearColor::Black, bOnlyRGB);
	Gradient.ColorRamp.SetColorAtTime(1.0f, FLinearColor::White, bOnlyRGB);
}

void FDataflowLinearGradientFromBoxFloatSamplerNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Sampler))
	{
		TSharedRef<FDataflowLinearGradientFromBoxFloatSampler> Impl = MakeShared<FDataflowLinearGradientFromBoxFloatSampler>(Gradient);
		Impl->Box = GetValue(Context, &Gradient.Box);
		Impl->Transform = GetValue(Context, &Gradient.Transform);
		Impl->ColorRamp = Gradient.ColorRamp;
		Impl->bClamp = Gradient.bClamp;

		FDataflowFloatSampler OutSampler(Impl.ToSharedPtr());
		SetValue(Context, OutSampler, &Sampler);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FBox FDataflowRadialGradientFloatSampler::GetRenderBounds() const
{
	// Radius is always greater than zero
	FVector Extent(Radius);
	FBox Box(FVector(Center) - Extent, FVector(Center) + Extent);

	return Box;
}

void FDataflowRadialGradientFloatSampler::Sample(TArrayView<const FVector3f> Positions, TArrayView<float> OutValues) const
{
	if (Radius < UE_SMALL_NUMBER)
	{
		for (float& OutValue : OutValues)
		{
			OutValue = EdgeValue;
		}
		return;
	}

	const float InvRadius = 1.f / Radius;

	const int32 NumSamples = FMath::Min(Positions.Num(), OutValues.Num());
	for (int32 Index = 0; Index < NumSamples; ++Index)
	{
		const FVector3f GradientVector = Positions[Index] - Center;
		const float Alpha = GradientVector.Size() * InvRadius;

		const float Value = FMath::Lerp(CenterValue, EdgeValue, Alpha);
		OutValues[Index] = (bClamp)
			? FMath::Clamp(Value, FMath::Min(CenterValue, EdgeValue), FMath::Max(CenterValue, EdgeValue))
			: Value;
	}
}

FDataflowRadialGradientFloatSamplerNode::FDataflowRadialGradientFloatSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Gradient.Center, GET_MEMBER_NAME_CHECKED(FDataflowRadialGradientFloatSampler, Center));
	RegisterInputConnection(&Gradient.Radius, GET_MEMBER_NAME_CHECKED(FDataflowRadialGradientFloatSampler, Radius));
	RegisterInputConnection(&Gradient.CenterValue, GET_MEMBER_NAME_CHECKED(FDataflowRadialGradientFloatSampler, CenterValue));
	RegisterInputConnection(&Gradient.EdgeValue, GET_MEMBER_NAME_CHECKED(FDataflowRadialGradientFloatSampler, EdgeValue));
	RegisterInputConnection(&Gradient.bClamp, GET_MEMBER_NAME_CHECKED(FDataflowRadialGradientFloatSampler, bClamp));
	RegisterOutputConnection(&Sampler);
}

void FDataflowRadialGradientFloatSamplerNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Sampler))
	{
		TSharedRef<FDataflowRadialGradientFloatSampler> Impl = MakeShared<FDataflowRadialGradientFloatSampler>(Gradient);
		Impl->Center = GetValue(Context, &Gradient.Center, Gradient.Center);
		Impl->Radius = GetValue(Context, &Gradient.Radius, Gradient.Radius);
		Impl->CenterValue = GetValue(Context, &Gradient.CenterValue, Gradient.CenterValue);
		Impl->EdgeValue = GetValue(Context, &Gradient.EdgeValue, Gradient.EdgeValue);
		Impl->bClamp = GetValue(Context, &Gradient.bClamp, Gradient.bClamp);

		FDataflowFloatSampler OutSampler(Impl.ToSharedPtr());
		SetValue(Context, OutSampler, &Sampler);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////


