// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowNode.h"
#include "Dataflow/SamplerNodes/DataflowSamplerTypes.h"
#include "Curves/LinearColorRamp.h"

#include "DataflowGradientSamplerNode.generated.h"

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

USTRUCT(meta = (Icon = "Icons.EyeDropper"))
struct FDataflowLinearGradientFloatSampler : public FDataflowFloatSamplerBase
{
	GENERATED_BODY();

	virtual ~FDataflowLinearGradientFloatSampler() override = default;
	virtual void Sample(TArrayView<const FVector3f> Positions, TArrayView<float> OutValues) const override;
	virtual FBox GetRenderBounds() const override;

public:
	// Point in space where the start value is defined
	UPROPERTY(EditAnywhere, Category = Gradient, meta = (DataflowInput, GizmoType = "Translate"))
	FVector3f StartPoint = { 0, 0, 100 };

	// Value defined at start point
	UPROPERTY(EditAnywhere, Category = Gradient, meta = (DataflowInput))
	float StartValue = 1.0;

	// Point in space where end value is defined
	UPROPERTY(EditAnywhere, Category = Gradient, meta = (DataflowInput, GizmoType = "Translate"))
	FVector3f EndPoint = { 0, 0, 0 };

	// Value defined at end point
	UPROPERTY(EditAnywhere, Category = Gradient, meta = (DataflowInput))
	float EndValue = 0.0;

	// If true generated value will be clamped beyond end and start point, otherwise, they will be extrapolated
	UPROPERTY(EditAnywhere, Category = Gradient, meta = (DataflowInput))
	bool bClamp = true;
};

/**
 * Output a linear gradient sampler 
 */
USTRUCT(meta = (Icon = "Icons.EyeDropper"))
struct FDataflowLinearGradientFloatSamplerNode : public FDataflowNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowLinearGradientFloatSamplerNode, "Linear Gradient Sampler", "Samplers", "")

public:
	FDataflowLinearGradientFloatSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	UPROPERTY(meta = (DataflowOutput))
	FDataflowFloatSampler Sampler;

	UPROPERTY(EditAnywhere, Category = Sampler, meta = (ShowOnlyInnerProperties, SkipInDisplayNameChain))
	FDataflowLinearGradientFloatSampler Gradient;

	void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

USTRUCT(meta = (Icon = "Icons.EyeDropper"))
struct FDataflowLinearGradientFromBoxFloatSampler : public FDataflowFloatSamplerBase
{
	GENERATED_BODY();

	virtual ~FDataflowLinearGradientFromBoxFloatSampler() override = default;
	virtual void Sample(TArrayView<const FVector3f> Positions, TArrayView<float> OutValues) const override;
	virtual FBox GetRenderBounds() const override;

public:
	// Box to use for the gradient - one axis is chosen to be the axis of the gradient
	UPROPERTY(EditAnywhere, Category = Gradient, meta = (DataflowInput))
	FBox Box;

	// Transform to apply to the box
	UPROPERTY(EditAnywhere, Category = Gradient, meta = (DataflowInput))
	FTransform Transform = FTransform::Identity;

	// Axis of the box to use for the gradient
	UPROPERTY(EditAnywhere, Category = Gradient)
	TEnumAsByte<EAxis::Type> Axis = EAxis::Z;

	// If true generated value will be clamped beyond end and start point, otherwise, they will be extrapolated
	UPROPERTY(EditAnywhere, Category = Gradient)
	bool bClamp = true;

	// Ramp representing the values for the gradient, since the values are float, it will read the R channel of the colors
	UPROPERTY(EditAnywhere, Category = Gradient)
	FLinearColorRamp ColorRamp;

private:
	void ComputePoints(FVector& StartPoint, FVector& EndPoint) const;
};

/**
 * Output a linear gradient sampler
 */
USTRUCT(meta = (Icon = "Icons.EyeDropper"))
struct FDataflowLinearGradientFromBoxFloatSamplerNode : public FDataflowNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowLinearGradientFromBoxFloatSamplerNode, "Linear Gradient From Box Sampler", "Samplers", "Bounds Bounding Box Axis")

public:
	FDataflowLinearGradientFromBoxFloatSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	UPROPERTY(meta = (DataflowOutput))
	FDataflowFloatSampler Sampler;

	UPROPERTY(EditAnywhere, Category = Sampler, meta = (ShowOnlyInnerProperties, SkipInDisplayNameChain))
	FDataflowLinearGradientFromBoxFloatSampler Gradient;
	
	void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

USTRUCT(meta = (Icon = "Icons.EyeDropper"))
struct FDataflowRadialGradientFloatSampler : public FDataflowFloatSamplerBase
{
	GENERATED_BODY();

	virtual ~FDataflowRadialGradientFloatSampler() override = default;
	virtual void Sample(TArrayView<const FVector3f> Positions, TArrayView<float> OutValues) const override;
	virtual FBox GetRenderBounds() const override;

public:
	// center defining the radial gradient
	UPROPERTY(EditAnywhere, Category = Gradient, meta = (DataflowInput))
	FVector3f Center = { 0, 0, 0 };

	// Radius of the gradient
	UPROPERTY(EditAnywhere, Category = Gradient, meta = (DataflowInput, UIMin = 0.01, ClampMin = 0.01))
	float Radius = 100.f;

	// Value defined at center
	UPROPERTY(EditAnywhere, Category = Gradient, meta = (DataflowInput))
	float CenterValue = 1.0;

	// Value defined at the radius distance from the center 
	UPROPERTY(EditAnywhere, Category = Gradient, meta = (DataflowInput))
	float EdgeValue = 0.0;

	// If true generated values will be clamped beyond the radius otherwise they will they will be extrapolated
	UPROPERTY(EditAnywhere, Category = Gradient, meta = (DataflowInput))
	bool bClamp = true;
};

/**
 * Output a radial gradient sampler
 */
USTRUCT(meta = (Icon = "Icons.EyeDropper"))
struct FDataflowRadialGradientFloatSamplerNode : public FDataflowNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowRadialGradientFloatSamplerNode, "Radial Gradient Sampler", "Samplers", "")

public:
	FDataflowRadialGradientFloatSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	UPROPERTY(meta = (DataflowOutput))
	FDataflowFloatSampler Sampler;

	UPROPERTY(EditAnywhere, Category = Sampler, meta = (ShowOnlyInnerProperties, SkipInDisplayNameChain))
	FDataflowRadialGradientFloatSampler Gradient;;

	void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

namespace UE::Dataflow
{
	void RegisterGradientSamplerNodes();
}