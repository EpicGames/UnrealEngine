// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/SamplerNodes/DataflowLerpSamplerNode.h"
#include "Dataflow/SamplerNodes/DataflowSamplerFunctions.h"
#include "Dataflow/DataflowNodeFactory.h"

FBox FDataflowLerpFloatSampler::GetRenderBounds() const
{
	FBox RenderBounds = FBox(ForceInit);

	if (FloatSamplerA.IsValid() || FloatSamplerB.IsValid() || BlendSampler.IsValid())
	{
		if (FloatSamplerA.IsValid())
		{
			RenderBounds += FloatSamplerA->GetRenderBounds();
		}

		if (FloatSamplerB.IsValid())
		{
			RenderBounds += FloatSamplerB->GetRenderBounds();
		}

		if (BlendSampler.IsValid())
		{
			RenderBounds += BlendSampler->GetRenderBounds();
		}

		return RenderBounds;
	}

	return FBox(FVector(-50.0), FVector(50.0));
}

void FDataflowLerpFloatSampler::Sample(TArrayView<const FVector3f> Positions, TArrayView<float> OutValues) const
{
	if (FloatSamplerA.IsValid() && Positions.Num() == OutValues.Num())
	{
		FloatSamplerA->Sample(Positions, OutValues);

		if (FloatSamplerB.IsValid())
		{
			TArray<float> OutValueFromFloatSamplerB;
			OutValueFromFloatSamplerB.SetNumUninitialized(OutValues.Num());

			FloatSamplerB->Sample(Positions, OutValueFromFloatSamplerB);

			if (BlendSampler.IsValid())
			{
				TArray<float> OutValueFromFloatSampler;
				OutValueFromFloatSampler.SetNumUninitialized(OutValues.Num());

				BlendSampler->Sample(Positions, OutValueFromFloatSampler);

				for (int32 Idx = 0; Idx < OutValues.Num(); ++Idx)
				{
					OutValues[Idx] = FMath::Lerp(OutValues[Idx], OutValueFromFloatSamplerB[Idx], OutValueFromFloatSampler[Idx]);
				}
			}
			else
			{
				for (int32 Idx = 0; Idx < OutValues.Num(); ++Idx)
				{
					OutValues[Idx] = FMath::Lerp(OutValues[Idx], OutValueFromFloatSamplerB[Idx], Blend);
				}
			}
		}
	}
	else
	{
		for (int32 Idx = 0; Idx < OutValues.Num(); ++Idx)
		{
			OutValues[Idx] = 0.f;
		}
	}
}

FBox FDataflowLerpVectorSampler::GetRenderBounds() const
{
	FBox RenderBounds = FBox(ForceInit);

	if (VectorSamplerA.IsValid() || VectorSamplerB.IsValid() || BlendSampler.IsValid())
	{
		if (VectorSamplerA.IsValid())
		{
			RenderBounds += VectorSamplerA->GetRenderBounds();
		}

		if (VectorSamplerB.IsValid())
		{
			RenderBounds += VectorSamplerB->GetRenderBounds();
		}

		if (BlendSampler.IsValid())
		{
			RenderBounds += BlendSampler->GetRenderBounds();
		}

		return RenderBounds;
	}

	return FBox(FVector(-50.0), FVector(50.0));
}

void FDataflowLerpVectorSampler::Sample(TArrayView<const FVector3f> Positions, TArrayView<FVector3f> OutValues) const
{
	if (VectorSamplerA.IsValid() && Positions.Num() == OutValues.Num())
	{
		VectorSamplerA->Sample(Positions, OutValues);

		if (VectorSamplerB.IsValid())
		{
			TArray<FVector3f> OutValueFromFloatSamplerB;
			OutValueFromFloatSamplerB.SetNumUninitialized(OutValues.Num());

			VectorSamplerB->Sample(Positions, OutValueFromFloatSamplerB);

			if (BlendSampler.IsValid())
			{
				TArray<float> OutValueFromFloatSampler;
				OutValueFromFloatSampler.SetNumUninitialized(OutValues.Num());

				BlendSampler->Sample(Positions, OutValueFromFloatSampler);

				for (int32 Idx = 0; Idx < OutValues.Num(); ++Idx)
				{
					OutValues[Idx] = FMath::Lerp(OutValues[Idx], OutValueFromFloatSamplerB[Idx], OutValueFromFloatSampler[Idx]);

				}
			}
			else
			{
				for (int32 Idx = 0; Idx < OutValues.Num(); ++Idx)
				{
					OutValues[Idx] = FMath::Lerp(OutValues[Idx], OutValueFromFloatSamplerB[Idx], Blend);
				}
			}
		}
	}
	else
	{
		for (int32 Idx = 0; Idx < OutValues.Num(); ++Idx)
		{
			OutValues[Idx] = FVector3f::ZeroVector;
		}
	}
}

FDataflowLerpSamplerNode::FDataflowLerpSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	static const FName MainTypeGroup("Main");

	RegisterInputConnection(&SamplerA)
		.SetTypeDependencyGroup(MainTypeGroup);
	RegisterInputConnection(&SamplerB)
		.SetTypeDependencyGroup(MainTypeGroup);
	RegisterInputConnection(&BlendSampler);

	RegisterInputConnection(&Blend);

	RegisterOutputConnection(&SamplerA)
		.SetPassthroughInput(&SamplerA)
		.SetTypeDependencyGroup(MainTypeGroup);
}

void FDataflowLerpSamplerNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&SamplerA))
	{
		const float InBlend = GetValue(Context, &Blend);

		const FDataflowFloatSampler* InFloatSamplerA = GetValue(Context, &SamplerA).TryGet <FDataflowFloatSampler>();
		const FDataflowFloatSampler* InFloatSamplerB = GetValue(Context, &SamplerB).TryGet <FDataflowFloatSampler>();
		const FDataflowFloatSampler& InBlendSampler = GetValue(Context, &BlendSampler);

		if (InFloatSamplerA && InFloatSamplerB)
		{
			TSharedRef<FDataflowLerpFloatSampler> Impl = MakeShared<FDataflowLerpFloatSampler>();

			Impl->FloatSamplerA = InFloatSamplerA->GetImpl();
			Impl->FloatSamplerB = InFloatSamplerB->GetImpl();
			Impl->Blend = InBlend;

			Impl->BlendSampler = InBlendSampler.GetImpl();

			FDataflowFloatSampler OutSampler(Impl.ToSharedPtr());

			SetValue(Context, OutSampler, &SamplerA);
			return;
		}

		const FDataflowVectorSampler* VectorSamplerA = GetValue(Context, &SamplerA).TryGet<FDataflowVectorSampler>();
		const FDataflowVectorSampler* VectorSamplerB = GetValue(Context, &SamplerB).TryGet<FDataflowVectorSampler>();

		if (VectorSamplerA && VectorSamplerB)
		{
			TSharedRef<FDataflowLerpVectorSampler> Impl = MakeShared<FDataflowLerpVectorSampler>();

			Impl->VectorSamplerA = VectorSamplerA->GetImpl();
			Impl->VectorSamplerB = VectorSamplerB->GetImpl();
			Impl->Blend = InBlend;

			Impl->BlendSampler = InBlendSampler.GetImpl();

			FDataflowVectorSampler OutSampler(Impl.ToSharedPtr());

			SetValue(Context, OutSampler, &SamplerA);
			return;
		}

		SafeForwardInput(Context, &SamplerA, &SamplerA);
	}

}
