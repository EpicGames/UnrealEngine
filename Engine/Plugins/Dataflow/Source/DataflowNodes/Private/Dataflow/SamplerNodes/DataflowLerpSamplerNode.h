// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowNode.h"
#include "Dataflow/SamplerNodes/DataflowSamplerTypes.h"

#include "DataflowLerpSamplerNode.generated.h"

USTRUCT()
struct FDataflowLerpFloatSampler : public FDataflowFloatSamplerBase
{
	GENERATED_BODY();

	virtual ~FDataflowLerpFloatSampler() override = default;
	virtual void Sample(TArrayView<const FVector3f> Positions, TArrayView<float> OutValues) const override;
	virtual FBox GetRenderBounds() const override;

public:
	UPROPERTY()
	float Blend = 0.f;

	TSharedPtr<const FDataflowFloatSamplerBase> FloatSamplerA;

	TSharedPtr<const FDataflowFloatSamplerBase> FloatSamplerB;

	TSharedPtr<const FDataflowFloatSamplerBase> BlendSampler;
};

USTRUCT()
struct FDataflowLerpVectorSampler : public FDataflowVectorSamplerBase
{
	GENERATED_BODY();

	virtual ~FDataflowLerpVectorSampler() override = default;
	virtual void Sample(TArrayView<const FVector3f> Positions, TArrayView<FVector3f> OutValues) const override;
	virtual FBox GetRenderBounds() const override;

public:
	UPROPERTY()
	float Blend = 0.f;

	TSharedPtr<const FDataflowVectorSamplerBase> VectorSamplerA;

	TSharedPtr<const FDataflowVectorSamplerBase> VectorSamplerB;

	TSharedPtr<const FDataflowFloatSamplerBase> BlendSampler;
};

/**
 *
 * LerpSampler
 * Input(s) : 2 Float or Vector Sampler, optional BlendSampler (if not connected blend value can be used)
 * Output(s): Same type of sampler as the input ooutputting the lerped value
 *
 */
USTRUCT(meta = (Icon = "Icons.EyeDropper"))
struct FDataflowLerpSamplerNode : public FDataflowNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowLerpSamplerNode, "Lerp Sampler", "Samplers", "Linear Interpolation")

public:
	FDataflowLerpSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	/** First input for lerp */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = SamplerA))
	FDataflowSamplerTypes SamplerA;

	/** Second input for lerp */
	UPROPERTY(meta = (DataflowInput))
	FDataflowSamplerTypes SamplerB;

	/** Optional FloatSampler to control lerp per sampled points */
	UPROPERTY(meta = (DataflowInput))
	FDataflowFloatSampler BlendSampler;

	/** If BlendSampler is not connected this value controls the lerp */
	UPROPERTY(EditAnywhere, Category = Lerp, meta = (DataflowInput, UIMin = 0.f, ClampMin = 0.f, UIMax = 1.f, ClampMax = 1.f))
	float Blend = 0.f;

	void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};
