// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/SamplerNodes/DataflowSamplerTypes.h"
#include "Dataflow/SamplerNodes/DataflowSamplerMultiInput.h"

#include "DataflowMaxSamplerNode.generated.h"

USTRUCT()
struct FDataflowMaxFloatSampler : public FDataflowFloatSamplerBase
{
	GENERATED_BODY();
	virtual ~FDataflowMaxFloatSampler() override = default;
	virtual void Sample(TArrayView<const FVector3f> Positions, TArrayView<float> OutValues) const override;
	virtual FBox GetRenderBounds() const override;

public:
	/** list of samplers to Max results from */
	TArray<TSharedPtr<const FDataflowFloatSamplerBase>> Samplers;
};

USTRUCT()
struct FDataflowMaxVectorSampler : public FDataflowVectorSamplerBase
{
	GENERATED_BODY();
	virtual ~FDataflowMaxVectorSampler() override = default;
	virtual void Sample(TArrayView<const FVector3f> Positions, TArrayView<FVector3f> OutValues) const override;
	virtual FBox GetRenderBounds() const override;

public:
	/** list of samplers to Max results from */
	TArray<TSharedPtr<const FDataflowVectorSamplerBase>> Samplers;
};

/**
* Max sampler results together
*/
USTRUCT(meta = (Icon = "Icons.EyeDropper"))
struct FDataflowMaxSamplerNode : public FDataflowMultiInputSamplerNodeBase
{
	GENERATED_USTRUCT_BODY()

	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowMaxSamplerNode, "Max Sampler", "Samplers", "Maximum")

public:
	FDataflowMaxSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
	virtual void EvaluateSampler(UE::Dataflow::FContext& Context, FDataflowSamplerTypes::FStorageType& OutSampler) const override;
};

