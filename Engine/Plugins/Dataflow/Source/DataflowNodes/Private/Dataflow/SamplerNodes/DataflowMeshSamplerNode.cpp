// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/SamplerNodes/DataflowMeshSamplerNode.h"
#include "Dataflow/SamplerNodes/DataflowSamplerFunctions.h"
#include "Dataflow/DataflowNodeFactory.h"
#include "IntBoxTypes.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "DynamicMesh/DynamicMeshOctree3.h"
#include "Intersection/IntrRay3AxisAlignedBox3.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/MeshNormals.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h" // FDynamicMeshUVOverlay
#include "Parameterization/DynamicMeshUVEditor.h"
#include "Dataflow/DataflowEngineUtil.h"

FBox FDataflowMeshFloatSampler::GetRenderBounds() const
{
	if (Mesh)
	{
		if (const UE::Geometry::FDynamicMesh3* InDynMeshPtr = Mesh->GetDynamicMesh())
		{
			UE::Geometry::FAxisAlignedBox3d AABox = InDynMeshPtr->GetBounds(true);

			FVector Min = AABox.Min;
			FVector Max = AABox.Max;
			FVector Center = 0.5 * (Min + Max);
			FVector Extent = 0.5 * (Max - Min);

			return FBox(Center - 1.2f * Extent, Center + 1.2f * Extent);
		}
	}

	return FBox(FVector(-50.0), FVector(50.0));
}

void FDataflowMeshFloatSampler::Sample(TArrayView<const FVector3f> Positions, TArrayView<float> OutValues) const
{
	for (int32 Idx = 0; Idx < OutValues.Num(); ++Idx)
	{
		OutValues[Idx] = 0.f;
	}

	if (Mesh)
	{
		if (const UE::Geometry::FDynamicMesh3* InDynMeshPtr = Mesh->GetDynamicMesh())
		{
			UE::Geometry::FDynamicMeshAABBTree3 AABBTree(InDynMeshPtr);

			if (Positions.Num() == OutValues.Num())
			{
				if (OutputType == EDataflowMeshFloatSamplerOutputType::ClosestPointDistance)
				{
					for (int32 Idx = 0; Idx < Positions.Num(); ++Idx)
					{
						FVector NearestPoint = AABBTree.FindNearestPoint(FVector(Positions[Idx]));

						double SquaredDist;
						const int32 NearestTriangleID = AABBTree.FindNearestTriangle(FVector(Positions[Idx]), SquaredDist);
						if (InDynMeshPtr->IsTriangle(NearestTriangleID))
						{
							FVector NearestTriangleNormal = InDynMeshPtr->GetTriNormal(NearestTriangleID);

							FVector ToNearestPointVector = NearestPoint - FVector(Positions[Idx]);
							float DistanceToNearestPoint = (NearestPoint - FVector(Positions[Idx])).Length();

							if (ToNearestPointVector.Dot(NearestTriangleNormal) > 0.0)
							{
								OutValues[Idx] = -1.f * DistanceToNearestPoint;
							}
							else
							{
								OutValues[Idx] = DistanceToNearestPoint;
							}
						}
					}
				}
				else if (OutputType == EDataflowMeshFloatSamplerOutputType::ClosestTriangleID)
				{
					double SquaredDist;

					for (int32 Idx = 0; Idx < Positions.Num(); ++Idx)
					{
						const int32 NearestTriangleID = AABBTree.FindNearestTriangle(FVector(Positions[Idx]), SquaredDist);

						if (InDynMeshPtr->IsTriangle(NearestTriangleID))
						{
							OutValues[Idx] = float(NearestTriangleID);
						}
					}
				}

				return;
			}
		}
	}
}

FDataflowMeshFloatSamplerNode::FDataflowMeshFloatSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&MeshSampler.Mesh, GET_MEMBER_NAME_CHECKED(FDataflowMeshFloatSampler, Mesh));
	RegisterOutputConnection(&Sampler);
}

void FDataflowMeshFloatSamplerNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Sampler))
	{
		TSharedRef<FDataflowMeshFloatSampler> Impl = MakeShared<FDataflowMeshFloatSampler>();

		Impl->Mesh = GetValue(Context, &MeshSampler.Mesh);
		Impl->OutputType = MeshSampler.OutputType;

		FDataflowFloatSampler OutSampler(Impl.ToSharedPtr());

		SetValue(Context, OutSampler, &Sampler);
	}
}

// --------------------------------------------------------------------------------------------------------------------

FBox FDataflowMeshVectorSampler::GetRenderBounds() const
{
	if (Mesh)
	{
		if (const UE::Geometry::FDynamicMesh3* InDynMeshPtr = Mesh->GetDynamicMesh())
		{
			UE::Geometry::FAxisAlignedBox3d AABox = InDynMeshPtr->GetBounds(true);

			FVector Min = AABox.Min;
			FVector Max = AABox.Max;
			FVector Center = 0.5 * (Min + Max);
			FVector Extent = 0.5 * (Max - Min);

			return FBox(Center - 1.2f * Extent, Center + 1.2f * Extent);
		}
	}

	return FBox(FVector(-50.0), FVector(50.0));
}

void FDataflowMeshVectorSampler::Sample(TArrayView<const FVector3f> Positions, TArrayView<FVector3f> OutValues) const
{
	for (int32 Idx = 0; Idx < OutValues.Num(); ++Idx)
	{
		OutValues[Idx] = FVector3f::ZeroVector;
	}

	if (Mesh)
	{
		if (const UE::Geometry::FDynamicMesh3* InDynMeshPtr = Mesh->GetDynamicMesh())
		{
			UE::Geometry::FDynamicMeshAABBTree3 AABBTree(InDynMeshPtr);

			if (Positions.Num() == OutValues.Num())
			{
				if (OutputType == EDataflowMeshVectorSamplerOutputType::VectorToClosestPoint)
				{
					for (int32 Idx = 0; Idx < Positions.Num(); ++Idx)
					{
						FVector NearestPoint = AABBTree.FindNearestPoint(FVector(Positions[Idx]));

						FVector ToNearestPointVector = NearestPoint - FVector(Positions[Idx]);

						OutValues[Idx] = FVector3f(ToNearestPointVector);
					}

					return;
				}
				else if (OutputType == EDataflowMeshVectorSamplerOutputType::ClosestPoint)
				{
					for (int32 Idx = 0; Idx < Positions.Num(); ++Idx)
					{
						FVector NearestPoint = AABBTree.FindNearestPoint(FVector(Positions[Idx]));

						OutValues[Idx] = FVector3f(NearestPoint);
					}

					return;
				}
				else if (OutputType == EDataflowMeshVectorSamplerOutputType::ClosestTriangleNormal)
				{
					for (int32 Idx = 0; Idx < Positions.Num(); ++Idx)
					{
						double SquaredDist;
						const int32 NearestTriangleID = AABBTree.FindNearestTriangle(FVector(Positions[Idx]), SquaredDist);

						if (InDynMeshPtr->IsTriangle(NearestTriangleID))
						{
							FVector NearestTriangleNormal = InDynMeshPtr->GetTriNormal(NearestTriangleID);

							OutValues[Idx] = FVector3f(NearestTriangleNormal);
						}
					}

					return;
				}
				else if (OutputType == EDataflowMeshVectorSamplerOutputType::UV)
				{
					TArray<FVector2f> UVs;
					if (UE::Dataflow::Mesh::GetMeshUVs(InDynMeshPtr, UVs, UVLayer))
					{
						// Get UV at closest point
						for (int32 Idx = 0; Idx < Positions.Num(); ++Idx)
						{
							FVector NearestPoint = AABBTree.FindNearestPoint(FVector(Positions[Idx]));

							double SquaredDist;
							const int32 NearestTriangleID = AABBTree.FindNearestTriangle(FVector(Positions[Idx]), SquaredDist);
							
							if (InDynMeshPtr->IsTriangle(NearestTriangleID))
							{
								const UE::Geometry::FIndex3i Tri = InDynMeshPtr->GetTriangle(NearestTriangleID);

								FVector VertexA = InDynMeshPtr->GetVertex(Tri.A);
								FVector VertexB = InDynMeshPtr->GetVertex(Tri.B);
								FVector VertexC = InDynMeshPtr->GetVertex(Tri.C);
								FVector P = NearestPoint;

								FVector2f UV = UE::Dataflow::Mesh::InterpolateAttribute<FVector2f>(VertexA,
									VertexB,
									VertexC,
									NearestPoint,
									UVs[Tri.A],
									UVs[Tri.B],
									UVs[Tri.C],
									FVector2f::ZeroVector);

								OutValues[Idx] = FVector3f(UV.X, UV.Y, 0.0);
							}
						}

						return;
					}
				}
			}
		}
	}	
}

FDataflowMeshVectorSamplerNode::FDataflowMeshVectorSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&MeshSampler.Mesh, GET_MEMBER_NAME_CHECKED(FDataflowMeshVectorSampler, Mesh));
	RegisterOutputConnection(&Sampler);
}

void FDataflowMeshVectorSamplerNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Sampler))
	{
		TSharedRef<FDataflowMeshVectorSampler> Impl = MakeShared<FDataflowMeshVectorSampler>();

		Impl->Mesh = GetValue(Context, &MeshSampler.Mesh);
		Impl->OutputType = MeshSampler.OutputType;
		Impl->UVLayer = MeshSampler.UVLayer;

		FDataflowVectorSampler OutSampler(Impl.ToSharedPtr());

		SetValue(Context, OutSampler, &Sampler);
	}
}