// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowNode.h"
#include "Dataflow/SamplerNodes/DataflowSamplerTypes.h"
#include "Dataflow/DataflowMesh.h"

#include "DataflowMeshSamplerNode.generated.h"

UENUM(BlueprintType)
enum class EDataflowMeshFloatSamplerOutputType : uint8
{
	ClosestPointDistance UMETA(DisplayName = "Closest Point Distance"),
	ClosestTriangleID UMETA(DisplayName = "Closest Triangle ID"),
};

USTRUCT()
struct FDataflowMeshFloatSampler : public FDataflowFloatSamplerBase
{
	GENERATED_BODY();

	virtual ~FDataflowMeshFloatSampler() override = default;
	virtual void Sample(TArrayView<const FVector3f> Positions, TArrayView<float> OutValues) const override;
	virtual FBox GetRenderBounds() const override;

public:
	/** Mesh to sample data from */
	UPROPERTY(meta = (DataflowInput))
	TObjectPtr<UDataflowMesh> Mesh;

	UPROPERTY(EditAnywhere, Category = Sampler, DisplayName = "Sampled Value")
	EDataflowMeshFloatSamplerOutputType OutputType = EDataflowMeshFloatSamplerOutputType::ClosestPointDistance;
};

/**
 * 
 * Mesh float sampler
 * Input: a DataflowMesh
 * Output: for every sampled point certain data from the mesh (ClosestPointDistance, ClosestTriangleID)
 * 
 */
USTRUCT(meta = (Icon = "Icons.EyeDropper"))
struct FDataflowMeshFloatSamplerNode : public FDataflowNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowMeshFloatSamplerNode, "Mesh Float Sampler", "Samplers", "")

public:
	FDataflowMeshFloatSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	UPROPERTY(meta = (DataflowOutput))
	FDataflowFloatSampler Sampler;

	UPROPERTY(EditAnywhere, Category = Sampler, meta = (ShowOnlyInnerProperties, SkipInDisplayNameChain))
	FDataflowMeshFloatSampler MeshSampler;

	void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

// --------------------------------------------------------------------------------------------------------------------

UENUM(BlueprintType)
enum class EDataflowMeshVectorSamplerOutputType : uint8
{
	VectorToClosestPoint UMETA(DisplayName = "Vector To Closest Point"),
	ClosestPoint UMETA(DisplayName = "Closest Point"),
	ClosestTriangleNormal UMETA(DisplayName = "Closest Triangle Normal"),
	UV UMETA(DisplayName = "UV")
};

USTRUCT()
struct FDataflowMeshVectorSampler : public FDataflowVectorSamplerBase
{
	GENERATED_BODY();

	virtual ~FDataflowMeshVectorSampler() override = default;
	virtual void Sample(TArrayView<const FVector3f> Positions, TArrayView<FVector3f> OutValues) const override;
	virtual FBox GetRenderBounds() const override;

public:
	/** Mesh to sample data from */
	UPROPERTY(meta = (DataflowInput))
	TObjectPtr<UDataflowMesh> Mesh;

	UPROPERTY(EditAnywhere, Category = Sampler, DisplayName = "Sampled Value")
	EDataflowMeshVectorSamplerOutputType OutputType = EDataflowMeshVectorSamplerOutputType::VectorToClosestPoint;

	/** UV layer */
	UPROPERTY(meta = (UIMin = 0, ClampMin = 0, EditCondition = "OutputType == EDataflowMeshVectorSamplerOutputType::UV"))
	int32 UVLayer = 0;
};

/**
 *
 * Mesh vector sampler
 * Input: a DataflowMesh
 * Output: for every sampled point certain data from the mesh (VectorToClosestPoint, ClosestTriangleNormal  
 * ClosestPoint, ClosestPointUV etc)
 *
 */
USTRUCT(meta = (Icon = "Icons.EyeDropper"))
struct FDataflowMeshVectorSamplerNode : public FDataflowNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowMeshVectorSamplerNode, "Mesh Vector Sampler", "Samplers", "")

public:
	FDataflowMeshVectorSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	UPROPERTY(meta = (DataflowOutput))
	FDataflowVectorSampler Sampler;

	UPROPERTY(EditAnywhere, Category = Sampler, meta = (ShowOnlyInnerProperties, SkipInDisplayNameChain))
	FDataflowMeshVectorSampler MeshSampler;

	void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};
