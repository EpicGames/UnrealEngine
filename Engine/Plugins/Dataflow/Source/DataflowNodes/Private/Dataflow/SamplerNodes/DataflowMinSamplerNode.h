// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/SamplerNodes/DataflowSamplerTypes.h"
#include "Dataflow/SamplerNodes/DataflowSamplerMultiInput.h"

#include "DataflowMinSamplerNode.generated.h"

USTRUCT()
struct FDataflowMinFloatSampler : public FDataflowFloatSamplerBase
{
	GENERATED_BODY();
	virtual ~FDataflowMinFloatSampler() override = default;
	virtual void Sample(TArrayView<const FVector3f> Positions, TArrayView<float> OutValues) const override;
	virtual FBox GetRenderBounds() const override;

public:
	/** list of samplers to Min results from */
	TArray<TSharedPtr<const FDataflowFloatSamplerBase>> Samplers;
};

USTRUCT()
struct FDataflowMinVectorSampler : public FDataflowVectorSamplerBase
{
	GENERATED_BODY();
	virtual ~FDataflowMinVectorSampler() override = default;
	virtual void Sample(TArrayView<const FVector3f> Positions, TArrayView<FVector3f> OutValues) const override;
	virtual FBox GetRenderBounds() const override;

public:
	/** list of samplers to Min results from */
	TArray<TSharedPtr<const FDataflowVectorSamplerBase>> Samplers;
};

/**
* Min sampler results together
*/
USTRUCT(meta = (Icon = "Icons.EyeDropper"))
struct FDataflowMinSamplerNode : public FDataflowMultiInputSamplerNodeBase
{
	GENERATED_USTRUCT_BODY()

	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowMinSamplerNode, "Min Sampler", "Samplers", "Minimum")

public:
	FDataflowMinSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
	virtual void EvaluateSampler(UE::Dataflow::FContext& Context, FDataflowSamplerTypes::FStorageType& OutSampler) const override;
};

