// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/SamplerNodes/DataflowModuloSamplerNode.h"
#include "Dataflow/SamplerNodes/DataflowSamplerFunctions.h"
#include "Dataflow/DataflowNodeFactory.h"

FBox FDataflowModuloFloatSampler::GetRenderBounds() const
{
	if (FloatSampler.IsValid())
	{
		return FloatSampler->GetRenderBounds();
	}

	return FBox(FVector(-50.0), FVector(50.0));
}

void FDataflowModuloFloatSampler::Sample(TArrayView<const FVector3f> Positions, TArrayView<float> OutValues) const
{
	for (int32 Idx = 0; Idx < OutValues.Num(); ++Idx)
	{
		OutValues[Idx] = 0.f;
	}

	if (FloatSampler.IsValid() && Positions.Num() == OutValues.Num() &&
		Base != 0.0)
	{
		FloatSampler->Sample(Positions, OutValues);

		for (int32 Idx = 0; Idx < OutValues.Num(); ++Idx)
		{
			OutValues[Idx] = FMath::Fmod((OutValues[Idx] - Offset), Base);
		}
	}
}

FDataflowModuloFloatSamplerNode::FDataflowModuloFloatSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Sampler);
	RegisterInputConnection(&ModuloSampler.Base, GET_MEMBER_NAME_CHECKED(FDataflowModuloFloatSampler, Base));
	RegisterInputConnection(&ModuloSampler.Offset, GET_MEMBER_NAME_CHECKED(FDataflowModuloFloatSampler, Offset));

	RegisterOutputConnection(&Sampler, &Sampler);
}

void FDataflowModuloFloatSamplerNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Sampler))
	{
		const FDataflowFloatSampler& InFloatSampler = GetValue(Context, &Sampler);

		TSharedRef<FDataflowModuloFloatSampler> Impl = MakeShared<FDataflowModuloFloatSampler>();

		Impl->Base = GetValue(Context, &ModuloSampler.Base);
		Impl->Offset = GetValue(Context, &ModuloSampler.Offset);
		Impl->FloatSampler = InFloatSampler.GetImpl();


		FDataflowFloatSampler OutSampler(Impl.ToSharedPtr());
		SetValue(Context, OutSampler, &Sampler);
	}
}

FBox FDataflowModuloVectorSampler::GetRenderBounds() const
{
	if (VectorSampler.IsValid())
	{
		return VectorSampler->GetRenderBounds();
	}

	return FBox(FVector(-50.0), FVector(50.0));
}

void FDataflowModuloVectorSampler::Sample(TArrayView<const FVector3f> Positions, TArrayView<FVector3f> OutValues) const
{
	for (int32 Idx = 0; Idx < OutValues.Num(); ++Idx)
	{
		OutValues[Idx] = FVector3f::ZeroVector;
	}

	if (VectorSampler.IsValid() && Positions.Num() == OutValues.Num() &&
		Base.X != 0.0 && Base.Y != 0.0 && Base.Z != 0.0)
	{
		VectorSampler->Sample(Positions, OutValues);

		for (int32 Idx = 0; Idx < OutValues.Num(); ++Idx)
		{			
			FVector SampledValueWithOffset = FVector(OutValues[Idx]) - Offset;

			OutValues[Idx] = FVector3f(FMath::Modulo(SampledValueWithOffset.X, Base.X),
				FMath::Modulo(SampledValueWithOffset.Y, Base.Y),
				FMath::Modulo(SampledValueWithOffset.Z, Base.Z));
		}
	}
}

FDataflowModuloVectorSamplerNode::FDataflowModuloVectorSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Sampler);
	RegisterInputConnection(&ModuloSampler.Base, GET_MEMBER_NAME_CHECKED(FDataflowModuloVectorSampler, Base));
	RegisterInputConnection(&ModuloSampler.Offset, GET_MEMBER_NAME_CHECKED(FDataflowModuloVectorSampler, Offset));

	RegisterOutputConnection(&Sampler, &Sampler);
}

void FDataflowModuloVectorSamplerNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Sampler))
	{
		const FDataflowVectorSampler& InVectorSampler = GetValue(Context, &Sampler);

		TSharedRef<FDataflowModuloVectorSampler> Impl = MakeShared<FDataflowModuloVectorSampler>();

		Impl->Base = GetValue(Context, &ModuloSampler.Base);
		Impl->Offset = GetValue(Context, &ModuloSampler.Offset);
		Impl->VectorSampler = InVectorSampler.GetImpl();

		FDataflowVectorSampler OutSampler(Impl.ToSharedPtr());
		SetValue(Context, OutSampler, &Sampler);
	}
}
