// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowNode.h"
#include "Dataflow/SamplerNodes/DataflowSamplerTypes.h"

#include "DataflowModuloSamplerNode.generated.h"

USTRUCT()
struct FDataflowModuloFloatSampler : public FDataflowFloatSamplerBase
{
	GENERATED_BODY();

	virtual ~FDataflowModuloFloatSampler() override = default;
	virtual void Sample(TArrayView<const FVector3f> Positions, TArrayView<float> OutValues) const override;
	virtual FBox GetRenderBounds() const override;

public:
	// Base for modulo. Base can't be zero
	UPROPERTY(EditAnywhere, Category = "Modulo", meta = (DataflowInput))
	float Base = 1.f;

	// Offset for modulo
	UPROPERTY(EditAnywhere, Category = "Modulo", meta = (DataflowInput))
	float Offset = 0.f;

	TSharedPtr<const FDataflowFloatSamplerBase> FloatSampler;
};

/**
 *
 * ModuloSampler
 * Input(s) : Float Sampler
 * Output(s): Modulo applied on sampled values
 *
 */
USTRUCT(meta = (Icon = "Icons.EyeDropper"))
struct FDataflowModuloFloatSamplerNode : public FDataflowNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowModuloFloatSamplerNode, "Modulo Float Sampler", "Samplers", "")

public:
	FDataflowModuloFloatSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = Sampler))
	FDataflowFloatSampler Sampler;

	UPROPERTY(EditAnywhere, Category = Sampler, meta = (ShowOnlyInnerProperties, SkipInDisplayNameChain))
	FDataflowModuloFloatSampler ModuloSampler;

	void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

USTRUCT()
struct FDataflowModuloVectorSampler : public FDataflowVectorSamplerBase
{
	GENERATED_BODY();

	virtual ~FDataflowModuloVectorSampler() override = default;
	virtual void Sample(TArrayView<const FVector3f> Positions, TArrayView<FVector3f> OutValues) const override;
	virtual FBox GetRenderBounds() const override;

public:
	// Base for component wise modulo, none of the components of Base can be zero 
	UPROPERTY(EditAnywhere, Category = "Modulo", meta = (DataflowInput))
	FVector Base = FVector(2.0);

	// Offset for the modulo
	UPROPERTY(EditAnywhere, Category = "Modulo", meta = (DataflowInput))
	FVector Offset = FVector::ZeroVector;

	TSharedPtr<const FDataflowVectorSamplerBase> VectorSampler;
};

/**
 *
 * ModuloSampler
 * Input(s) : Vector Sampler
 * Output(s): Vector modulo applied on components
 *
 */
USTRUCT(meta = (Icon = "Icons.EyeDropper"))
struct FDataflowModuloVectorSamplerNode : public FDataflowNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowModuloVectorSamplerNode, "Modulo Vector Sampler", "Samplers", "")

public:
	FDataflowModuloVectorSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = Sampler))
	FDataflowVectorSampler Sampler;

	UPROPERTY(EditAnywhere, Category = Sampler, meta = (ShowOnlyInnerProperties, SkipInDisplayNameChain))
	FDataflowModuloVectorSampler ModuloSampler;

	void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};
