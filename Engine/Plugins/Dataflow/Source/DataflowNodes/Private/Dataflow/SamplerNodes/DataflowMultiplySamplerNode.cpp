// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/SamplerNodes/DataflowMultiplySamplerNode.h"
#include "Dataflow/DataflowNodeFactory.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowMultiplySamplerNode)

#define LOCTEXT_NAMESPACE "DataflowMultiplySamplerNode"

FBox FDataflowMultiplyFloatSampler::GetRenderBounds() const
{
	FBox RenderBounds = FBox(ForceInit);

	int32 NumAddedRenderBounds = 0;

	for (const TSharedPtr<const FDataflowFloatSamplerBase>& Sampler : Samplers)
	{
		if (Sampler.IsValid())
		{
			RenderBounds += Sampler->GetRenderBounds();
		}

		NumAddedRenderBounds++;
	}

	if (NumAddedRenderBounds)
	{
		return RenderBounds;
	}
	else
	{
		return FBox(FVector(-50.0), FVector(50.0));
	}
}

void FDataflowMultiplyFloatSampler::Sample(TArrayView<const FVector3f> Positions, TArrayView<float> OutValues) const
{
	for (int32 Index = 0; Index < OutValues.Num(); ++Index)
	{
		OutValues[Index] = 1.f;
	}

	if (Positions.Num() == OutValues.Num())
	{
		TArray<float> OneSamplerValues;

		for (const TSharedPtr<const FDataflowFloatSamplerBase>& Sampler : Samplers)
		{
			OneSamplerValues.Init(0, OutValues.Num());
			Sampler->Sample(Positions, OneSamplerValues);
			for (int32 Index = 0; Index < OutValues.Num(); ++Index)
			{
				OutValues[Index] *= OneSamplerValues[Index];
			}
		}
	}
}

FBox FDataflowMultiplyVectorSampler::GetRenderBounds() const
{
	FBox RenderBounds = FBox(ForceInit);

	int32 NumAddedRenderBounds = 0;

	for (const TSharedPtr<const FDataflowVectorSamplerBase>& Sampler : Samplers)
	{
		if (Sampler.IsValid())
		{
			RenderBounds += Sampler->GetRenderBounds();
		}

		NumAddedRenderBounds++;
	}

	if (NumAddedRenderBounds)
	{
		return RenderBounds;
	}
	else
	{
		return FBox(FVector(-50.0), FVector(50.0));
	}
}

void FDataflowMultiplyVectorSampler::Sample(TArrayView<const FVector3f> Positions, TArrayView<FVector3f> OutValues) const
{
	for (int32 Index = 0; Index < OutValues.Num(); ++Index)
	{
		OutValues[Index] = FVector3f::OneVector;
	}

	if (Positions.Num() == OutValues.Num())
	{
		TArray<FVector3f> OneSamplerValues;

		for (const TSharedPtr<const FDataflowVectorSamplerBase>& Sampler : Samplers)
		{
			OneSamplerValues.Init(FVector3f::ZeroVector, OutValues.Num());
			Sampler->Sample(Positions, OneSamplerValues);
			for (int32 Index = 0; Index < OutValues.Num(); ++Index)
			{
				OutValues[Index].X *= OneSamplerValues[Index].X;
				OutValues[Index].Y *= OneSamplerValues[Index].Y;
				OutValues[Index].Z *= OneSamplerValues[Index].Z;
			}
		}
	}
}

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FDataflowMultiplySamplerNode::FDataflowMultiplySamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowMultiInputSamplerNodeBase(InParam, InGuid)
{
	RegisterInitialConnections();
}

void FDataflowMultiplySamplerNode::EvaluateSampler(UE::Dataflow::FContext& Context, FDataflowSamplerTypes::FStorageType& OutSampler) const
{
	// use the first sampler as a reference
	const int32 NumSamplers = GetNumSamplerInputs();
	if (NumSamplers == 0)
	{
		return;
	}

	const FDataflowSamplerTypes::FStorageType& FirstSampler = GetSamplerInput(Context, 0);
	if (FirstSampler.TryGet<FDataflowFloatSampler>())
	{
		TSharedPtr<FDataflowMultiplyFloatSampler> OutImpl = MakeShared<FDataflowMultiplyFloatSampler>();
		for (int32 Index = 0; Index < NumSamplers; ++Index)
		{
			if (const FDataflowFloatSampler* FloatSampler = GetSamplerInput(Context, Index).TryGet<FDataflowFloatSampler>())
			{
				if (TSharedPtr<const FDataflowFloatSamplerBase> SamplerImpl = FloatSampler->GetImpl())
				{
					OutImpl->Samplers.Add(SamplerImpl);
				}
			}
		}
		OutSampler = FDataflowFloatSampler(OutImpl);
	}
	else if (FirstSampler.TryGet<FDataflowVectorSampler>())
	{
		TSharedPtr<FDataflowMultiplyVectorSampler> OutImpl = MakeShared<FDataflowMultiplyVectorSampler>();
		for (int32 Index = 0; Index < NumSamplers; ++Index)
		{
			if (const FDataflowVectorSampler* VectorSampler = GetSamplerInput(Context, Index).TryGet<FDataflowVectorSampler>())
			{
				if (TSharedPtr<const FDataflowVectorSamplerBase> SamplerImpl = VectorSampler->GetImpl())
				{
					OutImpl->Samplers.Add(SamplerImpl);
				}
			}
		}
		OutSampler = FDataflowVectorSampler(OutImpl);
	}

}

#undef LOCTEXT_NAMESPACE 
