// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 
#include "Dataflow/SamplerNodes/DataflowSamplerTypes.h"
#include "Dataflow/SamplerNodes/DataflowSamplerMultiInput.h"

#include "DataflowMultiplySamplerNode.generated.h"

USTRUCT()
struct FDataflowMultiplyFloatSampler : public FDataflowFloatSamplerBase
{
	GENERATED_BODY();
	virtual ~FDataflowMultiplyFloatSampler() override = default;
	virtual void Sample(TArrayView<const FVector3f> Positions, TArrayView<float> OutValues) const override;
	virtual FBox GetRenderBounds() const override;

public:
	/** list of samplers to add results from */
	TArray<TSharedPtr<const FDataflowFloatSamplerBase>> Samplers;
};

USTRUCT()
struct FDataflowMultiplyVectorSampler : public FDataflowVectorSamplerBase
{
	GENERATED_BODY();
	virtual ~FDataflowMultiplyVectorSampler() override = default;
	virtual void Sample(TArrayView<const FVector3f> Positions, TArrayView<FVector3f> OutValues) const override;
	virtual FBox GetRenderBounds() const override;

public:
	/** list of samplers to add results from */
	TArray<TSharedPtr<const FDataflowVectorSamplerBase>> Samplers;
};

/**
* Multiply sampler results together
*/
USTRUCT(meta = (Icon = "Icons.EyeDropper"))
struct FDataflowMultiplySamplerNode : public FDataflowMultiInputSamplerNodeBase
{
	GENERATED_USTRUCT_BODY()

	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowMultiplySamplerNode, "Multiply Sampler", "Samplers", "* Multiply")

public:
	FDataflowMultiplySamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
	virtual void EvaluateSampler(UE::Dataflow::FContext& Context, FDataflowSamplerTypes::FStorageType& OutSampler) const override;
};

