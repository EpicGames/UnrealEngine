// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowNode.h"
#include "Dataflow/SamplerNodes/DataflowSamplerTypes.h"

#include "DataflowNegateSamplerNode.generated.h"

USTRUCT()
struct FDataflowNegateFloatSampler : public FDataflowFloatSamplerBase
{
	GENERATED_BODY();

	virtual ~FDataflowNegateFloatSampler() override = default;
	virtual void Sample(TArrayView<const FVector3f> Positions, TArrayView<float> OutValues) const override;
	virtual FBox GetRenderBounds() const override;

public:
	/**  */
	TSharedPtr<const FDataflowFloatSamplerBase> Sampler;
};

USTRUCT()
struct FDataflowNegateVectorSampler : public FDataflowVectorSamplerBase
{
	GENERATED_BODY();

	virtual ~FDataflowNegateVectorSampler() override = default;
	virtual void Sample(TArrayView<const FVector3f> Positions, TArrayView<FVector3f> OutValues) const override;
	virtual FBox GetRenderBounds() const override;

public:
	/**  */
	TSharedPtr<const FDataflowVectorSamplerBase> Sampler;
};

/**
 * Output an Negate float sampler
 */
USTRUCT(meta = (Icon = "Icons.EyeDropper"))
struct FDataflowNegateSamplerNode : public FDataflowNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowNegateSamplerNode, "Negate Sampler", "Samplers", "")

public:
	FDataflowNegateSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = Sampler))
	FDataflowSamplerTypes Sampler;

	void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};
