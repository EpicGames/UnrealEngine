// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowNode.h"
#include "Dataflow/SamplerNodes/DataflowSamplerTypes.h"

#include "DataflowNormalizeSamplerNode.generated.h"

USTRUCT()
struct FDataflowNormalizeFloatSampler : public FDataflowFloatSamplerBase
{
	GENERATED_BODY();

	virtual ~FDataflowNormalizeFloatSampler() override = default;
	virtual void Sample(TArrayView<const FVector3f> Positions, TArrayView<float> OutValues) const override;
	virtual FBox GetRenderBounds() const override;

public:
	/**  */
	UPROPERTY(EditAnywhere, Category = Normalize, meta = (DataflowInput))
	float Min = 0.f;

	/**  */
	UPROPERTY(EditAnywhere, Category = Normalize, meta = (DataflowInput))
	float Max = 1.f;
	
	/**  */
	TSharedPtr<const FDataflowFloatSamplerBase> Sampler;
};

USTRUCT()
struct FDataflowNormalizeVectorSampler : public FDataflowVectorSamplerBase
{
	GENERATED_BODY();

	virtual ~FDataflowNormalizeVectorSampler() override = default;
	virtual void Sample(TArrayView<const FVector3f> Positions, TArrayView<FVector3f> OutValues) const override;
	virtual FBox GetRenderBounds() const override;

public:
	/**  */
	TSharedPtr<const FDataflowVectorSamplerBase> Sampler;
};

/**
 * Output a normalize vector sampler
 */
USTRUCT(meta = (Icon = "Icons.EyeDropper"))
struct FDataflowNormalizeSamplerNode : public FDataflowNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowNormalizeSamplerNode, "Normalize Sampler", "Samplers", "Normal")

public:
	FDataflowNormalizeSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = Sampler))
	FDataflowSamplerTypes Sampler;

	void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

