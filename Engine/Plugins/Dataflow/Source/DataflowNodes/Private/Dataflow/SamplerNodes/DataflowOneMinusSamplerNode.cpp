// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/SamplerNodes/DataflowOneMinusSamplerNode.h"
#include "Dataflow/SamplerNodes/DataflowSamplerFunctions.h"
#include "Dataflow/DataflowNodeFactory.h"

FBox FDataflowOneMinusFloatSampler::GetRenderBounds() const
{
	if (Sampler.IsValid())
	{
		return Sampler->GetRenderBounds();
	}

	return FBox(FVector(-50.0), FVector(50.0));
}

void FDataflowOneMinusFloatSampler::Sample(TArrayView<const FVector3f> Positions, TArrayView<float> OutValues) const
{
	if (Sampler.IsValid() && Positions.Num() == OutValues.Num())
	{
		TArray<float> OutValueFromInputSampler;
		OutValueFromInputSampler.SetNumUninitialized(OutValues.Num());
	
		Sampler->Sample(Positions, OutValueFromInputSampler);
	
		for (int32 Idx = 0; Idx < OutValues.Num(); ++Idx)
		{
			OutValues[Idx] = 1.f - OutValueFromInputSampler[Idx];
		}
	}
	else
	{
		for (int32 Idx = 0; Idx < OutValues.Num(); ++Idx)
		{
			OutValues[Idx] = 0.f;
		}
	}
}

FDataflowOneMinusFloatSamplerNode::FDataflowOneMinusFloatSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Sampler);
	RegisterOutputConnection(&Sampler, &Sampler);
}

void FDataflowOneMinusFloatSamplerNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Sampler))
	{
		const FDataflowFloatSampler& FloatSampler = GetValue(Context, &Sampler);

		TSharedRef<FDataflowOneMinusFloatSampler> Impl = MakeShared<FDataflowOneMinusFloatSampler>();

		Impl->Sampler = FloatSampler.GetImpl();

		FDataflowFloatSampler OutSampler(Impl.ToSharedPtr());

		SetValue(Context, OutSampler, &Sampler);
	}
}


