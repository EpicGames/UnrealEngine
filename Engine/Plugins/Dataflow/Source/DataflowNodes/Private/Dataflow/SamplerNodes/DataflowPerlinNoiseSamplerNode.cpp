// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/SamplerNodes/DataflowPerlinNoiseSamplerNode.h"
#include "Dataflow/SamplerNodes/DataflowSamplerFunctions.h"
#include "Dataflow/DataflowNodeFactory.h"

FBox FDataflowPerlinNoiseFloatSampler::GetRenderBounds() const
{
	return RenderBounds;
}

void FDataflowPerlinNoiseFloatSampler::Sample(TArrayView<const FVector3f> Positions, TArrayView<float> OutValues) const
{
	const int32 NumSamples = FMath::Min(Positions.Num(), OutValues.Num());
	
	for (int32 Index = 0; Index < NumSamples; ++Index)
	{
		OutValues[Index] = Amplitude * UE::Dataflow::Sampler::SPerlinNoise(Frequency * FVector(Positions[Index]) + Offset);
	}	
}


FDataflowPerlinNoiseFloatSamplerNode::FDataflowPerlinNoiseFloatSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&PerlinNoiseSampler.Amplitude, GET_MEMBER_NAME_CHECKED(FDataflowPerlinNoiseFloatSampler, Amplitude));
	RegisterInputConnection(&PerlinNoiseSampler.Frequency, GET_MEMBER_NAME_CHECKED(FDataflowPerlinNoiseFloatSampler, Frequency));
	RegisterInputConnection(&PerlinNoiseSampler.Offset, GET_MEMBER_NAME_CHECKED(FDataflowPerlinNoiseFloatSampler, Offset));
	RegisterInputConnection(&PerlinNoiseSampler.RenderBounds, GET_MEMBER_NAME_CHECKED(FDataflowPerlinNoiseFloatSampler, RenderBounds))
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterOutputConnection(&Sampler);
}

void FDataflowPerlinNoiseFloatSamplerNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Sampler))
	{
		TSharedRef<FDataflowPerlinNoiseFloatSampler> Impl = MakeShared<FDataflowPerlinNoiseFloatSampler>(PerlinNoiseSampler);
		Impl->Amplitude = GetValue(Context, &PerlinNoiseSampler.Amplitude);
		Impl->Frequency = GetValue(Context, &PerlinNoiseSampler.Frequency);
		Impl->Offset = GetValue(Context, &PerlinNoiseSampler.Offset);
		Impl->RenderBounds = GetValue(Context, &PerlinNoiseSampler.RenderBounds);

		FDataflowFloatSampler OutSampler(Impl.ToSharedPtr());
		SetValue(Context, OutSampler, &Sampler);
	}
}

FBox FDataflowPerlinNoiseVectorSampler::GetRenderBounds() const
{
	return RenderBounds;
}

void FDataflowPerlinNoiseVectorSampler::Sample(TArrayView<const FVector3f> Positions, TArrayView<FVector3f> OutValues) const
{
	const int32 NumSamples = FMath::Min(Positions.Num(), OutValues.Num());

	for (int32 Index = 0; Index < NumSamples; ++Index)
	{
		OutValues[Index] = Amplitude * FVector3f(UE::Dataflow::Sampler::VPerlinNoise(Frequency * FVector(Positions[Index]) + Offset));
	}
}

FDataflowPerlinNoiseVectorSamplerNode::FDataflowPerlinNoiseVectorSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&PerlinNoiseSampler.Amplitude, GET_MEMBER_NAME_CHECKED(FDataflowPerlinNoiseVectorSampler, Amplitude));
	RegisterInputConnection(&PerlinNoiseSampler.Frequency, GET_MEMBER_NAME_CHECKED(FDataflowPerlinNoiseVectorSampler, Frequency));
	RegisterInputConnection(&PerlinNoiseSampler.Offset, GET_MEMBER_NAME_CHECKED(FDataflowPerlinNoiseVectorSampler, Offset));
	RegisterInputConnection(&PerlinNoiseSampler.RenderBounds, GET_MEMBER_NAME_CHECKED(FDataflowPerlinNoiseVectorSampler, RenderBounds))
		.SetCanHidePin(true)
		.SetPinIsHidden(true);

	RegisterOutputConnection(&Sampler);
}

void FDataflowPerlinNoiseVectorSamplerNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Sampler))
	{
		TSharedRef<FDataflowPerlinNoiseVectorSampler> Impl = MakeShared<FDataflowPerlinNoiseVectorSampler>(PerlinNoiseSampler);
		Impl->Amplitude = GetValue(Context, &PerlinNoiseSampler.Amplitude);
		Impl->Frequency = GetValue(Context, &PerlinNoiseSampler.Frequency);
		Impl->Offset = GetValue(Context, &PerlinNoiseSampler.Offset);
		Impl->RenderBounds = GetValue(Context, &PerlinNoiseSampler.RenderBounds);

		FDataflowVectorSampler OutSampler(Impl.ToSharedPtr());
		SetValue(Context, OutSampler, &Sampler);
	}
}

