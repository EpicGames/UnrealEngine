// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowNode.h"
#include "Dataflow/SamplerNodes/DataflowSamplerTypes.h"

#include "DataflowPerlinNoiseSamplerNode.generated.h"

USTRUCT()
struct FDataflowPerlinNoiseFloatSampler : public FDataflowFloatSamplerBase
{
	GENERATED_BODY();

	virtual ~FDataflowPerlinNoiseFloatSampler() override = default;
	virtual void Sample(TArrayView<const FVector3f> Positions, TArrayView<float> OutValues) const override;
	virtual FBox GetRenderBounds() const override;

public:
	UPROPERTY(EditAnywhere, Category = "Perlin Noise", meta = (DataflowInput, UIMin = 0, ClampMin = 0))
	float Amplitude = 1.f;

	// 
	UPROPERTY(EditAnywhere, Category = "Perlin Noise", meta = (DataflowInput, UIMin = 0, ClampMin = 0))
	float Frequency = 1.f;

	// 
	UPROPERTY(EditAnywhere, Category = "Perlin Noise", meta = (DataflowInput, UIMin = 0, ClampMin = 0))
	FVector Offset = FVector(0.f);

	UPROPERTY(EditAnywhere, Category = RenderBounds, meta = (DataflowInput))
	FBox RenderBounds = FBox(FVector(-50.0), FVector(50.0));
};

/**
 * Output a Perlin Noise float sampler
 */
USTRUCT(meta = (Icon = "Icons.EyeDropper"))
struct FDataflowPerlinNoiseFloatSamplerNode : public FDataflowNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowPerlinNoiseFloatSamplerNode, "Perlin Noise Float Sampler", "Samplers", "")

public:
	FDataflowPerlinNoiseFloatSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	UPROPERTY(meta = (DataflowOutput))
	FDataflowFloatSampler Sampler;

	UPROPERTY(EditAnywhere, Category = Sampler, meta = (ShowOnlyInnerProperties, SkipInDisplayNameChain))
	FDataflowPerlinNoiseFloatSampler PerlinNoiseSampler;

	void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

USTRUCT()
struct FDataflowPerlinNoiseVectorSampler : public FDataflowVectorSamplerBase
{
	GENERATED_BODY();

	virtual ~FDataflowPerlinNoiseVectorSampler() override = default;
	virtual void Sample(TArrayView<const FVector3f> Positions, TArrayView<FVector3f> OutValues) const override;
	virtual FBox GetRenderBounds() const override;

public:
	UPROPERTY(EditAnywhere, Category = "Perlin Noise", meta = (DataflowInput, UIMin = 0, ClampMin = 0))
	float Amplitude = 1.f;

	// 
	UPROPERTY(EditAnywhere, Category = "Perlin Noise", meta = (DataflowInput, UIMin = 0, ClampMin = 0))
	float Frequency = 1.f;

	// 
	UPROPERTY(EditAnywhere, Category = "Perlin Noise", meta = (DataflowInput, UIMin = 0, ClampMin = 0))
	FVector Offset = FVector(0.f);

	UPROPERTY(EditAnywhere, Category = RenderBounds, meta = (DataflowInput))
	FBox RenderBounds = FBox(FVector(-50.0), FVector(50.0));
};

/**
 * Output a Perlin Noise vector sampler
 */
USTRUCT(meta = (Icon = "Icons.EyeDropper"))
struct FDataflowPerlinNoiseVectorSamplerNode : public FDataflowNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowPerlinNoiseVectorSamplerNode, "Perlin Noise Vector Sampler", "Samplers", "")

public:
	FDataflowPerlinNoiseVectorSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	UPROPERTY(meta = (DataflowOutput))
	FDataflowVectorSampler Sampler;

	UPROPERTY(EditAnywhere, Category = Sampler, meta = (ShowOnlyInnerProperties, SkipInDisplayNameChain))
	FDataflowPerlinNoiseVectorSampler PerlinNoiseSampler;

	void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

