// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/SamplerNodes/DataflowRandomSamplerNode.h"
#include "Dataflow/SamplerNodes/DataflowSamplerFunctions.h"
#include "Dataflow/DataflowNodeFactory.h"

FBox FDataflowRandomFloatSampler::GetRenderBounds() const
{
	return RenderBounds;
}

void FDataflowRandomFloatSampler::Sample(TArrayView<const FVector3f> Positions, TArrayView<float> OutValues) const
{
	const int32 NumSamples = FMath::Min(Positions.Num(), OutValues.Num());
	
	for (int32 Index = 0; Index < NumSamples; ++Index)
	{
		OutValues[Index] = UE::Dataflow::Sampler::FRandom(FVector(Positions[Index]), Min, Max, Seed);
	}
}

FDataflowRandomFloatSamplerNode::FDataflowRandomFloatSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&RandomSampler.Seed, GET_MEMBER_NAME_CHECKED(FDataflowRandomFloatSampler, Seed));
	RegisterInputConnection(&RandomSampler.Min, GET_MEMBER_NAME_CHECKED(FDataflowRandomFloatSampler, Min));
	RegisterInputConnection(&RandomSampler.Max, GET_MEMBER_NAME_CHECKED(FDataflowRandomFloatSampler, Max));
	RegisterInputConnection(&RandomSampler.RenderBounds, GET_MEMBER_NAME_CHECKED(FDataflowRandomFloatSampler, RenderBounds))
		.SetCanHidePin(true)
		.SetPinIsHidden(true);

	RegisterOutputConnection(&Sampler);
}

void FDataflowRandomFloatSamplerNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Sampler))
	{
		TSharedRef<FDataflowRandomFloatSampler> Impl = MakeShared<FDataflowRandomFloatSampler>(RandomSampler);
		Impl->Seed = GetValue(Context, &RandomSampler.Seed);
		Impl->Min = GetValue(Context, &RandomSampler.Min);
		Impl->Max = GetValue(Context, &RandomSampler.Max);
		Impl->RenderBounds = GetValue(Context, &RandomSampler.RenderBounds);

		FDataflowFloatSampler OutSampler(Impl.ToSharedPtr());
		SetValue(Context, OutSampler, &Sampler);
	}
}

FBox FDataflowRandomVectorSampler::GetRenderBounds() const
{
	return RenderBounds;
}

void FDataflowRandomVectorSampler::Sample(TArrayView<const FVector3f> Positions, TArrayView<FVector3f> OutValues) const
{
	const int32 NumSamples = FMath::Min(Positions.Num(), OutValues.Num());

	for (int32 Index = 0; Index < NumSamples; ++Index)
	{
		OutValues[Index] = FVector3f(UE::Dataflow::Sampler::VRandom(FVector(Positions[Index]), Min, Max, Seed));
	}
}

FDataflowRandomVectorSamplerNode::FDataflowRandomVectorSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&RandomSampler.Seed, GET_MEMBER_NAME_CHECKED(FDataflowRandomVectorSampler, Seed));
	RegisterInputConnection(&RandomSampler.Min, GET_MEMBER_NAME_CHECKED(FDataflowRandomVectorSampler, Min));
	RegisterInputConnection(&RandomSampler.Max, GET_MEMBER_NAME_CHECKED(FDataflowRandomVectorSampler, Max));
	RegisterInputConnection(&RandomSampler.RenderBounds, GET_MEMBER_NAME_CHECKED(FDataflowRandomVectorSampler, RenderBounds))
		.SetCanHidePin(true)
		.SetPinIsHidden(true);

	RegisterOutputConnection(&Sampler);
}

void FDataflowRandomVectorSamplerNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Sampler))
	{
		TSharedRef<FDataflowRandomVectorSampler> Impl = MakeShared<FDataflowRandomVectorSampler>(RandomSampler);
		Impl->Seed = GetValue(Context, &RandomSampler.Seed);
		Impl->Min = GetValue(Context, &RandomSampler.Min);
		Impl->Max = GetValue(Context, &RandomSampler.Max);
		Impl->RenderBounds = GetValue(Context, &RandomSampler.RenderBounds);

		FDataflowVectorSampler OutSampler(Impl.ToSharedPtr());
		SetValue(Context, OutSampler, &Sampler);
	}
}

