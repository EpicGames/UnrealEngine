// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowNode.h"
#include "Dataflow/SamplerNodes/DataflowSamplerTypes.h"

#include "DataflowRandomSamplerNode.generated.h"

USTRUCT()
struct FDataflowRandomFloatSampler : public FDataflowFloatSamplerBase
{
	GENERATED_BODY();

	virtual ~FDataflowRandomFloatSampler() override = default;
	virtual void Sample(TArrayView<const FVector3f> Positions, TArrayView<float> OutValues) const override;
	virtual FBox GetRenderBounds() const override;

public:
	// Random seed
	UPROPERTY(EditAnywhere, Category = "Random", meta = (DataflowInput, UIMin = 0, ClampMin = 0))
	int32 Seed = 0.f;
	
	// Random Min
	UPROPERTY(EditAnywhere, Category = "Random", meta = (DataflowInput))
	float Min = 0.f;
	
	// Random Max
	UPROPERTY(EditAnywhere, Category = "Random", meta = (DataflowInput))
	float Max = 1.f;

	UPROPERTY(EditAnywhere, Category = RenderBounds, meta = (DataflowInput))
	FBox RenderBounds = FBox(FVector(-50.0), FVector(50.0));
};

/**
 * Output a Random float sampler
 */
USTRUCT(meta = (Icon = "Icons.EyeDropper"))
struct FDataflowRandomFloatSamplerNode : public FDataflowNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowRandomFloatSamplerNode, "Random Float Sampler", "Samplers", "")

public:
	FDataflowRandomFloatSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	UPROPERTY(meta = (DataflowOutput))
	FDataflowFloatSampler Sampler;

	UPROPERTY(EditAnywhere, Category = Sampler, meta = (ShowOnlyInnerProperties, SkipInDisplayNameChain))
	FDataflowRandomFloatSampler RandomSampler;

	void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

USTRUCT()
struct FDataflowRandomVectorSampler : public FDataflowVectorSamplerBase
{
	GENERATED_BODY();

	virtual ~FDataflowRandomVectorSampler() override = default;
	virtual void Sample(TArrayView<const FVector3f> Positions, TArrayView<FVector3f> OutValues) const override;
	virtual FBox GetRenderBounds() const override;

public:
	// Random seed
	UPROPERTY(EditAnywhere, Category = "Random", meta = (DataflowInput, UIMin = 0, ClampMin = 0))
	int32 Seed = 0.f;

	// Random Min
	UPROPERTY(EditAnywhere, Category = "Random", meta = (DataflowInput))
	float Min = 0.f;

	// Random Max
	UPROPERTY(EditAnywhere, Category = "Random", meta = (DataflowInput))
	float Max = 1.f;

	UPROPERTY(EditAnywhere, Category = RenderBounds, meta = (DataflowInput))
	FBox RenderBounds = FBox(FVector(-50.0), FVector(50.0));
};

/**
 * Output a random vector sampler
 */
USTRUCT(meta = (Icon = "Icons.EyeDropper"))
struct FDataflowRandomVectorSamplerNode : public FDataflowNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowRandomVectorSamplerNode, "Random Vector Sampler", "Samplers", "")

public:
	FDataflowRandomVectorSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	UPROPERTY(meta = (DataflowOutput))
	FDataflowVectorSampler Sampler;

	UPROPERTY(EditAnywhere, Category = Sampler, meta = (ShowOnlyInnerProperties, SkipInDisplayNameChain))
	FDataflowRandomVectorSampler RandomSampler;

	void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

