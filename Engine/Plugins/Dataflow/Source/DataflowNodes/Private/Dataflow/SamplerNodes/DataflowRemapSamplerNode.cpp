// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/SamplerNodes/DataflowRemapSamplerNode.h"
#include "Dataflow/SamplerNodes/DataflowSamplerFunctions.h"
#include "Dataflow/DataflowNodeFactory.h"

FBox FDataflowRemapFloatSampler::GetRenderBounds() const
{
	if (Sampler.IsValid())
	{
		return Sampler->GetRenderBounds();
	}

	return FBox(FVector(-50.0), FVector(50.0));
}

void FDataflowRemapFloatSampler::Sample(TArrayView<const FVector3f> Positions, TArrayView<float> OutValues) const
{
	if (Sampler.IsValid() && Positions.Num() == OutValues.Num())
	{
		TArray<float> OutValueFromInputSampler;
		OutValueFromInputSampler.SetNumUninitialized(OutValues.Num());
	
		Sampler->Sample(Positions, OutValueFromInputSampler);
	
		for (int32 Idx = 0; Idx < OutValues.Num(); ++Idx)
		{
			OutValues[Idx] = UE::Dataflow::Sampler::Remap(OutValueFromInputSampler[Idx], OriginalMin, OriginalMax, NewMin, NewMax, Power, bClamp);
		}
	}
	else
	{
		for (int32 Idx = 0; Idx < OutValues.Num(); ++Idx)
		{
			OutValues[Idx] = 0.f;
		}
	}
}

FDataflowRemapFloatSamplerNode::FDataflowRemapFloatSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Sampler);
	RegisterInputConnection(&RemapSampler.OriginalMin, GET_MEMBER_NAME_CHECKED(FDataflowRemapFloatSampler, OriginalMin));
	RegisterInputConnection(&RemapSampler.OriginalMax, GET_MEMBER_NAME_CHECKED(FDataflowRemapFloatSampler, OriginalMax));
	RegisterInputConnection(&RemapSampler.NewMin, GET_MEMBER_NAME_CHECKED(FDataflowRemapFloatSampler, NewMin));
	RegisterInputConnection(&RemapSampler.NewMax, GET_MEMBER_NAME_CHECKED(FDataflowRemapFloatSampler, NewMax));
	RegisterInputConnection(&RemapSampler.Power, GET_MEMBER_NAME_CHECKED(FDataflowRemapFloatSampler, Power));
	RegisterInputConnection(&RemapSampler.bClamp, GET_MEMBER_NAME_CHECKED(FDataflowRemapFloatSampler, bClamp));
	RegisterOutputConnection(&Sampler, &Sampler);
}

void FDataflowRemapFloatSamplerNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Sampler))
	{
		const FDataflowFloatSampler& FloatSampler = GetValue(Context, &Sampler);

		TSharedRef<FDataflowRemapFloatSampler> Impl = MakeShared<FDataflowRemapFloatSampler>();
		Impl->OriginalMin = GetValue(Context, &RemapSampler.OriginalMin);
		Impl->OriginalMax = GetValue(Context, &RemapSampler.OriginalMax);
		Impl->NewMin = GetValue(Context, &RemapSampler.NewMin);
		Impl->NewMax = GetValue(Context, &RemapSampler.NewMax);
		Impl->Power = GetValue(Context, &RemapSampler.Power);
		Impl->bClamp = GetValue(Context, &RemapSampler.bClamp);

		Impl->Sampler = FloatSampler.GetImpl();

		FDataflowFloatSampler OutSampler(Impl.ToSharedPtr());

		SetValue(Context, OutSampler, &Sampler);
	}
}


