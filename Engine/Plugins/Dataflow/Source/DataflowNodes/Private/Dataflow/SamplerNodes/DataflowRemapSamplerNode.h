// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowNode.h"
#include "Dataflow/SamplerNodes/DataflowSamplerTypes.h"

#include "DataflowRemapSamplerNode.generated.h"


USTRUCT()
struct FDataflowRemapFloatSampler : public FDataflowFloatSamplerBase
{
	GENERATED_BODY();

	virtual ~FDataflowRemapFloatSampler() override = default;
	virtual void Sample(TArrayView<const FVector3f> Positions, TArrayView<float> OutValues) const override;
	virtual FBox GetRenderBounds() const override;

public:
	/**  */
	UPROPERTY(EditAnywhere, Category = Remap, meta = (DataflowInput))
	float OriginalMin = 0.f;
	
	/**  */
	UPROPERTY(EditAnywhere, Category = Remap, meta = (DataflowInput))
	float OriginalMax = 1.f;
	
	/**  */
	UPROPERTY(EditAnywhere, Category = Remap, meta = (DataflowInput))
	float NewMin = 0.f;
	
	/**  */
	UPROPERTY(EditAnywhere, Category = Remap, meta = (DataflowInput))
	float NewMax = 2.f;
	
	/**  */
	UPROPERTY(EditAnywhere, Category = Remap, meta = (DataflowInput))
	float Power = 1.f;
	
	/**  */
	UPROPERTY(EditAnywhere, Category = Remap)
	bool bClamp = false;

	/**  */
	TSharedPtr<const FDataflowFloatSamplerBase> Sampler;
};

/**
 * Output an Remap float sampler
 */
USTRUCT(meta = (Icon = "Icons.EyeDropper"))
struct FDataflowRemapFloatSamplerNode : public FDataflowNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowRemapFloatSamplerNode, "Remap Float Sampler", "Samplers", "map")

public:
	FDataflowRemapFloatSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = Sampler))
	FDataflowFloatSampler Sampler;

	UPROPERTY(EditAnywhere, Category = Sampler, meta = (ShowOnlyInnerProperties, SkipInDisplayNameChain))
	FDataflowRemapFloatSampler RemapSampler;

	void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};
