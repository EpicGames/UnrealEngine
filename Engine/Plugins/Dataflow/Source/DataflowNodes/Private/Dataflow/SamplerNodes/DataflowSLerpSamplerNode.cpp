// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/SamplerNodes/DataflowSLerpSamplerNode.h"
#include "Dataflow/SamplerNodes/DataflowSamplerFunctions.h"
#include "Dataflow/DataflowNodeFactory.h"

FBox FDataflowSLerpVectorSampler::GetRenderBounds() const
{
	FBox RenderBounds = FBox(ForceInit);

	if (VectorSamplerA.IsValid())
	{
		RenderBounds += VectorSamplerA->GetRenderBounds();
	}

	if (VectorSamplerB.IsValid())
	{
		RenderBounds += VectorSamplerB->GetRenderBounds();
	}

	if (VectorSamplerA.IsValid() || VectorSamplerB.IsValid())
	{
		return RenderBounds;
	}
	else
	{
		return FBox(FVector(-50.0), FVector(50.0));
	}
}

void FDataflowSLerpVectorSampler::Sample(TArrayView<const FVector3f> Positions, TArrayView<FVector3f> OutValues) const
{
	if (VectorSamplerA.IsValid() && Positions.Num() == OutValues.Num())
	{
		VectorSamplerA->Sample(Positions, OutValues);

		if (VectorSamplerB.IsValid())
		{
			TArray<FVector3f> OutValueFromVectorSamplerB;
			OutValueFromVectorSamplerB.SetNumUninitialized(OutValues.Num());

			VectorSamplerB->Sample(Positions, OutValueFromVectorSamplerB);

			if (BlendSampler.IsValid())
			{
				TArray<float> OutValueFromFloatSampler;
				OutValueFromFloatSampler.SetNumUninitialized(OutValues.Num());

				BlendSampler->Sample(Positions, OutValueFromFloatSampler);

				for (int32 Idx = 0; Idx < OutValues.Num(); ++Idx)
				{
					FVector VectorA = FVector(OutValues[Idx]);
					FVector VectorB = FVector(OutValueFromVectorSamplerB[Idx]);
					double Alpha = OutValueFromFloatSampler[Idx];

					double VectorALength = VectorA.Length();
					double VectorBLength = VectorB.Length();

					VectorA.Normalize();
					VectorB.Normalize();

					FQuat QuatA = VectorA.ToOrientationQuat();
					FQuat QuatB = VectorB.ToOrientationQuat();

					FQuat ResultQuat = FQuat::Slerp(QuatA, QuatB, Alpha);
					FVector ResultVector = ResultQuat.GetForwardVector() * FMath::Lerp(VectorALength, VectorBLength, Alpha);

					OutValues[Idx] = FVector3f(ResultVector);
				}
			}
			else
			{
				for (int32 Idx = 0; Idx < OutValues.Num(); ++Idx)
				{
					FVector VectorA = FVector(OutValues[Idx]);
					FVector VectorB = FVector(OutValueFromVectorSamplerB[Idx]);
					double Alpha = Blend;

					double VectorALength = VectorA.Length();
					double VectorBLength = VectorB.Length();

					VectorA.Normalize();
					VectorB.Normalize();

					FQuat QuatA = VectorA.ToOrientationQuat();
					FQuat QuatB = VectorB.ToOrientationQuat();

					FQuat ResultQuat = FQuat::Slerp(QuatA, QuatB, Alpha);
					FVector ResultVector = ResultQuat.GetForwardVector() * FMath::Lerp(VectorALength, VectorBLength, Alpha);

					OutValues[Idx] = FVector3f(ResultVector);
				}
			}
		}
	}
	else
	{
		for (int32 Idx = 0; Idx < OutValues.Num(); ++Idx)
		{
			OutValues[Idx] = FVector3f::ZeroVector;
		}
	}
}

FDataflowSLerpSamplerNode::FDataflowSLerpSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&SamplerA);
	RegisterInputConnection(&SamplerB);
	RegisterInputConnection(&BlendSampler);

	RegisterInputConnection(&Blend);

	RegisterOutputConnection(&SamplerA)
		.SetPassthroughInput(&SamplerA);
}

void FDataflowSLerpSamplerNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&SamplerA))
	{
		const float InBlend = GetValue(Context, &Blend);

		const FDataflowFloatSampler& InBlendSampler = GetValue(Context, &BlendSampler);
		const FDataflowVectorSampler& VectorSamplerA = GetValue(Context, &SamplerA);
		const FDataflowVectorSampler& VectorSamplerB = GetValue(Context, &SamplerB);

		TSharedRef<FDataflowSLerpVectorSampler> Impl = MakeShared<FDataflowSLerpVectorSampler>();

		Impl->VectorSamplerA = VectorSamplerA.GetImpl();
		Impl->VectorSamplerB = VectorSamplerB.GetImpl();
		Impl->Blend = InBlend;

		Impl->BlendSampler = InBlendSampler.GetImpl();

		FDataflowVectorSampler OutSampler(Impl.ToSharedPtr());

		SetValue(Context, OutSampler, &SamplerA);
	}
}
