// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowNode.h"
#include "Dataflow/SamplerNodes/DataflowSamplerTypes.h"

#include "DataflowSLerpSamplerNode.generated.h"

USTRUCT()
struct FDataflowSLerpVectorSampler : public FDataflowVectorSamplerBase
{
	GENERATED_BODY();

	virtual ~FDataflowSLerpVectorSampler() override = default;
	virtual void Sample(TArrayView<const FVector3f> Positions, TArrayView<FVector3f> OutValues) const override;
	virtual FBox GetRenderBounds() const override;

public:
	UPROPERTY()
	float Blend = 0.f;

	TSharedPtr<const FDataflowVectorSamplerBase> VectorSamplerA;

	TSharedPtr<const FDataflowVectorSamplerBase> VectorSamplerB;

	TSharedPtr<const FDataflowFloatSamplerBase> BlendSampler;
};

/**
 *
 * SLerpSampler
 * Input(s) : 2 Vector Samplers, optional BlendSampler (if not connected blend value will be used)
 * Output(s): Vector sampler outputting the spherical linear interpolated value of the incoming vectors
 *
 */
USTRUCT(meta = (Icon = "Icons.EyeDropper"))
struct FDataflowSLerpSamplerNode : public FDataflowNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowSLerpSamplerNode, "SLerp Sampler", "Samplers", "Linear Interpolation")

public:
	FDataflowSLerpSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	/** First vector input for slerp */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = SamplerA))
	FDataflowVectorSampler SamplerA;

	/** Second vector input for slerp */
	UPROPERTY(meta = (DataflowInput))
	FDataflowVectorSampler SamplerB;

	/** Optional FloatSampler to control blend per sampled vectors */
	UPROPERTY(meta = (DataflowInput))
	FDataflowFloatSampler BlendSampler;

	/** If BlendSampler is not connected this value controls the blend */
	UPROPERTY(EditAnywhere, Category = SLerp, meta = (DataflowInput, UIMin = 0.f, ClampMin = 0.f, UIMax = 1.f, ClampMax = 1.f))
	float Blend = 0.f;

	void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};
