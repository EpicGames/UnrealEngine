// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/SamplerNodes/DataflowSamplerFunctions.h"
#include "Math/RandomStream.h"

namespace UE::Dataflow::Sampler
{
	float FRandom(const FVector InPosition, float InRandomMin, float InRandomMax, const int32 InSeed)
	{
		const uint32 SeedX = uint32(FMath::Abs(InPosition.X));
		const uint32 SeedY = uint32(FMath::Abs(InPosition.Y));
		const uint32 SeedZ = uint32(FMath::Abs(InPosition.Z));
	
		const uint32 SeedAll = ::HashCombineFast(SeedX, SeedY, SeedZ, InSeed);
	
		FRandomStream RandStream((int32)SeedAll);

		if (InRandomMin > InRandomMax)
		{
			InRandomMax = InRandomMin + 0.1f;
		}
		
		return RandStream.FRandRange(InRandomMin, InRandomMax);
	}

	FVector VRandom(const FVector InPosition, float InRandomMin, float InRandomMax, const int32 InSeed)
	{
		const uint32 SeedX = uint32(FMath::Abs(InPosition.X));
		const uint32 SeedY = uint32(FMath::Abs(InPosition.Y));
		const uint32 SeedZ = uint32(FMath::Abs(InPosition.Z));

		const uint32 SeedAll = ::HashCombineFast(SeedX, SeedY, SeedZ, InSeed);

		FRandomStream RandStream((int32)SeedAll);

		if (InRandomMin > InRandomMax)
		{
			InRandomMax = InRandomMin + 0.1f;
		}

		return FVector(RandStream.FRandRange(InRandomMin, InRandomMax), 
			RandStream.FRandRange(InRandomMin, InRandomMax), 
			RandStream.FRandRange(InRandomMin, InRandomMax));
	}

	float SPerlinNoise(const FVector& InPosition)
	{
		return FMath::PerlinNoise3D(InPosition);
	}

	FVector VPerlinNoise(const FVector& InPosition)
	{
		return FVector(FMath::PerlinNoise3D(InPosition), 
			FMath::PerlinNoise3D(InPosition + FVector(12.34)), 
			FMath::PerlinNoise3D(InPosition + FVector(112.34f)));
	}

	float fBm(const FVector InPoisition, const float InAmplitude, const float InFrequency, const FVector& InOffset, const int32 InOctaves, const float InLacunarity, const float InGain)
	{
		float Sum = 0.f;
		float Amp = InAmplitude;
		FVector Position = InPoisition;

		for (int32 Idx = 0; Idx < InOctaves; ++Idx)
		{
			Sum += Amp * SPerlinNoise(Position * InFrequency + InOffset);

			Amp *= InGain;
			Position *= InLacunarity;
		}

		return Sum;
	}

	FVector VfBm(const FVector InPoisition, const float InAmplitude, const float InFrequency, const FVector& InOffset, const int32 InOctaves, const float InLacunarity, const float InGain)
	{
		FVector Sum = FVector(0.f);
		float Amp = InAmplitude;
		FVector Position = InPoisition;

		for (int32 Idx = 0; Idx < InOctaves; ++Idx)
		{
			Sum += Amp * VPerlinNoise(Position * InFrequency + InOffset);

			Amp *= InGain;
			Position *= InLacunarity;
		}

		return Sum;
	}

	float Turbulence(const FVector InPoisition, const float InAmplitude, const float InFrequency, const FVector& InOffset, const int32 InOctaves, const float InLacunarity, const float InGain)
	{
		float Sum = 0.f;
		float Amp = InAmplitude;
		FVector Position = InPoisition;

		for (int32 Idx = 0; Idx < InOctaves; ++Idx)
		{
			Sum += Amp * FMath::Abs(SPerlinNoise(Position * InFrequency + InOffset));

			Amp *= InGain;
			Position *= InLacunarity;
		}

		return Sum;
	}

	float Remap(float Value, const float OriginalMin, const float OriginalMax, const float NewMin, const float NewMax, const float Power, const bool bClamp)
	{
		if ((OriginalMax - OriginalMin) > SMALL_NUMBER && (NewMax - NewMin) > SMALL_NUMBER)
		{
			if (bClamp)
			{
				Value = FMath::Clamp(Value, OriginalMin, OriginalMax);
			}
			float D = (Value - OriginalMin) / (OriginalMax - OriginalMin);
			if (D >= 0)
			{
				D = FMath::Pow(D, Power);
				float NewValue = NewMin + D * (NewMax - NewMin);

				return NewValue;
			}
			else
			{
				return NewMin;
			}
		}

		return NewMin;
	}
}
