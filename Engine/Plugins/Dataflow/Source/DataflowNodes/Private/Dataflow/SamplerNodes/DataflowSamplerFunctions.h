// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "HAL/Platform.h"
#include "Math/MathFwd.h"

namespace UE::Dataflow::Sampler
{
	float FRandom(const FVector InPosition, float InRandomMin, float InRandomMax, const int32 InSeed);

	FVector VRandom(const FVector InPosition, float InRandomMin, float InRandomMax, const int32 InSeed);

	float SPerlinNoise(const FVector& InPosition);

	FVector VPerlinNoise(const FVector& InPosition);

	float fBm(const FVector InPoisition, const float InAmplitude, const float InFrequency, const FVector& InOffset, const int32 InOctaves, const float InLacunarity, const float InGain);

	FVector VfBm(const FVector InPoisition, const float InAmplitude, const float InFrequency, const FVector& InOffset, const int32 InOctaves, const float InLacunarity, const float InGain);

	float Turbulence(const FVector InPoisition, const float InAmplitude, const float InFrequency, const FVector& InOffset, const int32 InOctaves, const float InLacunarity, const float InGain);

	float Remap(float Value, const float OriginalMin, const float OriginalMax, const float NewMin, const float NewMax, const float Power, const bool bClamp);
}
