// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/SamplerNodes/DataflowSamplerMultiInput.h"

const FName FDataflowMultiInputSamplerNodeBase::DependencyTypeGroup = "Main";
FDataflowMultiInputSamplerNodeBase::FDataflowMultiInputSamplerNodeBase(const UE::Dataflow::FNodeParameters& Param, FGuid InGuid)
	: FDataflowNode(Param, InGuid)
{
}

void FDataflowMultiInputSamplerNodeBase::RegisterInitialConnections()
{
	// Add two sets of pins to start.
	for (int32 Index = 0; Index < NumInitialInputs; ++Index)
	{
		AddPins();
	}
	RegisterOutputConnection(&Sampler)
		.SetPassthroughInput(GetConnectionReference(0))
		.SetTypeDependencyGroup(DependencyTypeGroup);
	check(NumRequiredDataflowInputs + NumInitialInputs == GetNumInputs()); // Update NumRequiredDataflowInputs when adding more inputs. This is used by Serialize
}

bool FDataflowMultiInputSamplerNodeBase::CanAddPin() const
{
	return true;
}

bool FDataflowMultiInputSamplerNodeBase::CanRemovePin() const
{
	return InputSamplers.Num() > NumInitialInputs;
}

TArray<UE::Dataflow::FPin> FDataflowMultiInputSamplerNodeBase::AddPins()
{
	const int32 Index = InputSamplers.AddDefaulted();
	FDataflowInput& Input = RegisterInputArrayConnection(GetConnectionReference(Index));
	if (Index > 0)
	{
		// Set concrete type the same as Index0.
		FDataflowInput* const Input0 = FindInput(GetConnectionReference(0));
		check(Input0);
		SetConnectionConcreteType(&Input, Input0->GetType(), DependencyTypeGroup);
	}
	else
	{
		Input.SetTypeDependencyGroup(DependencyTypeGroup);
	}
	return { { UE::Dataflow::FPin::EDirection::INPUT, Input.GetType(), Input.GetName() } };
}

TArray<UE::Dataflow::FPin> FDataflowMultiInputSamplerNodeBase::GetPinsToRemove() const
{
	const int32 Index = InputSamplers.Num() - 1;
	check(InputSamplers.IsValidIndex(Index));
	if (const FDataflowInput* const Input = FindInput(GetConnectionReference(Index)))
	{
		return { { UE::Dataflow::FPin::EDirection::INPUT, Input->GetType(), Input->GetName() } };
	}
	return FDataflowNode::GetPinsToRemove();
}

void FDataflowMultiInputSamplerNodeBase::OnPinRemoved(const UE::Dataflow::FPin& Pin)
{
	const int32 Index = InputSamplers.Num() - 1;
	check(InputSamplers.IsValidIndex(Index));
#if DO_CHECK
	const FDataflowInput* const Input = FindInput(GetConnectionReference(Index));
	check(Input);
	check(Input->GetName() == Pin.Name);
	check(Input->GetType() == Pin.Type);
#endif
	InputSamplers.SetNum(Index);
	return FDataflowNode::OnPinRemoved(Pin);
}

void FDataflowMultiInputSamplerNodeBase::PostSerialize(const FArchive& Ar)
{
	if (Ar.IsLoading())
	{
		check(InputSamplers.Num() >= NumInitialInputs);
		for (int32 Index = 0; Index < NumInitialInputs; ++Index)
		{
			check(FindInput(GetConnectionReference(Index)));
		}
		for (int32 Index = NumInitialInputs; Index < InputSamplers.Num(); ++Index)
		{
			FDataflowInput& Input = FindOrRegisterInputArrayConnection(GetConnectionReference(Index));
			// reset the type to allow the type group to be properly set as well 
			SetConnectionConcreteType(&Input, Input.GetType(), DependencyTypeGroup);
		}
		if (Ar.IsTransacting())
		{
			const int32 OrigNumRegisteredInputs = GetNumInputs() - NumRequiredDataflowInputs;
			const int32 OrigNumInputs = InputSamplers.Num();
			if (OrigNumRegisteredInputs > OrigNumInputs)
			{
				// Inputs have been removed.
				// Temporarily expand Inputs so we can get connection references.
				InputSamplers.SetNum(OrigNumRegisteredInputs);
				for (int32 Index = OrigNumInputs; Index < InputSamplers.Num(); ++Index)
				{
					UnregisterInputConnection(GetConnectionReference(Index));
				}
				InputSamplers.SetNum(OrigNumInputs);
			}
		}
		else
		{
			// Index + all Inputs
			ensureAlways(InputSamplers.Num() + NumRequiredDataflowInputs == GetNumInputs());
		}
	}
}

UE::Dataflow::TConnectionReference<FDataflowSamplerTypes> FDataflowMultiInputSamplerNodeBase::GetConnectionReference(int32 Index) const
{
	return { &InputSamplers[Index], Index, &InputSamplers };
}

int32 FDataflowMultiInputSamplerNodeBase::GetNumSamplerInputs() const
{
	return InputSamplers.Num();
}

FDataflowSamplerTypes::FStorageType FDataflowMultiInputSamplerNodeBase::GetSamplerInput(UE::Dataflow::FContext& Context, int32 Index) const
{
	static const FDataflowSamplerTypes::FStorageType EmptyStorage;
	if (InputSamplers.IsValidIndex(Index))
	{
		const UE::Dataflow::TConnectionReference<FDataflowSamplerTypes> InputReference = GetConnectionReference(Index);
		return GetValue(Context, InputReference);
	}
	return EmptyStorage;
}

void FDataflowMultiInputSamplerNodeBase::EvaluateSampler(UE::Dataflow::FContext& Context, FDataflowSamplerTypes::FStorageType& OutSampler) const
{
	// need to be implemented by the derived class
	check(false);
}

void FDataflowMultiInputSamplerNodeBase::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Sampler))
	{
		FDataflowSamplerTypes::FStorageType OutSampler;
		EvaluateSampler(Context, OutSampler);
		SetValue(Context, OutSampler, &Sampler);
	}
}
