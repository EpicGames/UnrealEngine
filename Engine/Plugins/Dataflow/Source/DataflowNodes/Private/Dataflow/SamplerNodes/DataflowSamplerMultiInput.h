// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/SamplerNodes/DataflowSamplerTypes.h"
#include "Dataflow/DataflowNode.h"
#include "Dataflow/DataflowAnyType.h"

#include "DataflowSamplerMultiInput.generated.h"

#define UE_API DATAFLOWNODES_API

USTRUCT()
struct FDataflowMultiInputSamplerNodeBase : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()

public:
	FDataflowMultiInputSamplerNodeBase() = default;
	FDataflowMultiInputSamplerNodeBase(const UE::Dataflow::FNodeParameters& Param, FGuid InGuid = FGuid::NewGuid());

protected:
	void RegisterInitialConnections();
	int32 GetNumSamplerInputs() const;
	FDataflowSamplerTypes::FStorageType GetSamplerInput(UE::Dataflow::FContext& Context, int32 Index) const;
	virtual void EvaluateSampler(UE::Dataflow::FContext& Context, FDataflowSamplerTypes::FStorageType& OutSampler) const;
	virtual void PostSerialize(const FArchive& Ar) override;

private:
	UPROPERTY()
	TArray<FDataflowSamplerTypes> InputSamplers;

	UPROPERTY(meta = (DataflowOutput, DataflowPassthrough = "InputSamplers[0]"))
	FDataflowSamplerTypes Sampler;

	virtual TArray<UE::Dataflow::FPin> AddPins() override;
	virtual bool CanAddPin() const override;
	virtual bool CanRemovePin() const override;
	virtual TArray<UE::Dataflow::FPin> GetPinsToRemove() const override;
	virtual void OnPinRemoved(const UE::Dataflow::FPin& Pin) override;
	void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const;
	UE::Dataflow::TConnectionReference<FDataflowSamplerTypes> GetConnectionReference(int32 Index) const;

	static constexpr int32 NumRequiredDataflowInputs = 0;
	static constexpr int32 NumInitialInputs = 2;
	static const FName DependencyTypeGroup;
};

#undef UE_API
