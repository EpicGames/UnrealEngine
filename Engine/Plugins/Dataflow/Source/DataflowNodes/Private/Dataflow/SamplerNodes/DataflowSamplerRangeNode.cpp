// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/SamplerNodes/DataflowSamplerRangeNode.h"
#include "Dataflow/SamplerNodes/DataflowSamplerFunctions.h"
#include "Dataflow/DataflowNodeFactory.h"
#include "Dataflow/DataflowUtils.h"
#include "GeometryCollection/Facades/PointsFacade.h"
#include "Dataflow/DataflowPoints.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowSamplerRangeNode)

#define LOCTEXT_NAMESPACE "DataflowSamplerRangeNode"

FSamplerRangeDataflowNode::FSamplerRangeDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	static const FName MainTypeGroup("Main");

	RegisterInputConnection(&Sampler)
		.SetTypeDependencyGroup(MainTypeGroup);

	RegisterOutputConnection(&Sampler)
		.SetPassthroughInput(&Sampler)
		.SetTypeDependencyGroup(MainTypeGroup);
	RegisterOutputConnection(&NumSamplePoints);
	RegisterOutputConnection(&MinSampledValue);
	RegisterOutputConnection(&MaxSampledValue);
}

void FSamplerRangeDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&NumSamplePoints) || Out->IsA(&MinSampledValue) || Out->IsA(&MaxSampledValue))
	{
		float MinValue = 0.0;
		float MaxValue = 0.0;
		int32 NumPoints = 0;

		const FDataflowFloatSampler* InFloatSampler = GetValue(Context, &Sampler).TryGet <FDataflowFloatSampler>();
		const FDataflowVectorSampler* InVectorSampler = GetValue(Context, &Sampler).TryGet<FDataflowVectorSampler>();

		FBox RenderBounds;
		if (InFloatSampler)
		{
			RenderBounds = InFloatSampler->GetRenderBounds();
		}
		else if (InVectorSampler)
		{
			RenderBounds = InVectorSampler->GetRenderBounds();
		}

		FVector Extent = RenderBounds.GetExtent();
		if ((Extent.X > 0.0 && Extent.Y > 0.0 && Extent.Z >= 0.0) ||
			(Extent.X >= 0.0 && Extent.Y > 0.0 && Extent.Z > 0.0) ||
			(Extent.X > 0.0 && Extent.Y >= 0.0 && Extent.Z > 0.0))
		{
			FDataflowPoints Points;
			GeometryCollection::Facades::FPointsFacade PointsFacade = Points.GetPointsFacade();

			if (PointsFacade.GeneratePointsInBox(RenderBounds, PointSeparation))
			{
				NumPoints = PointsFacade.GetNumPoints();
				if (NumPoints > 0)
				{
					TArray<FVector3f> SamplePoints = PointsFacade.GetPointsAsFloatArray();

					if (InFloatSampler)
					{
						TArray<float> SampledValues;
						SampledValues.Init(0.0, NumPoints);

						InFloatSampler->Sample(SamplePoints, SampledValues);

						UE::Dataflow::Utils::GetMinMaxInArray(SampledValues, MinValue, MaxValue);
					}
					else if (InVectorSampler)
					{
						TArray<FVector3f> SampledValues;
						SampledValues.Init(FVector3f::ZeroVector, NumPoints);

						InVectorSampler->Sample(SamplePoints, SampledValues);

						TArray<float> SampledVectorLengths;
						SampledVectorLengths.Init(0.0, NumPoints);
						for (int32 Idx = 0; Idx < NumPoints; ++Idx)
						{
							SampledVectorLengths[Idx] = SampledValues[Idx].Length();
						}

						UE::Dataflow::Utils::GetMinMaxInArray(SampledVectorLengths, MinValue, MaxValue);
					}

					SetValue(Context, NumPoints, &NumSamplePoints);
					SetValue(Context, MinValue, &MinSampledValue);
					SetValue(Context, MaxValue, &MaxSampledValue);

					return;
				}
			}
		}

		SetValue(Context, NumPoints, &NumSamplePoints);
		SetValue(Context, MinValue, &MinSampledValue);
		SetValue(Context, MaxValue, &MaxSampledValue);
	}
	else if (Out->IsA(&Sampler))
	{
		SafeForwardInput(Context, &Sampler, &Sampler);
	}
}

#undef LOCTEXT_NAMESPACE

