// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowNode.h"
#include "Dataflow/SamplerNodes/DataflowSamplerTypes.h"

#include "DataflowSamplerRangeNode.generated.h"

 /**
 * If the input is a FloatSampler, it samples a 3D grid of points into the input samplers RenderBounds, samples 
 * the the input sampler using these points and computes the Min/Max values of the sampled values.
 * If the input is a VectorSampler, it samples a 3D grid of points into the input samplers RenderBounds, samples
 * the the input sampler using these points and computes the Min/Max values of the sampled vectors length values.
 */
USTRUCT(meta = (Icon = "Icons.EyeDropper"))
struct FSamplerRangeDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FSamplerRangeDataflowNode, "SamplerRange", "Samplers|Utility", "")

private:
	/** Input sampler */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowIntrinsic, DataflowPassthrough = "Sampler"))
	FDataflowSamplerTypes Sampler;

	/** PointSeparation used for the 3D grid point generation in the input sampler's RenderBounds */
	UPROPERTY(EditAnywhere, Category = "Points", meta = (DisplayName = "Point Separation", UIMin = 0.01f, ClampMin = 0.01f))
	float PointSeparation = 2.f;

	/** Number of sampled points output */
	UPROPERTY(meta = (DataflowOutput))
	int32 NumSamplePoints = 0;

	/** Minimum value of the sampled points output */
	UPROPERTY(meta = (DataflowOutput))
	float MinSampledValue = 0.0;

	/** Maximum value of the sampled points output */
	UPROPERTY(meta = (DataflowOutput))
	float MaxSampledValue = 0.0;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FSamplerRangeDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};
