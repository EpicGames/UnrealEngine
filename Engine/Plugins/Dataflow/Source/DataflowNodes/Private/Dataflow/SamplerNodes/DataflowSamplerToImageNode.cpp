// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/SamplerNodes/DataflowSamplerToImageNode.h"
#include "Dataflow/SamplerNodes/DataflowSamplerFunctions.h"
#include "Dataflow/DataflowNodeFactory.h"

FDataflowSamplerToImageNode::FDataflowSamplerToImageNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Sampler);
	RegisterOutputConnection(&Image);
}

namespace UE::DataflowSamplerToImage::Private
{
	static void GetPointsOnSlicePlane(const FVector& InCenter, const FVector& InExtent, const EDataflowSlicePlaneOrientation& InPlaneOrientation, const EDataflowImageResolution& InResolution, const float InOffset, TArray<FVector>& OutPoints)
	{
		const int32 NumberOfPoints = static_cast<int32>(InResolution);

		if (NumberOfPoints > 1)
		{
			OutPoints.Reserve(NumberOfPoints * NumberOfPoints);
			OutPoints.SetNumUninitialized(NumberOfPoints * NumberOfPoints);

			const double PointDistanceX = InExtent.X / double(NumberOfPoints - 1);
			const double PointDistanceY = InExtent.Y / double(NumberOfPoints - 1);
			const double PointDistanceZ = InExtent.Z / double(NumberOfPoints - 1);

			FVector Min = InCenter - 0.5 * InExtent;
			FVector Max = InCenter + 0.5 * InExtent;

			float X, Y, Z;
			if (InPlaneOrientation == EDataflowSlicePlaneOrientation::XYPlane)
			{
				for (int32 IdxX = 0; IdxX < NumberOfPoints; ++IdxX)
				{
					for (int32 IdxY = 0; IdxY < NumberOfPoints; ++IdxY)
					{
						X = Min.X + float(IdxX) * PointDistanceX;
						Y = Min.Y + float(IdxY) * PointDistanceY;
						Z = InCenter.Z + InOffset * 0.5 * InExtent.Z;

						OutPoints[IdxY + NumberOfPoints * IdxX] = FVector(X, Y, Z);
					}
				}
			}
			else if (InPlaneOrientation == EDataflowSlicePlaneOrientation::YZPlane)
			{
				for (int32 IdxY = 0; IdxY < NumberOfPoints; ++IdxY)
				{
					for (int32 IdxZ = 0; IdxZ < NumberOfPoints; ++IdxZ)
					{
						X = InCenter.X + InOffset * 0.5 * InExtent.X;
						Y = Min.Y + float(IdxY) * PointDistanceY;
						Z = Min.Z + float(IdxZ) * PointDistanceZ;

						OutPoints[IdxZ + NumberOfPoints * IdxY] = FVector(X, Y, Z);
					}
				}
			}
			else if (InPlaneOrientation == EDataflowSlicePlaneOrientation::ZXPlane)
			{
				for (int32 IdxZ = 0; IdxZ < NumberOfPoints; ++IdxZ)
				{
					for (int32 IdxX = 0; IdxX < NumberOfPoints; ++IdxX)
					{
						X = Min.X + float(IdxX) * PointDistanceX;
						Y = InCenter.Y + InOffset * 0.5 * InExtent.Y;
						Z = Min.Z + float(IdxZ) * PointDistanceZ;

						OutPoints[IdxX + NumberOfPoints * IdxZ] = FVector(X, Y, Z);
					}
				}
			}
		}
	}
}

void FDataflowSamplerToImageNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Image))
	{
		const int32 NumberOfPixels = static_cast<int32>(Resolution);

		TArray<FVector> SamplePoints;
		UE::DataflowSamplerToImage::Private::GetPointsOnSlicePlane(Center, Extent, Plane, Resolution, Offset, SamplePoints);

		if (SamplePoints.Num() == NumberOfPixels * NumberOfPixels)
		{
			if (const FDataflowFloatSampler* FloatSampler = GetValue(Context, &Sampler).TryGet<FDataflowFloatSampler>())
			{
				TArray<float> SampledValues;
				SampledValues.Init(0.f, SamplePoints.Num());

				FloatSampler->Sample(TArray<FVector3f>(SamplePoints), SampledValues);

				TArray64<float> Pixels;
				Pixels.Init(0.f, NumberOfPixels * NumberOfPixels);

				int32 Idx = 0;
				for (float& Pixel : Pixels)
				{
					Pixel = SampledValues[Idx++];
				}

				FDataflowImage OutImage;
				OutImage.CreateR32F(Resolution, Pixels);

				SetValue(Context, MoveTemp(OutImage), &Image);
				return;
			}
			else if (const FDataflowVectorSampler* VectorSampler = GetValue(Context, &Sampler).TryGet<FDataflowVectorSampler>())
			{
				TArray<FVector3f> SampledValues;
				SampledValues.Init(FVector3f::ZeroVector, SamplePoints.Num());

				VectorSampler->Sample(TArray<FVector3f>(SamplePoints), SampledValues);

				TArray64<FVector4f> Pixels;
				Pixels.Init(FVector4f(0.f, 0.f, 0.f, 1.f), NumberOfPixels * NumberOfPixels);

				int32 Idx = 0;
				for (FVector4f& Pixel : Pixels)
				{
					FVector3f SampledVector = SampledValues[Idx++];

					if (bNormalize)
					{
						SampledVector.Normalize();
						SampledVector = 0.5f * (SampledVector + FVector3f(1.f, 1.f, 1.f));
					}

					FLinearColor LinearColor = FLinearColor(SampledVector.X, SampledVector.Y, SampledVector.Z);

					Pixel = FVector4f(LinearColor.R, LinearColor.G, LinearColor.B, 1.f);
				}

				FDataflowImage OutImage;
				OutImage.CreateRGBA32F(Resolution, Pixels);

				SetValue(Context, MoveTemp(OutImage), &Image);
				return;
			}
		}

		SetValue(Context, FDataflowImage(), &Image);
	}
}


