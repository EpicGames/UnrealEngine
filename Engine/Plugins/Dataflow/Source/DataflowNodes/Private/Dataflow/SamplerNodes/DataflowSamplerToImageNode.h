// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowNode.h"
#include "Dataflow/SamplerNodes/DataflowSamplerTypes.h"
#include "Dataflow/DataflowImage.h"
#include "Curves/LinearColorRamp.h"

#include "DataflowSamplerToImageNode.generated.h"

UENUM(BlueprintType)
enum class EDataflowSlicePlaneOrientation : uint8
{
	XYPlane UMETA(DisplayName = "XY Plane"),
	YZPlane UMETA(DisplayName = "YZ Plane"),
	ZXPlane UMETA(DisplayName = "ZX Plane"),
};

/**
 *
 * Sampler to Image
 * Input(s) : Float or Vector Sample
 * Output(s): an image created from the sampled values
 *
 */
USTRUCT(meta = (Icon = "Icons.EyeDropper"))
struct FDataflowSamplerToImageNode : public FDataflowNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowSamplerToImageNode, "Sampler To Image", "Samplers", "Texture, Slice")

public:
	FDataflowSamplerToImageNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	/** Sampler input */
	UPROPERTY(meta = (DataflowInput))
	FDataflowSamplerTypes Sampler;

	/** Center of the render bounds */
	UPROPERTY(EditAnywhere, Category = "Render Bounds")
	FVector Center = FVector(0.f);

	/** Extent of the render bounds */
	UPROPERTY(EditAnywhere, Category = "Render Bounds", meta = (UIMin = 0.01f, ClampMin = 0.01f))
	FVector Extent = FVector(50.0f);

	/** Slice plane orientation */
	UPROPERTY(EditAnywhere, Category = "Slice Settings")
	EDataflowSlicePlaneOrientation Plane = EDataflowSlicePlaneOrientation::XYPlane;

	/** Relative offset of the slice plane inside the render bounds. The offset is beetween -1 and 1 */
	UPROPERTY(EditAnywhere, Category = "Slice Settings", meta = (ClampMin = "-1", ClampMax = "1"))
	float Offset = 0.f;

	/** Normalize the incoming vector data, if incoming vector data represents colors this must be turned off */
	UPROPERTY(EditAnywhere, Category = "Normalize")
	bool bNormalize = false;

	/** Output image resolution */
	UPROPERTY(EditAnywhere, Category = "Image");
	EDataflowImageResolution Resolution = EDataflowImageResolution::Resolution256;

	/** Output image */
	UPROPERTY(meta = (DataflowOutput))
	FDataflowImage Image;
	
	void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};
