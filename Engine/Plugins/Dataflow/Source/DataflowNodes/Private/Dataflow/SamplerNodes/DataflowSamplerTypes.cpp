// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/SamplerNodes/DataflowSamplerTypes.h"
#include "Dataflow/DataflowAnyTypeRegistry.h"
#include "Dataflow/SamplerNodes/DataflowGradientSamplerNode.h"
#include "Dataflow/SamplerNodes/DataflowTransformSamplerNode.h"
#include "Dataflow/SamplerNodes/DataflowUniformSamplerNode.h"
#include "Dataflow/SamplerNodes/DataflowRandomSamplerNode.h"
#include "Dataflow/SamplerNodes/DataflowPerlinNoiseSamplerNode.h"
#include "Dataflow/SamplerNodes/DataflowfBmSamplerNode.h"
#include "Dataflow/SamplerNodes/DataflowTurbulenceSamplerNode.h"
#include "Dataflow/SamplerNodes/DataflowAbsSamplerNode.h"
#include "Dataflow/SamplerNodes/DataflowSignSamplerNode.h"
#include "Dataflow/SamplerNodes/DataflowAddSamplerNode.h"
#include "Dataflow/SamplerNodes/DataflowMultiplySamplerNode.h"
#include "Dataflow/SamplerNodes/DataflowMinSamplerNode.h"
#include "Dataflow/SamplerNodes/DataflowMaxSamplerNode.h"
#include "Dataflow/SamplerNodes/DataflowScaleSamplerNode.h"
#include "Dataflow/SamplerNodes/DataflowClampSamplerNode.h"
#include "Dataflow/SamplerNodes/DataflowNegateSamplerNode.h"
#include "Dataflow/SamplerNodes/DataflowNormalizeSamplerNode.h"
#include "Dataflow/SamplerNodes/DataflowOneMinusSamplerNode.h"
#include "Dataflow/SamplerNodes/DataflowRemapSamplerNode.h"
#include "Dataflow/SamplerNodes/DataflowCombineSamplerNode.h"
#include "Dataflow/SamplerNodes/DataflowLerpSamplerNode.h"
#include "Dataflow/SamplerNodes/DataflowSplitSamplerNode.h"
#include "Dataflow/SamplerNodes/DataflowDistanceFromPlaneSamplerNode.h"
#include "Dataflow/SamplerNodes/DataflowMeshSamplerNode.h"
#include "Dataflow/SamplerNodes/DataflowTextureSamplerNode.h"
#include "Dataflow/SamplerNodes/DataflowSamplerToImageNode.h"
#include "Dataflow/SamplerNodes/DataflowColorRampSamplerNode.h"
#include "Dataflow/SamplerNodes/DataflowDistanceFromSphereSamplerNode.h"
#include "Dataflow/SamplerNodes/DataflowDistanceFromBoxSamplerNode.h"
#include "Dataflow/SamplerNodes/DataflowSLerpSamplerNode.h"
#include "Dataflow/SamplerNodes/DataflowTilingSamplerNode.h"
#include "Dataflow/SamplerNodes/DataflowVectorLengthSamplerNode.h"
#include "Dataflow/SamplerNodes/DataflowModuloSamplerNode.h"
#include "Dataflow/SamplerNodes/DataflowStepSamplerNode.h"
#include "Dataflow/SamplerNodes/DataflowSmoothStepSamplerNode.h"
#include "Dataflow/SamplerNodes/DataflowSamplerRangeNode.h"
#include "Dataflow/SamplerNodes/DataflowVectorDeviationSamplerNode.h"
#include "Dataflow/SamplerNodes/DataflowElectricFieldVectorSamplerNode.h"

#include "Dataflow/DataflowNodeFactory.h"
#include "Dataflow/DataflowCategoryRegistry.h"
#include "Dataflow/DataflowNodeColorsRegistry.h"

namespace UE::Dataflow
{
	void RegisterSamplerTypes()
	{
		UE_DATAFLOW_REGISTER_ANYTYPE(FDataflowSamplerTypes);
	}
};

namespace UE::Dataflow
{
	void RegisterSamplerNodes()
	{
		static const FLinearColor CDefaultNodeBodyTintColor = FLinearColor(0.f, 0.f, 0.f, 0.5f);
		static const float CDefaultWireThickness = 2.f;

		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowLinearGradientFloatSamplerNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowLinearGradientFromBoxFloatSamplerNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowRadialGradientFloatSamplerNode);

		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowAbsSamplerNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowAddSamplerNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowClampFloatSamplerNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowfBmFloatSamplerNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowfBmVectorSamplerNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowMaxSamplerNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowMinSamplerNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowMultiplySamplerNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowNegateSamplerNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowNormalizeSamplerNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowOneMinusFloatSamplerNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowPerlinNoiseFloatSamplerNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowPerlinNoiseVectorSamplerNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowRandomFloatSamplerNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowRandomVectorSamplerNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowRemapFloatSamplerNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowScaleVectorSamplerNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowSignSamplerNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowTransformFloatSamplerNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowTurbulenceFloatSamplerNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowUniformFloatSamplerNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowUniformVectorSamplerNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowCombineFloatSamplerNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowLerpSamplerNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowSplitVectorSamplerNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowDistanceFromPlaneFloatSamplerNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowMeshFloatSamplerNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowMeshVectorSamplerNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowTextureVectorSamplerNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowSamplerToImageNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowColorRampSamplerNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowDistanceFromSphereFloatSamplerNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowDistanceFromBoxFloatSamplerNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowSLerpSamplerNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowTilingSamplerNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowVectorLengthSamplerNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowModuloFloatSamplerNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowModuloVectorSamplerNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowStepFloatSamplerNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowStepVectorSamplerNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowSmoothStepFloatSamplerNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowSmoothStepVectorSamplerNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FSamplerRangeDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowVectorDeviationSamplerNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowElectricFieldVectorSamplerNode);
		
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_NODE_COLORS_BY_CATEGORY("Samplers", FLinearColor(1.f, 0.f, 0.93f), CDefaultNodeBodyTintColor);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_PIN_SETTINGS_BY_TYPE("FDataflowFloatSampler", FLinearColor(1.f, 0.3f, 0.95f), CDefaultWireThickness);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_PIN_SETTINGS_BY_TYPE("FDataflowVectorSampler", FLinearColor(0.4f, 0.f, 0.35f), CDefaultWireThickness);
	}
}

FDataflowFloatSampler::FDataflowFloatSampler(const TSharedPtr<const FDataflowFloatSamplerBase>& InImpl)
	: Impl(InImpl)
{
}

void FDataflowFloatSampler::Sample(TArrayView<const FVector3f> Positions, TArrayView<float> OutValues) const
{
	if (Impl)
	{
		Impl->Sample(Positions, OutValues);
	}
}
FBox FDataflowFloatSampler::GetRenderBounds() const
{
	if (Impl)
	{
		return Impl->GetRenderBounds();
	}
	return FDataflowFloatSamplerBase::GetRenderBounds();
}

/////////////////////////////////////////////////////////////////////////////////////////

FDataflowVectorSampler::FDataflowVectorSampler(const TSharedPtr<const FDataflowVectorSamplerBase>& InImpl)
	: Impl(InImpl)
{
}

void FDataflowVectorSampler::Sample(TArrayView<const FVector3f> Positions, TArrayView<FVector3f> OutValues) const
{
	if (Impl)
	{
		Impl->Sample(Positions, OutValues);
	}
}

FBox FDataflowVectorSampler::GetRenderBounds() const
{
	if (Impl)
	{
		return Impl->GetRenderBounds();
	}
	return FDataflowVectorSamplerBase::GetRenderBounds();
}





