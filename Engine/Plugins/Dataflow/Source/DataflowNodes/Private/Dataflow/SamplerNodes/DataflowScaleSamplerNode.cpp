// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/SamplerNodes/DataflowScaleSamplerNode.h"
#include "Dataflow/SamplerNodes/DataflowSamplerFunctions.h"
#include "Dataflow/DataflowNodeFactory.h"

FBox FDataflowScaleVectorSampler::GetRenderBounds() const
{
	FBox RenderBounds = FBox(ForceInit);

	if (VectorSampler.IsValid() || FloatSampler.IsValid())
	{
		if (VectorSampler.IsValid())
		{
			RenderBounds += VectorSampler->GetRenderBounds();
		}

		if (FloatSampler.IsValid())
		{
			RenderBounds += FloatSampler->GetRenderBounds();
		}

		return RenderBounds;
	}

	return FBox(FVector(-50.0), FVector(50.0));
}

void FDataflowScaleVectorSampler::Sample(TArrayView<const FVector3f> Positions, TArrayView<FVector3f> OutValues) const
{
	if (VectorSampler.IsValid() && Positions.Num() == OutValues.Num())
	{
		VectorSampler->Sample(Positions, OutValues);

		if (FloatSampler.IsValid())
		{
			TArray<float> OutValueFromFloatSampler;
			OutValueFromFloatSampler.SetNumUninitialized(OutValues.Num());

			FloatSampler->Sample(Positions, OutValueFromFloatSampler);

			for (int32 Idx = 0; Idx < OutValues.Num(); ++Idx)
			{
				OutValues[Idx] *= OutValueFromFloatSampler[Idx];
			}
		}
	}
	else
	{
		for (int32 Idx = 0; Idx < OutValues.Num(); ++Idx)
		{
			OutValues[Idx] = FVector3f::ZeroVector;
		}
	}
}

FDataflowScaleVectorSamplerNode::FDataflowScaleVectorSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&VectorSampler);
	RegisterInputConnection(&FloatSampler);
	RegisterOutputConnection(&VectorSampler, &VectorSampler);
}

void FDataflowScaleVectorSamplerNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&VectorSampler))
	{
		const FDataflowFloatSampler& InFloatSampler = GetValue(Context, &FloatSampler);
		const FDataflowVectorSampler& InVectorSampler = GetValue(Context, &VectorSampler);

		TSharedRef<FDataflowScaleVectorSampler> Impl = MakeShared<FDataflowScaleVectorSampler>();

		Impl->FloatSampler = InFloatSampler.GetImpl();
		Impl->VectorSampler = InVectorSampler.GetImpl();

		FDataflowVectorSampler OutSampler(Impl.ToSharedPtr());

		SetValue(Context, OutSampler, &VectorSampler);
	}
}
