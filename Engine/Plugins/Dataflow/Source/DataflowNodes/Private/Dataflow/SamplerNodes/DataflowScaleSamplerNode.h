// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowNode.h"
#include "Dataflow/SamplerNodes/DataflowSamplerTypes.h"

#include "DataflowScaleSamplerNode.generated.h"

USTRUCT()
struct FDataflowScaleVectorSampler : public FDataflowVectorSamplerBase
{
	GENERATED_BODY();

	virtual ~FDataflowScaleVectorSampler() override = default;
	virtual void Sample(TArrayView<const FVector3f> Positions, TArrayView<FVector3f> OutValues) const override;
	virtual FBox GetRenderBounds() const override;

public:
	/**  */
	TSharedPtr<const FDataflowFloatSamplerBase> FloatSampler;

	/**  */
	TSharedPtr<const FDataflowVectorSamplerBase> VectorSampler;
};

/**
 * Output a scale vector sampler
 */
USTRUCT(meta = (Icon = "Icons.EyeDropper"))
struct FDataflowScaleVectorSamplerNode : public FDataflowNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowScaleVectorSamplerNode, "Scale Vector Sampler", "Samplers", "* Multiply Magnitude")

public:
	FDataflowScaleVectorSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = VectorSampler))
	FDataflowVectorSampler VectorSampler;

	UPROPERTY(meta = (DataflowInput))
	FDataflowFloatSampler FloatSampler;

	void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};
