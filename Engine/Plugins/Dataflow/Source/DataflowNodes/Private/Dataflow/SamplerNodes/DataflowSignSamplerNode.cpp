// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/SamplerNodes/DataflowSignSamplerNode.h"
#include "Dataflow/SamplerNodes/DataflowSamplerFunctions.h"
#include "Dataflow/DataflowNodeFactory.h"

FBox FDataflowSignFloatSampler::GetRenderBounds() const
{
	if (Sampler.IsValid())
	{
		return Sampler->GetRenderBounds();
	}

	return FBox(FVector(-50.0), FVector(50.0));
}

void FDataflowSignFloatSampler::Sample(TArrayView<const FVector3f> Positions, TArrayView<float> OutValues) const
{
	if (Sampler.IsValid() && Positions.Num() == OutValues.Num())
	{
		TArray<float> OutValueFromInputSampler;
		OutValueFromInputSampler.SetNumUninitialized(OutValues.Num());
	
		Sampler->Sample(Positions, OutValueFromInputSampler);
	
		for (int32 Idx = 0; Idx < OutValues.Num(); ++Idx)
		{
			OutValues[Idx] = FMath::Sign(OutValueFromInputSampler[Idx]);
		}
	}
	else
	{
		for (int32 Idx = 0; Idx < OutValues.Num(); ++Idx)
		{
			OutValues[Idx] = 0.f;
		}
	}
}

FBox FDataflowSignVectorSampler::GetRenderBounds() const
{
	if (Sampler.IsValid())
	{
		return Sampler->GetRenderBounds();
	}

	return FBox(FVector(-50.0), FVector(50.0));
}

void FDataflowSignVectorSampler::Sample(TArrayView<const FVector3f> Positions, TArrayView<FVector3f> OutValues) const
{
	if (Sampler.IsValid() && Positions.Num() == OutValues.Num())
	{
		TArray<FVector3f> OutValueFromInputSampler;
		OutValueFromInputSampler.SetNumUninitialized(OutValues.Num());
	
		Sampler->Sample(Positions, OutValueFromInputSampler);
	
		for (int32 Idx = 0; Idx < OutValues.Num(); ++Idx)
		{
			OutValues[Idx] = FVector3f(FMath::Sign(OutValueFromInputSampler[Idx].X),
				FMath::Sign(OutValueFromInputSampler[Idx].Y),
				FMath::Sign(OutValueFromInputSampler[Idx].Z));
		}
	}
	else
	{
		for (int32 Idx = 0; Idx < OutValues.Num(); ++Idx)
		{
			OutValues[Idx] = FVector3f::ZeroVector;
		}
	}
}

FDataflowSignSamplerNode::FDataflowSignSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	static const FName MainTypeGroup("Main");
	
	RegisterInputConnection(&Sampler)
		.SetTypeDependencyGroup(MainTypeGroup);
	RegisterOutputConnection(&Sampler)
		.SetPassthroughInput(&Sampler)
		.SetTypeDependencyGroup(MainTypeGroup);
}

void FDataflowSignSamplerNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Sampler))
	{
		const FDataflowFloatSampler* FloatSampler = GetValue(Context, &Sampler).TryGet <FDataflowFloatSampler>();
		if (FloatSampler)
		{
			TSharedRef<FDataflowSignFloatSampler> Impl = MakeShared<FDataflowSignFloatSampler>();

			Impl->Sampler = FloatSampler->GetImpl();

			FDataflowFloatSampler OutSampler(Impl.ToSharedPtr());

			SetValue(Context, OutSampler, &Sampler);
			return;
		}

		const FDataflowVectorSampler* VectorSampler = GetValue(Context, &Sampler).TryGet<FDataflowVectorSampler>();
		if (VectorSampler)
		{
			TSharedRef<FDataflowSignVectorSampler> Impl = MakeShared<FDataflowSignVectorSampler>();
		
			Impl->Sampler = VectorSampler->GetImpl();
		
			FDataflowVectorSampler OutSampler(Impl.ToSharedPtr());

			SetValue(Context, OutSampler, &Sampler);
			return;
		}

		SafeForwardInput(Context, &Sampler, &Sampler);
	}
}


