// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowNode.h"
#include "Dataflow/SamplerNodes/DataflowSamplerTypes.h"

#include "DataflowSignSamplerNode.generated.h"

/***********************************************************************************************************************
 * No Input Sampler nodes
 ***********************************************************************************************************************/
USTRUCT()
struct FDataflowSignFloatSampler : public FDataflowFloatSamplerBase
{
	GENERATED_BODY();

	virtual ~FDataflowSignFloatSampler() override = default;
	virtual void Sample(TArrayView<const FVector3f> Positions, TArrayView<float> OutValues) const override;
	virtual FBox GetRenderBounds() const override;

public:
	/**  */
	TSharedPtr<const FDataflowFloatSamplerBase> Sampler;
};

USTRUCT()
struct FDataflowSignVectorSampler : public FDataflowVectorSamplerBase
{
	GENERATED_BODY();

	virtual ~FDataflowSignVectorSampler() override = default;
	virtual void Sample(TArrayView<const FVector3f> Positions, TArrayView<FVector3f> OutValues) const override;
	virtual FBox GetRenderBounds() const override;

public:
	/**  */
	TSharedPtr<const FDataflowVectorSamplerBase> Sampler;
};

/**
 * Output an Sign float sampler
 */
USTRUCT(meta = (Icon = "Icons.EyeDropper"))
struct FDataflowSignSamplerNode : public FDataflowNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowSignSamplerNode, "Sign Sampler", "Samplers", "")

public:
	FDataflowSignSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = Sampler))
	FDataflowSamplerTypes Sampler;
	
	void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};
