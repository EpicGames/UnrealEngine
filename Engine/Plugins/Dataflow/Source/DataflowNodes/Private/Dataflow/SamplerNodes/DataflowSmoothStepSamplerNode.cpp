// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/SamplerNodes/DataflowSmoothStepSamplerNode.h"
#include "Dataflow/SamplerNodes/DataflowSamplerFunctions.h"
#include "Dataflow/DataflowNodeFactory.h"

FBox FDataflowSmoothStepFloatSampler::GetRenderBounds() const
{
	if (FloatSampler.IsValid())
	{
		return FloatSampler->GetRenderBounds();
	}

	return FBox(FVector(-50.0), FVector(50.0));
}

namespace UE::Dataflow::SmoothStepSampler::Private
{
	/**
	 * Returns a smooth Hermite interpolation between 0 and 1 for the value X (where X ranges between A and B)
	 * Clamped to 0 for X <= A and 1 for X >= B.
	 *
	 * @param A Minimum value of X
	 * @param B Maximum value of X
	 * @param X Parameter
	 *
	 * @return Smoothed value between 0 and 1
	 */
	static double SmoothStep(const double InValue, const double InMinValue, const double InMaxValue, const double InValueA, const double InValueB)
	{
		double ReturnValue = 0.0;

		if (InMaxValue > InMinValue)
		{
			if (InValue <= InMinValue)
			{
				ReturnValue = InValueA;
			}
			else if (InValue >= InMaxValue)
			{
				ReturnValue = InValueB;
			}
			else if (InValueB > InValueA)
			{
				ReturnValue = (InValueB - InValueA) * FMath::SmoothStep(InMinValue, InMaxValue, InValue) + InValueA;
			}
			else if (InValueA > InValueB)
			{
				ReturnValue = (InValueA - InValueB) * FMath::SmoothStep(InMinValue, InMaxValue, InValue) + InValueB;
			}
		}

		return ReturnValue;
	}
}

void FDataflowSmoothStepFloatSampler::Sample(TArrayView<const FVector3f> Positions, TArrayView<float> OutValues) const
{
	if (FloatSampler.IsValid() && Positions.Num() == OutValues.Num())
	{
		FloatSampler->Sample(Positions, OutValues);

		for (int32 Idx = 0; Idx < OutValues.Num(); ++Idx)
		{
			OutValues[Idx] = UE::Dataflow::SmoothStepSampler::Private::SmoothStep(OutValues[Idx],
				MinValue,
				MaxValue,
				ValueA,
				ValueB);
		}
	}
	else
	{
		for (int32 Idx = 0; Idx < OutValues.Num(); ++Idx)
		{
			OutValues[Idx] = 0.f;
		}
	}
}

FDataflowSmoothStepFloatSamplerNode::FDataflowSmoothStepFloatSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Sampler);
	RegisterInputConnection(&SmoothStepSampler.MinValue, GET_MEMBER_NAME_CHECKED(FDataflowSmoothStepFloatSampler, MinValue));
	RegisterInputConnection(&SmoothStepSampler.MaxValue, GET_MEMBER_NAME_CHECKED(FDataflowSmoothStepFloatSampler, MaxValue));
	RegisterInputConnection(&SmoothStepSampler.ValueA, GET_MEMBER_NAME_CHECKED(FDataflowSmoothStepFloatSampler, ValueA));
	RegisterInputConnection(&SmoothStepSampler.ValueB, GET_MEMBER_NAME_CHECKED(FDataflowSmoothStepFloatSampler, ValueB));

	RegisterOutputConnection(&Sampler, &Sampler);
}

void FDataflowSmoothStepFloatSamplerNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Sampler))
	{
		const FDataflowFloatSampler& InFloatSampler = GetValue(Context, &Sampler);

		TSharedRef<FDataflowSmoothStepFloatSampler> Impl = MakeShared<FDataflowSmoothStepFloatSampler>();

		Impl->MinValue = GetValue(Context, &SmoothStepSampler.MinValue);
		Impl->MaxValue = GetValue(Context, &SmoothStepSampler.MaxValue);
		Impl->ValueA = GetValue(Context, &SmoothStepSampler.ValueA);
		Impl->ValueB = GetValue(Context, &SmoothStepSampler.ValueB);

		Impl->FloatSampler = InFloatSampler.GetImpl();


		FDataflowFloatSampler OutSampler(Impl.ToSharedPtr());
		SetValue(Context, OutSampler, &Sampler);
	}
}

FBox FDataflowSmoothStepVectorSampler::GetRenderBounds() const
{
	if (VectorSampler.IsValid())
	{
		return VectorSampler->GetRenderBounds();
	}

	return FBox(FVector(-50.0), FVector(50.0));
}

void FDataflowSmoothStepVectorSampler::Sample(TArrayView<const FVector3f> Positions, TArrayView<FVector3f> OutValues) const
{
	if (VectorSampler.IsValid() && Positions.Num() == OutValues.Num())
	{
		VectorSampler->Sample(Positions, OutValues);

		for (int32 Idx = 0; Idx < OutValues.Num(); ++Idx)
		{
			double X, Y, Z;

			X = UE::Dataflow::SmoothStepSampler::Private::SmoothStep(OutValues[Idx].X,
				MinValue.X,
				MaxValue.X,
				ValueA.X,
				ValueB.X);

			Y = UE::Dataflow::SmoothStepSampler::Private::SmoothStep(OutValues[Idx].Y,
				MinValue.Y,
				MaxValue.Y,
				ValueA.Y,
				ValueB.Y);

			Z = UE::Dataflow::SmoothStepSampler::Private::SmoothStep(OutValues[Idx].Z,
				MinValue.Z,
				MaxValue.Z,
				ValueA.Z,
				ValueB.Z);

			OutValues[Idx] = FVector3f(FVector(X, Y, Z));
		}
	}
	else
	{
		for (int32 Idx = 0; Idx < OutValues.Num(); ++Idx)
		{
			OutValues[Idx] = FVector3f::ZeroVector;
		}
	}
}

FDataflowSmoothStepVectorSamplerNode::FDataflowSmoothStepVectorSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Sampler);
	RegisterInputConnection(&SmoothStepSampler.MinValue, GET_MEMBER_NAME_CHECKED(FDataflowSmoothStepFloatSampler, MinValue));
	RegisterInputConnection(&SmoothStepSampler.MaxValue, GET_MEMBER_NAME_CHECKED(FDataflowSmoothStepFloatSampler, MaxValue));
	RegisterInputConnection(&SmoothStepSampler.ValueA, GET_MEMBER_NAME_CHECKED(FDataflowSmoothStepFloatSampler, ValueA));
	RegisterInputConnection(&SmoothStepSampler.ValueB, GET_MEMBER_NAME_CHECKED(FDataflowSmoothStepFloatSampler, ValueB));

	RegisterOutputConnection(&Sampler, &Sampler);
}

void FDataflowSmoothStepVectorSamplerNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Sampler))
	{
		const FDataflowVectorSampler& InVectorSampler = GetValue(Context, &Sampler);

		TSharedRef<FDataflowSmoothStepVectorSampler> Impl = MakeShared<FDataflowSmoothStepVectorSampler>();

		Impl->MinValue = GetValue(Context, &SmoothStepSampler.MinValue);
		Impl->MaxValue = GetValue(Context, &SmoothStepSampler.MaxValue);
		Impl->ValueA = GetValue(Context, &SmoothStepSampler.ValueA);
		Impl->ValueB = GetValue(Context, &SmoothStepSampler.ValueB);

		Impl->VectorSampler = InVectorSampler.GetImpl();

		FDataflowVectorSampler OutSampler(Impl.ToSharedPtr());
		SetValue(Context, OutSampler, &Sampler);
	}
}
