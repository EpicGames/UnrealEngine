// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowNode.h"
#include "Dataflow/SamplerNodes/DataflowSamplerTypes.h"

#include "DataflowSmoothStepSamplerNode.generated.h"

USTRUCT()
struct FDataflowSmoothStepFloatSampler : public FDataflowFloatSamplerBase
{
	GENERATED_BODY();

	virtual ~FDataflowSmoothStepFloatSampler() override = default;
	virtual void Sample(TArrayView<const FVector3f> Positions, TArrayView<float> OutValues) const override;
	virtual FBox GetRenderBounds() const override;

public:
	// x <= MinValue -> ValueA, MinValue < x < MaxValue -> smooth Hermite interpolation between ValueA and ValueB

	// MinValue for SmoothStep
	UPROPERTY(EditAnywhere, Category = "Step", meta = (DataflowInput))
	float MinValue = 0.f;

	// MaxValue for SmoothStep
	UPROPERTY(EditAnywhere, Category = "Step", meta = (DataflowInput))
	float MaxValue = 1.f;

	// Return value at MinValue
	UPROPERTY(EditAnywhere, Category = "Step", meta = (DataflowInput))
	float ValueA = 0.f;

	// Return value at MaxValue
	UPROPERTY(EditAnywhere, Category = "Step", meta = (DataflowInput))
	float ValueB = 1.f;

	TSharedPtr<const FDataflowFloatSamplerBase> FloatSampler;
};

/**
 *
 * SmoothStepSampler
 * Input(s) : Float Sampler
 * Output(s): Returns a smooth Hermite interpolation between ValueA and ValueB for the sampled value (where X ranges between MinValue and MaxValue)
 *
 */
USTRUCT(meta = (Icon = "Icons.EyeDropper"))
struct FDataflowSmoothStepFloatSamplerNode : public FDataflowNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowSmoothStepFloatSamplerNode, "SmoothStep Float Sampler", "Samplers", "")

public:
	FDataflowSmoothStepFloatSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = Sampler))
	FDataflowFloatSampler Sampler;

	UPROPERTY(EditAnywhere, Category = Sampler, meta = (ShowOnlyInnerProperties, SkipInDisplayNameChain))
	FDataflowSmoothStepFloatSampler SmoothStepSampler;

	void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

USTRUCT()
struct FDataflowSmoothStepVectorSampler : public FDataflowVectorSamplerBase
{
	GENERATED_BODY();

	virtual ~FDataflowSmoothStepVectorSampler() override = default;
	virtual void Sample(TArrayView<const FVector3f> Positions, TArrayView<FVector3f> OutValues) const override;
	virtual FBox GetRenderBounds() const override;

public:
	// MinValue for SmoothStep
	UPROPERTY(EditAnywhere, Category = "Step", meta = (DataflowInput))
	FVector MinValue = FVector(0.0);

	// MaxValue for SmoothStep
	UPROPERTY(EditAnywhere, Category = "Step", meta = (DataflowInput))
	FVector MaxValue = FVector(1.0);

	// Return value at MinValue
	UPROPERTY(EditAnywhere, Category = "Step", meta = (DataflowInput))
	FVector ValueA = FVector(0.0);

	// Return value at MaxValue
	UPROPERTY(EditAnywhere, Category = "Step", meta = (DataflowInput))
	FVector ValueB = FVector(1.0);

	TSharedPtr<const FDataflowVectorSamplerBase> VectorSampler;
};

/**
 *
 * SmoothStepSampler
 * Input(s) : Vector Sampler
 * Output(s): Returns a component wise smooth Hermite interpolation between ValueA and ValueB for the sampled value (where X ranges between MinValue and MaxValue)
 *
 */
USTRUCT(meta = (Icon = "Icons.EyeDropper"))
struct FDataflowSmoothStepVectorSamplerNode : public FDataflowNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowSmoothStepVectorSamplerNode, "SmoothStep Vector Sampler", "Samplers", "")

public:
	FDataflowSmoothStepVectorSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = Sampler))
	FDataflowVectorSampler Sampler;

	UPROPERTY(EditAnywhere, Category = Sampler, meta = (ShowOnlyInnerProperties, SkipInDisplayNameChain))
	FDataflowSmoothStepVectorSampler SmoothStepSampler;

	void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};
