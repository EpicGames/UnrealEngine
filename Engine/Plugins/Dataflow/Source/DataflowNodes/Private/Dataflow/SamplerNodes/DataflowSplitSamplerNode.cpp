// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/SamplerNodes/DataflowSplitSamplerNode.h"
#include "Dataflow/SamplerNodes/DataflowSamplerFunctions.h"
#include "Dataflow/DataflowNodeFactory.h"

FBox FDataflowVectorToFloatSampler::GetRenderBounds() const
{
	if (VectorSampler.IsValid())
	{
		return VectorSampler->GetRenderBounds();
	}

	return FBox(FVector(-50.0), FVector(50.0));
}

void FDataflowVectorToFloatSampler::Sample(TArrayView<const FVector3f> Positions, TArrayView<float> OutValues) const
{
	if (VectorSampler.IsValid() && Positions.Num() == OutValues.Num() && ElementIndex >= 0 && ElementIndex <= 2)
	{
		TArray<FVector3f> OutValueFromVectorSampler;
		OutValueFromVectorSampler.SetNumUninitialized(OutValues.Num());

		VectorSampler->Sample(Positions, OutValueFromVectorSampler);

		for (int32 Idx = 0; Idx < OutValues.Num(); ++Idx)
		{
			OutValues[Idx] = OutValueFromVectorSampler[Idx][ElementIndex];
		}
	}
}

FDataflowSplitVectorSamplerNode::FDataflowSplitVectorSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Sampler);
	RegisterOutputConnection(&SamplerX);
	RegisterOutputConnection(&SamplerY);
	RegisterOutputConnection(&SamplerZ);
}

void FDataflowSplitVectorSamplerNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&SamplerX))
	{
		const FDataflowVectorSampler& InSampler = GetValue(Context, &Sampler);

		TSharedRef<FDataflowVectorToFloatSampler> Impl = MakeShared<FDataflowVectorToFloatSampler>();

		Impl->VectorSampler = InSampler.GetImpl();
		Impl->ElementIndex = 0;

		FDataflowFloatSampler OutSampler(Impl.ToSharedPtr());

		SetValue(Context, OutSampler, &SamplerX);
	}
	else if (Out->IsA(&SamplerY))
	{
		const FDataflowVectorSampler& InSampler = GetValue(Context, &Sampler);

		TSharedRef<FDataflowVectorToFloatSampler> Impl = MakeShared<FDataflowVectorToFloatSampler>();

		Impl->VectorSampler = InSampler.GetImpl();
		Impl->ElementIndex = 1;

		FDataflowFloatSampler OutSampler(Impl.ToSharedPtr());

		SetValue(Context, OutSampler, &SamplerY);
	}
	else if (Out->IsA(&SamplerZ))
	{
		const FDataflowVectorSampler& InSampler = GetValue(Context, &Sampler);

		TSharedRef<FDataflowVectorToFloatSampler> Impl = MakeShared<FDataflowVectorToFloatSampler>();

		Impl->VectorSampler = InSampler.GetImpl();
		Impl->ElementIndex = 2;

		FDataflowFloatSampler OutSampler(Impl.ToSharedPtr());

		SetValue(Context, OutSampler, &SamplerZ);
	}
}
