// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowNode.h"
#include "Dataflow/SamplerNodes/DataflowSamplerTypes.h"

#include "DataflowSplitSamplerNode.generated.h"

USTRUCT()
struct FDataflowVectorToFloatSampler : public FDataflowFloatSamplerBase
{
	GENERATED_BODY();

	virtual ~FDataflowVectorToFloatSampler() override = default;
	virtual void Sample(TArrayView<const FVector3f> Positions, TArrayView<float> OutValues) const override;
	virtual FBox GetRenderBounds() const override;

public:
	/**  */
	TSharedPtr<const FDataflowVectorSamplerBase> VectorSampler;

	UPROPERTY()
	int32 ElementIndex = 0;
};

/**
 *
 * Split vector sampler
 * Input(s): a vector sampler to split it into X, Y, Z float samplers
 * Output(s): 3 float samplers (X, Y, Z)
 *
 */
USTRUCT()
struct FDataflowSplitVectorSamplerNode : public FDataflowNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowSplitVectorSamplerNode, "Split Vector Sampler", "Samplers", "")

public:
	FDataflowSplitVectorSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	FDataflowVectorSampler Sampler;

	UPROPERTY(meta = (DataflowOutput))
	FDataflowFloatSampler SamplerX;

	UPROPERTY(meta = (DataflowOutput))
	FDataflowFloatSampler SamplerY;

	UPROPERTY(meta = (DataflowOutput))
	FDataflowFloatSampler SamplerZ;

	void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};
