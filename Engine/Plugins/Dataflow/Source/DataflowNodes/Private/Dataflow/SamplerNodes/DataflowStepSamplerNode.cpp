// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/SamplerNodes/DataflowStepSamplerNode.h"
#include "Dataflow/SamplerNodes/DataflowSamplerFunctions.h"
#include "Dataflow/DataflowNodeFactory.h"

FBox FDataflowStepFloatSampler::GetRenderBounds() const
{
	if (FloatSampler.IsValid())
	{
		return FloatSampler->GetRenderBounds();
	}

	return FBox(FVector(-50.0), FVector(50.0));
}

void FDataflowStepFloatSampler::Sample(TArrayView<const FVector3f> Positions, TArrayView<float> OutValues) const
{
	if (FloatSampler.IsValid() && Positions.Num() == OutValues.Num())
	{
		FloatSampler->Sample(Positions, OutValues);

		for (int32 Idx = 0; Idx < OutValues.Num(); ++Idx)
		{
			if (OutValues[Idx] < Step)
			{
				OutValues[Idx] = ValueA;
			}
			else
			{
				OutValues[Idx] = ValueB;
			}
		}
	}
	else
	{
		for (int32 Idx = 0; Idx < OutValues.Num(); ++Idx)
		{
			OutValues[Idx] = 0.f;
		}
	}
}

FDataflowStepFloatSamplerNode::FDataflowStepFloatSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Sampler);
	RegisterInputConnection(&StepSampler.Step, GET_MEMBER_NAME_CHECKED(FDataflowStepFloatSampler, Step));
	RegisterInputConnection(&StepSampler.ValueA, GET_MEMBER_NAME_CHECKED(FDataflowStepFloatSampler, ValueA));
	RegisterInputConnection(&StepSampler.ValueB, GET_MEMBER_NAME_CHECKED(FDataflowStepFloatSampler, ValueB));

	RegisterOutputConnection(&Sampler, &Sampler);
}

void FDataflowStepFloatSamplerNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Sampler))
	{
		const FDataflowFloatSampler& InFloatSampler = GetValue(Context, &Sampler);

		TSharedRef<FDataflowStepFloatSampler> Impl = MakeShared<FDataflowStepFloatSampler>();

		Impl->Step = GetValue(Context, &StepSampler.Step);
		Impl->ValueA = GetValue(Context, &StepSampler.ValueA);
		Impl->ValueB = GetValue(Context, &StepSampler.ValueB);

		Impl->FloatSampler = InFloatSampler.GetImpl();


		FDataflowFloatSampler OutSampler(Impl.ToSharedPtr());
		SetValue(Context, OutSampler, &Sampler);
	}
}

FBox FDataflowStepVectorSampler::GetRenderBounds() const
{
	if (VectorSampler.IsValid())
	{
		return VectorSampler->GetRenderBounds();
	}

	return FBox(FVector(-50.0), FVector(50.0));
}

void FDataflowStepVectorSampler::Sample(TArrayView<const FVector3f> Positions, TArrayView<FVector3f> OutValues) const
{
	if (VectorSampler.IsValid() && Positions.Num() == OutValues.Num())
	{
		VectorSampler->Sample(Positions, OutValues);

		for (int32 Idx = 0; Idx < OutValues.Num(); ++Idx)
		{
			double X, Y, Z;

			if (OutValues[Idx].X < Step.X)
			{
				X = ValueA.X;
			}
			else
			{
				X = ValueB.X;
			}

			if (OutValues[Idx].Y < Step.Y)
			{
				Y = ValueA.Y;
			}
			else
			{
				Y = ValueB.Y;
			}

			if (OutValues[Idx].Z < Step.Z)
			{
				Z = ValueA.Z;
			}
			else
			{
				Z = ValueB.Z;
			}

			OutValues[Idx] = FVector3f(FVector(X, Y, Z));
		}
	}
	else
	{
		for (int32 Idx = 0; Idx < OutValues.Num(); ++Idx)
		{
			OutValues[Idx] = FVector3f::ZeroVector;
		}
	}
}

FDataflowStepVectorSamplerNode::FDataflowStepVectorSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Sampler);
	RegisterInputConnection(&StepSampler.Step, GET_MEMBER_NAME_CHECKED(FDataflowStepFloatSampler, Step));
	RegisterInputConnection(&StepSampler.ValueA, GET_MEMBER_NAME_CHECKED(FDataflowStepFloatSampler, ValueA));
	RegisterInputConnection(&StepSampler.ValueB, GET_MEMBER_NAME_CHECKED(FDataflowStepFloatSampler, ValueB));

	RegisterOutputConnection(&Sampler, &Sampler);
}

void FDataflowStepVectorSamplerNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Sampler))
	{
		const FDataflowVectorSampler& InVectorSampler = GetValue(Context, &Sampler);

		TSharedRef<FDataflowStepVectorSampler> Impl = MakeShared<FDataflowStepVectorSampler>();

		Impl->Step = GetValue(Context, &StepSampler.Step);
		Impl->ValueA = GetValue(Context, &StepSampler.ValueA);
		Impl->ValueB = GetValue(Context, &StepSampler.ValueB);

		Impl->VectorSampler = InVectorSampler.GetImpl();

		FDataflowVectorSampler OutSampler(Impl.ToSharedPtr());
		SetValue(Context, OutSampler, &Sampler);
	}
}
