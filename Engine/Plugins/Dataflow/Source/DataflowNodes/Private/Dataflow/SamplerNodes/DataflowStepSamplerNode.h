// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowNode.h"
#include "Dataflow/SamplerNodes/DataflowSamplerTypes.h"

#include "DataflowStepSamplerNode.generated.h"

USTRUCT()
struct FDataflowStepFloatSampler : public FDataflowFloatSamplerBase
{
	GENERATED_BODY();

	virtual ~FDataflowStepFloatSampler() override = default;
	virtual void Sample(TArrayView<const FVector3f> Positions, TArrayView<float> OutValues) const override;
	virtual FBox GetRenderBounds() const override;

public:
	// Step value, x < Step -> ValueA, x >= Step -> ValueB
	UPROPERTY(EditAnywhere, Category = "Step", meta = (DataflowInput))
	float Step = 1.f;

	// Return value when x < Step
	UPROPERTY(EditAnywhere, Category = "Step", meta = (DataflowInput))
	float ValueA = 0.f;

	// Return value when x >= Step
	UPROPERTY(EditAnywhere, Category = "Step", meta = (DataflowInput))
	float ValueB = 1.f;

	TSharedPtr<const FDataflowFloatSamplerBase> FloatSampler;
};

/**
 *
 * StepSampler
 * Input(s) : Float Sampler
 * Output(s): Step applied on sampled values
 *
 */
USTRUCT(meta = (Icon = "Icons.EyeDropper"))
struct FDataflowStepFloatSamplerNode : public FDataflowNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowStepFloatSamplerNode, "Step Float Sampler", "Samplers", "")

public:
	FDataflowStepFloatSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = Sampler))
	FDataflowFloatSampler Sampler;

	UPROPERTY(EditAnywhere, Category = Sampler, meta = (ShowOnlyInnerProperties, SkipInDisplayNameChain))
	FDataflowStepFloatSampler StepSampler;

	void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

USTRUCT()
struct FDataflowStepVectorSampler : public FDataflowVectorSamplerBase
{
	GENERATED_BODY();

	virtual ~FDataflowStepVectorSampler() override = default;
	virtual void Sample(TArrayView<const FVector3f> Positions, TArrayView<FVector3f> OutValues) const override;
	virtual FBox GetRenderBounds() const override;

public:
	// Step value, x < Step -> ValueA, x >= Step -> ValueB
	UPROPERTY(EditAnywhere, Category = "Step", meta = (DataflowInput))
	FVector Step = FVector(1.0);

	// Return value when x < Step
	UPROPERTY(EditAnywhere, Category = "Step", meta = (DataflowInput))
	FVector ValueA = FVector(0.0);

	// Return value when x >= Step
	UPROPERTY(EditAnywhere, Category = "Step", meta = (DataflowInput))
	FVector ValueB = FVector(1.0);

	TSharedPtr<const FDataflowVectorSamplerBase> VectorSampler;
};

/**
 *
 * StepSampler
 * Input(s) : Vector Sampler
 * Output(s): Returns a component wise step on the sampled values
 */
USTRUCT(meta = (Icon = "Icons.EyeDropper"))
struct FDataflowStepVectorSamplerNode : public FDataflowNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowStepVectorSamplerNode, "Step Vector Sampler", "Samplers", "")

public:
	FDataflowStepVectorSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = Sampler))
	FDataflowVectorSampler Sampler;

	UPROPERTY(EditAnywhere, Category = Sampler, meta = (ShowOnlyInnerProperties, SkipInDisplayNameChain))
	FDataflowStepVectorSampler StepSampler;

	void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};
