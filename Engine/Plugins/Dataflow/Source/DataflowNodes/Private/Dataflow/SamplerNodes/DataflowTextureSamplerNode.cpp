// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/SamplerNodes/DataflowTextureSamplerNode.h"
#include "Dataflow/SamplerNodes/DataflowSamplerFunctions.h"
#include "Dataflow/DataflowNodeFactory.h"
#include "IntBoxTypes.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "DynamicMesh/DynamicMeshOctree3.h"
#include "Intersection/IntrRay3AxisAlignedBox3.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/MeshNormals.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h" // FDynamicMeshUVOverlay
#include "Parameterization/DynamicMeshUVEditor.h"
#include "Engine/Texture2D.h"
#include "ImageCore.h"

namespace UE::Dataflow::Samplers
{
	namespace Private
	{
		static bool GetImageFromTexture2D(UTexture2D& InTexture, FImage& OutImage)
		{
			if (const FSharedImageConstRef CpuCopy = InTexture.GetCPUCopy())
			{
				CpuCopy->CopyTo(OutImage, ERawImageFormat::RGBA32F, EGammaSpace::Linear);
				return true;
			}
#if WITH_EDITORONLY_DATA
			if (InTexture.Source.IsValid())
			{
				FImage MipImage;
				if (InTexture.Source.GetMipImage(MipImage, 0))
				{
					MipImage.CopyTo(OutImage, ERawImageFormat::RGBA32F, EGammaSpace::Linear);
					return true;
				}
			}
#endif
			return false;
		}
	}
};

FBox FDataflowTextureVectorSampler::GetRenderBounds() const
{
	if (Sampler.IsValid())
	{
		return Sampler->GetRenderBounds();
	}

	return FBox(FVector(-50.0), FVector(50.0));
}

void FDataflowTextureVectorSampler::Sample(TArrayView<const FVector3f> Positions, TArrayView<FVector3f> OutValues) const
{
	if (Texture)
	{
		if (Sampler.IsValid() && Positions.Num() == OutValues.Num())
		{
			TArray<FVector3f> OutValueFromInputSampler;
			OutValueFromInputSampler.SetNumUninitialized(OutValues.Num());

			Sampler->Sample(Positions, OutValueFromInputSampler);

			// ---------------------------------------------------------------------------------------------------------

			FImage Image;
			const bool bGotImage = UE::Dataflow::Samplers::Private::GetImageFromTexture2D(*Texture, Image);
			if (bGotImage && Image.SizeX > 0 && Image.SizeY > 0)
			{
				TArrayView64<FLinearColor> Pixels = Image.AsRGBA32F();

				const int32 MaxX = Image.SizeX - 1;
				const int32 MaxY = Image.SizeY - 1;

				for (int32 Idx = 0; Idx < OutValues.Num(); ++Idx)
				{
					const FVector2f UvCoord = FVector2f(OutValueFromInputSampler[Idx].X, OutValueFromInputSampler[Idx].Y);
					const FVector2f NormUvCoord
					{
						FMath::Fractional(FMath::Fractional(UvCoord.X) + 1.0f), // [0,1[
						FMath::Fractional(FMath::Fractional(UvCoord.Y) + 1.0f), // [0,1[
					};
					const FVector2f PixelCoord
					{
						NormUvCoord.X * (float)(Image.SizeX) - 0.5f,
						NormUvCoord.Y * (float)(Image.SizeY) - 0.5f,
					};
					const FVector2f PixelFraction
					{
						FMath::Fractional(PixelCoord.X),
						FMath::Fractional(PixelCoord.Y),
					};

					const int32 PixelX0 = FMath::Clamp(int32(PixelCoord.X), 0, MaxX);
					const int32 PixelX1 = FMath::Clamp(PixelX0 + 1, 0, MaxX);
					const int32 PixelY0 = FMath::Clamp(int32(PixelCoord.Y), 0, MaxY);
					const int32 PixelY1 = FMath::Clamp(PixelY0 + 1, 0, MaxY);
					const FLinearColor Pixel00 = Pixels[PixelX0 + PixelY0 * Image.SizeX];
					const FLinearColor Pixel10 = Pixels[PixelX1 + PixelY0 * Image.SizeX];
					const FLinearColor Pixel01 = Pixels[PixelX0 + PixelY1 * Image.SizeX];
					const FLinearColor Pixel11 = Pixels[PixelX1 + PixelY1 * Image.SizeX];

					const FLinearColor Lerp0 = FMath::Lerp(Pixel00, Pixel10, PixelFraction.X);
					const FLinearColor Lerp1 = FMath::Lerp(Pixel01, Pixel11, PixelFraction.X);

					OutValues[Idx] = FVector3f(FMath::Lerp(Lerp0, Lerp1, PixelFraction.Y).R,
						FMath::Lerp(Lerp0, Lerp1, PixelFraction.Y).G,
						FMath::Lerp(Lerp0, Lerp1, PixelFraction.Y).B);
				}
			}
		}
	}
	else
	{
		for (int32 Idx = 0; Idx < OutValues.Num(); ++Idx)
		{
			OutValues[Idx] = FVector3f::ZeroVector;
		}
	}
}

FDataflowTextureVectorSamplerNode::FDataflowTextureVectorSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Sampler);
	RegisterInputConnection(&TextureSampler.Texture, GET_MEMBER_NAME_CHECKED(FDataflowTextureVectorSampler, Texture));
	RegisterOutputConnection(&Sampler, &Sampler);
}

void FDataflowTextureVectorSamplerNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Sampler))
	{
		const FDataflowVectorSampler& VectorSampler = GetValue(Context, &Sampler);

		TSharedRef<FDataflowTextureVectorSampler> Impl = MakeShared<FDataflowTextureVectorSampler>();

		Impl->Sampler = VectorSampler.GetImpl();

		Impl->Texture = GetValue(Context, &TextureSampler.Texture);

		FDataflowVectorSampler OutSampler(Impl.ToSharedPtr());

		SetValue(Context, OutSampler, &Sampler);
	}
}