// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowNode.h"
#include "Dataflow/SamplerNodes/DataflowSamplerTypes.h"

#include "DataflowTextureSamplerNode.generated.h"

class UTexture2D;

USTRUCT()
struct FDataflowTextureVectorSampler : public FDataflowVectorSamplerBase
{
	GENERATED_BODY();

	virtual ~FDataflowTextureVectorSampler() override = default;
	virtual void Sample(TArrayView<const FVector3f> Positions, TArrayView<FVector3f> OutValues) const override;
	virtual FBox GetRenderBounds() const override;

public:
	/** Texture to sample data from */
	UPROPERTY(meta = (DataflowInput))
	TObjectPtr<UTexture2D> Texture = nullptr;

	/** This must be a vector sampler with UV values */
	TSharedPtr<const FDataflowVectorSamplerBase> Sampler;
};

/**
 *
 * Texture vector sampler
 * Inputs: - a vector sampler to get the UVs from
 *         - a 2D texture
 * Output: for every sampled point color from the texture
 *
 */
USTRUCT(meta = (Icon = "Icons.EyeDropper"))
struct FDataflowTextureVectorSamplerNode : public FDataflowNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowTextureVectorSamplerNode, "Texture Vector Sampler", "Samplers", "")

public:
	FDataflowTextureVectorSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = Sampler))
	FDataflowVectorSampler Sampler;

	UPROPERTY(EditAnywhere, Category = Sampler, meta = (ShowOnlyInnerProperties, SkipInDisplayNameChain))
	FDataflowTextureVectorSampler TextureSampler;

	void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};
