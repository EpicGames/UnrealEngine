// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/SamplerNodes/DataflowTilingSamplerNode.h"
#include "Dataflow/SamplerNodes/DataflowSamplerFunctions.h"
#include "Dataflow/DataflowNodeFactory.h"

FBox FDataflowTilingFloatSampler::GetRenderBounds() const
{
	constexpr double Padding = 25.0;

	FVector Min = Offset - TileSize - FVector(Padding);
	FVector Max = Offset + 2.0 * TileSize + FVector(Padding);

	return FBox(Min, Max);
}

void FDataflowTilingFloatSampler::Sample(TArrayView<const FVector3f> Positions, TArrayView<float> OutValues) const
{
	if (Sampler.IsValid() && Positions.Num() == OutValues.Num())
	{
		TArray<FVector3f> TiledPositions;
		TiledPositions.SetNumUninitialized(OutValues.Num());

		for (int32 Idx = 0; Idx < Positions.Num(); ++Idx)
		{
			FVector Position = FVector(Positions[Idx]);
			FVector Min = Offset;
			FVector Max = Offset + TileSize;

			TiledPositions[Idx] = FVector3f(FVector(FMath::Wrap(Position.X, Min.X, Max.X),
				FMath::Wrap(Position.Y, Min.Y, Max.Y),
				FMath::Wrap(Position.Z, Min.Z, Max.Z)));
		}

		Sampler->Sample(TiledPositions, OutValues);
	}
	else
	{
		for (int32 Idx = 0; Idx < OutValues.Num(); ++Idx)
		{
			OutValues[Idx] = 0.f;
		}
	}
}

FBox FDataflowTilingVectorSampler::GetRenderBounds() const
{
	constexpr double Padding = 25.0;

	FVector Min = Offset - TileSize - FVector(Padding);
	FVector Max = Offset + 2.0 * TileSize + FVector(Padding);

	return FBox(Min, Max);
}

void FDataflowTilingVectorSampler::Sample(TArrayView<const FVector3f> Positions, TArrayView<FVector3f> OutValues) const
{
	if (Sampler.IsValid() && Positions.Num() == OutValues.Num())
	{
		TArray<FVector3f> TiledPositions;
		TiledPositions.SetNumUninitialized(OutValues.Num());

		for (int32 Idx = 0; Idx < Positions.Num(); ++Idx)
		{
			FVector Position = FVector(Positions[Idx]);
			FVector Min = Offset;
			FVector Max = Offset + TileSize;

			TiledPositions[Idx] = FVector3f(FVector(FMath::Wrap(Position.X, Min.X, Max.X),
				FMath::Wrap(Position.Y, Min.Y, Max.Y),
				FMath::Wrap(Position.Z, Min.Z, Max.Z)));
		}

		Sampler->Sample(TiledPositions, OutValues);
	}
	else
	{
		for (int32 Idx = 0; Idx < OutValues.Num(); ++Idx)
		{
			OutValues[Idx] = FVector3f::ZeroVector;
		}
	}
}

FDataflowTilingSamplerNode::FDataflowTilingSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	static const FName MainTypeGroup("Main");

	RegisterInputConnection(&Sampler)
		.SetTypeDependencyGroup(MainTypeGroup);
	RegisterInputConnection(&Offset);
	RegisterInputConnection(&TileSize);

	RegisterOutputConnection(&Sampler)
		.SetPassthroughInput(&Sampler)
		.SetTypeDependencyGroup(MainTypeGroup);
}

void FDataflowTilingSamplerNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Sampler))
	{
		const FVector InOffset = GetValue(Context, &Offset);
		const FVector InTileSize = GetValue(Context, &TileSize);

		const FDataflowFloatSampler* InFloatSampler = GetValue(Context, &Sampler).TryGet <FDataflowFloatSampler>();

		if (InFloatSampler)
		{
			TSharedRef<FDataflowTilingFloatSampler> Impl = MakeShared<FDataflowTilingFloatSampler>();

			Impl->Sampler = InFloatSampler->GetImpl();
			Impl->Offset = InOffset;
			Impl->TileSize = InTileSize;

			FDataflowFloatSampler OutSampler(Impl.ToSharedPtr());

			SetValue(Context, OutSampler, &Sampler);
			return;
		}

		const FDataflowVectorSampler* InVectorSampler = GetValue(Context, &Sampler).TryGet<FDataflowVectorSampler>();

		if (InVectorSampler)
		{
			TSharedRef<FDataflowTilingVectorSampler> Impl = MakeShared<FDataflowTilingVectorSampler>();

			Impl->Sampler = InVectorSampler->GetImpl();
			Impl->Offset = InOffset;
			Impl->TileSize = InTileSize;

			FDataflowVectorSampler OutSampler(Impl.ToSharedPtr());

			SetValue(Context, OutSampler, &Sampler);
			return;
		}

		SafeForwardInput(Context, &Sampler, &Sampler);
	}

}
