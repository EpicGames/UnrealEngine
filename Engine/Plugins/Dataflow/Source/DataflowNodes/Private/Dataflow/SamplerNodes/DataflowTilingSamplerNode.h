// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowNode.h"
#include "Dataflow/SamplerNodes/DataflowSamplerTypes.h"

#include "DataflowTilingSamplerNode.generated.h"

USTRUCT()
struct FDataflowTilingFloatSampler : public FDataflowFloatSamplerBase
{
	GENERATED_BODY();

	virtual ~FDataflowTilingFloatSampler() override = default;
	virtual void Sample(TArrayView<const FVector3f> Positions, TArrayView<float> OutValues) const override;
	virtual FBox GetRenderBounds() const override;

public:
	UPROPERTY()
	FVector Offset = FVector(0.0);

	UPROPERTY()
	FVector TileSize = FVector(1.0);

	TSharedPtr<const FDataflowFloatSamplerBase> Sampler;
};

USTRUCT()
struct FDataflowTilingVectorSampler : public FDataflowVectorSamplerBase
{
	GENERATED_BODY();

	virtual ~FDataflowTilingVectorSampler() override = default;
	virtual void Sample(TArrayView<const FVector3f> Positions, TArrayView<FVector3f> OutValues) const override;
	virtual FBox GetRenderBounds() const override;

public:
	UPROPERTY()
	FVector Offset = FVector(0.0);

	UPROPERTY()
	FVector TileSize = FVector(1.0);

	TSharedPtr<const FDataflowVectorSamplerBase> Sampler;
};

/**
 *
 * TilingSampler
 * Input(s) : Float or Vector Sampler
 * Output(s): Same type of sampler as the input outputting the tiled values
 *
 */
USTRUCT(meta = (Icon = "Icons.EyeDropper"))
struct FDataflowTilingSamplerNode : public FDataflowNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowTilingSamplerNode, "Tiling Sampler", "Samplers", "")

public:
	FDataflowTilingSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	/** Input for Tiling */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = Sampler))
	FDataflowSamplerTypes Sampler;

	/** Tile offset (min point of the tile) */
	UPROPERTY(EditAnywhere, Category = Tiling, meta = (DataflowInput))
	FVector Offset = FVector::ZeroVector;

	/** Tile size */
	UPROPERTY(EditAnywhere, Category = Tiling, meta = (DataflowInput))
	FVector TileSize = FVector(50.0);

	void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};
