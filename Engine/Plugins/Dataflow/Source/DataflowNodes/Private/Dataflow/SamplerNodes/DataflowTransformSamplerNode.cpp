// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/SamplerNodes/DataflowTransformSamplerNode.h"
#include "Dataflow/DataflowNodeFactory.h"

namespace UE::Dataflow
{
	void RegisterTransformSamplerNodes()
	{
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowTransformFloatSamplerNode);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FBox FDataflowTransformFloatSampler::GetRenderBounds() const
{
	if (Sampler.IsValid())
	{
		return Sampler->GetRenderBounds();
	}

	return FBox(FVector(-50.0), FVector(50.0));
}

void FDataflowTransformFloatSampler::Sample(TArrayView<const FVector3f> Positions, TArrayView<float> OutValues) const
{
	if (Sampler.IsValid())
	{
		TArray<FVector3f> TransformedPositions;
		TransformedPositions.SetNumUninitialized(Positions.Num());
		for (int32 Index = 0; Index < Positions.Num(); ++Index)
		{
			if (bInverseTransform)
			{
				TransformedPositions[Index] = FVector3f(Transform.InverseTransformPosition(FVector(Positions[Index])));
			}
			else
			{
				TransformedPositions[Index] = FVector3f(Transform.TransformPosition(FVector(Positions[Index])));
			}
		}
		Sampler->Sample(TransformedPositions, OutValues);
	}
}

FDataflowTransformFloatSamplerNode::FDataflowTransformFloatSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Sampler);
	RegisterInputConnection(&TransformSampler.Transform, GET_MEMBER_NAME_CHECKED(FDataflowTransformFloatSampler, Transform));
	RegisterOutputConnection(&Sampler, &Sampler);
}

void FDataflowTransformFloatSamplerNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Sampler))
	{
		TSharedRef<FDataflowTransformFloatSampler> Impl = MakeShared<FDataflowTransformFloatSampler>(TransformSampler);
		Impl->Transform = GetValue(Context, &TransformSampler.Transform);
		Impl->Sampler = GetValue(Context, &Sampler).GetImpl();

		FDataflowFloatSampler OutSampler(Impl.ToSharedPtr());
		SetValue(Context, OutSampler, &Sampler);
	}
}
