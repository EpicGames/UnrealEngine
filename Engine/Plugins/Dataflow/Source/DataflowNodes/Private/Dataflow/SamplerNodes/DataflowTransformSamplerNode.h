// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowNode.h"
#include "Dataflow/SamplerNodes/DataflowSamplerTypes.h"

#include "DataflowTransformSamplerNode.generated.h"

USTRUCT()
struct FDataflowTransformFloatSampler : public FDataflowFloatSamplerBase
{
	GENERATED_BODY();

	virtual ~FDataflowTransformFloatSampler() override = default;
	virtual void Sample(TArrayView<const FVector3f> Positions, TArrayView<float> OutValues) const override;
	virtual FBox GetRenderBounds() const override;

public:
	/** Transform to transform the points into a different space */
	UPROPERTY(EditAnywhere, Category = Transform, meta = (DataflowInput))
	FTransform Transform;

	/** Use inverse transform */
	UPROPERTY(EditAnywhere, Category = Transform)
	bool bInverseTransform = false;

	/** Sampler to send the transformed point to */
	TSharedPtr<const FDataflowFloatSamplerBase> Sampler;
};

/**
 * Output a transform sampler
 */
USTRUCT(meta = (Icon = "Icons.EyeDropper"))
struct FDataflowTransformFloatSamplerNode : public FDataflowNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowTransformFloatSamplerNode, "Transform Sampler", "Samplers", "Offset Translation Rotation Scale Space points")

public:
	FDataflowTransformFloatSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = Sampler))
	FDataflowFloatSampler Sampler;

	UPROPERTY(EditAnywhere, Category = Sampler, meta = (ShowOnlyInnerProperties, SkipInDisplayNameChain))
	FDataflowTransformFloatSampler TransformSampler;

	void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

namespace UE::Dataflow
{
	void RegisterTransformSamplerNodes();
}