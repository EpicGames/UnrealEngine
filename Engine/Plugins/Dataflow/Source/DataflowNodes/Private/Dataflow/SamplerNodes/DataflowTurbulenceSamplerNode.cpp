// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/SamplerNodes/DataflowTurbulenceSamplerNode.h"
#include "Dataflow/SamplerNodes/DataflowSamplerFunctions.h"
#include "Dataflow/DataflowNodeFactory.h"

FBox FDataflowTurbulenceFloatSampler::GetRenderBounds() const
{
	return RenderBounds;
}

void FDataflowTurbulenceFloatSampler::Sample(TArrayView<const FVector3f> Positions, TArrayView<float> OutValues) const
{
	const int32 NumSamples = FMath::Min(Positions.Num(), OutValues.Num());

	for (int32 Index = 0; Index < NumSamples; ++Index)
	{
		OutValues[Index] = UE::Dataflow::Sampler::Turbulence(FVector(Positions[Index]),
			Amplitude,
			Frequency,
			Offset,
			Octaves,
			Lacunarity,
			Gain);
	}
}


FDataflowTurbulenceFloatSamplerNode::FDataflowTurbulenceFloatSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&TurbulenceSampler.Amplitude, GET_MEMBER_NAME_CHECKED(FDataflowTurbulenceFloatSampler, Amplitude));
	RegisterInputConnection(&TurbulenceSampler.Frequency, GET_MEMBER_NAME_CHECKED(FDataflowTurbulenceFloatSampler, Frequency));
	RegisterInputConnection(&TurbulenceSampler.Offset, GET_MEMBER_NAME_CHECKED(FDataflowTurbulenceFloatSampler, Offset));
	RegisterInputConnection(&TurbulenceSampler.Octaves, GET_MEMBER_NAME_CHECKED(FDataflowTurbulenceFloatSampler, Octaves));
	RegisterInputConnection(&TurbulenceSampler.Lacunarity, GET_MEMBER_NAME_CHECKED(FDataflowTurbulenceFloatSampler, Lacunarity));
	RegisterInputConnection(&TurbulenceSampler.Gain, GET_MEMBER_NAME_CHECKED(FDataflowTurbulenceFloatSampler, Gain));
	RegisterInputConnection(&TurbulenceSampler.RenderBounds, GET_MEMBER_NAME_CHECKED(FDataflowTurbulenceFloatSampler, RenderBounds))
		.SetCanHidePin(true)
		.SetPinIsHidden(true);

	RegisterOutputConnection(&Sampler);
}

void FDataflowTurbulenceFloatSamplerNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Sampler))
	{
		TSharedRef<FDataflowTurbulenceFloatSampler> Impl = MakeShared<FDataflowTurbulenceFloatSampler>(TurbulenceSampler);
		Impl->Amplitude = GetValue(Context, &TurbulenceSampler.Amplitude);
		Impl->Frequency = GetValue(Context, &TurbulenceSampler.Frequency);
		Impl->Offset = GetValue(Context, &TurbulenceSampler.Offset);
		Impl->Octaves = GetValue(Context, &TurbulenceSampler.Octaves);
		Impl->Lacunarity = GetValue(Context, &TurbulenceSampler.Lacunarity);
		Impl->Gain = GetValue(Context, &TurbulenceSampler.Gain);
		Impl->RenderBounds = GetValue(Context, &TurbulenceSampler.RenderBounds);

		FDataflowFloatSampler OutSampler(Impl.ToSharedPtr());
		SetValue(Context, OutSampler, &Sampler);
	}
}

