// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowNode.h"
#include "Dataflow/SamplerNodes/DataflowSamplerTypes.h"

#include "DataflowTurbulenceSamplerNode.generated.h"

USTRUCT()
struct FDataflowTurbulenceFloatSampler : public FDataflowFloatSamplerBase
{
	GENERATED_BODY();

	virtual ~FDataflowTurbulenceFloatSampler() override = default;
	virtual void Sample(TArrayView<const FVector3f> Positions, TArrayView<float> OutValues) const override;
	virtual FBox GetRenderBounds() const override;

public:
	// Amplitude of the noise
	UPROPERTY(EditAnywhere, Category = "fBm", meta = (DataflowInput, UIMin = 0, ClampMin = 0))
	float Amplitude = 1.f;

	// Frequency of the noise
	UPROPERTY(EditAnywhere, Category = "fBm", meta = (DataflowInput, UIMin = 0, ClampMin = 0))
	float Frequency = 1.f;

	// Offset for the noise
	UPROPERTY(EditAnywhere, Category = "fBm", meta = (DataflowInput, UIMin = 0, ClampMin = 0))
	FVector Offset = FVector(0.f);

	// Number of layers of the summed instances of noise()
	UPROPERTY(EditAnywhere, Category = "fBm", meta = (DataflowInput, UIMin = 0, ClampMin = 0))
	int32 Octaves = 1;

	// Controls the uniform irregularity exponent of the series
	UPROPERTY(EditAnywhere, Category = "fBm", meta = (DataflowInput, UIMin = 0, ClampMin = 0))
	float Lacunarity = 2.f;

	// Gain
	UPROPERTY(EditAnywhere, Category = "fBm", meta = (DataflowInput, UIMin = 0, ClampMin = 0))
	float Gain = 0.5f;

	UPROPERTY(EditAnywhere, Category = RenderBounds, meta = (DataflowInput))
	FBox RenderBounds = FBox(FVector(-50.0), FVector(50.0));
};

/**
 * Output a Turbulence float sampler
 */
USTRUCT(meta = (Icon = "Icons.EyeDropper"))
struct FDataflowTurbulenceFloatSamplerNode : public FDataflowNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowTurbulenceFloatSamplerNode, "Turbulence Float Sampler", "Samplers", "")

public:
	FDataflowTurbulenceFloatSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	UPROPERTY(meta = (DataflowOutput))
	FDataflowFloatSampler Sampler;

	UPROPERTY(EditAnywhere, Category = Sampler, meta = (ShowOnlyInnerProperties, SkipInDisplayNameChain))
	FDataflowTurbulenceFloatSampler TurbulenceSampler;

	void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

