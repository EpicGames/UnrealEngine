// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/SamplerNodes/DataflowUniformSamplerNode.h"
#include "Dataflow/DataflowNodeFactory.h"

FBox FDataflowUniformFloatSampler::GetRenderBounds() const
{
	return RenderBounds;
}

void FDataflowUniformFloatSampler::Sample(TArrayView<const FVector3f> Positions, TArrayView<float> OutValues) const
{
	for (float& OutValue : OutValues)
	{
		OutValue = Value;
	}
}


FDataflowUniformFloatSamplerNode::FDataflowUniformFloatSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&UniformSampler.Value, GET_MEMBER_NAME_CHECKED(FDataflowUniformFloatSampler, Value));
	RegisterInputConnection(&UniformSampler.RenderBounds, GET_MEMBER_NAME_CHECKED(FDataflowUniformFloatSampler, RenderBounds))
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	
	RegisterOutputConnection(&Sampler);
}

void FDataflowUniformFloatSamplerNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Sampler))
	{
		TSharedRef<FDataflowUniformFloatSampler> Impl = MakeShared<FDataflowUniformFloatSampler>(UniformSampler);
		Impl->Value = GetValue(Context, &UniformSampler.Value);
		Impl->RenderBounds = GetValue(Context, &UniformSampler.RenderBounds);

		FDataflowFloatSampler OutSampler(Impl.ToSharedPtr());
		SetValue(Context, OutSampler, &Sampler);
	}
}

FBox FDataflowUniformVectorSampler::GetRenderBounds() const
{
	return RenderBounds;
}

void FDataflowUniformVectorSampler::Sample(TArrayView<const FVector3f> Positions, TArrayView<FVector3f> OutValues) const
{
	for (FVector3f& OutValue : OutValues)
	{
		OutValue = Value;
	}
}

FDataflowUniformVectorSamplerNode::FDataflowUniformVectorSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&UniformSampler.Value, GET_MEMBER_NAME_CHECKED(FDataflowUniformVectorSampler, Value));
	RegisterInputConnection(&UniformSampler.RenderBounds, GET_MEMBER_NAME_CHECKED(FDataflowUniformVectorSampler, RenderBounds))
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	
	RegisterOutputConnection(&Sampler);
}

void FDataflowUniformVectorSamplerNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Sampler))
	{
		TSharedRef<FDataflowUniformVectorSampler> Impl = MakeShared<FDataflowUniformVectorSampler>(UniformSampler);
		Impl->Value = GetValue(Context, &UniformSampler.Value);
		Impl->RenderBounds = GetValue(Context, &UniformSampler.RenderBounds);

		FDataflowVectorSampler OutSampler(Impl.ToSharedPtr());
		SetValue(Context, OutSampler, &Sampler);
	}
}

