// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowNode.h"
#include "Dataflow/SamplerNodes/DataflowSamplerTypes.h"

#include "DataflowUniformSamplerNode.generated.h"

USTRUCT()
struct FDataflowUniformFloatSampler : public FDataflowFloatSamplerBase
{
	GENERATED_BODY();

	virtual ~FDataflowUniformFloatSampler() override = default;
	virtual void Sample(TArrayView<const FVector3f> Positions, TArrayView<float> OutValues) const override;
	virtual FBox GetRenderBounds() const override;

public:
	// Uniform float value to set on all sampled points
	UPROPERTY(EditAnywhere, Category = "Uniform", meta = (DataflowInput))
	float Value = 0.f;

	UPROPERTY(EditAnywhere, Category = RenderBounds, meta = (DataflowInput))
	FBox RenderBounds = FBox(FVector(-50.0), FVector(50.0));
};

/**
 * Output a uniform float sampler
 */
USTRUCT(meta = (Icon = "Icons.EyeDropper"))
struct FDataflowUniformFloatSamplerNode : public FDataflowNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowUniformFloatSamplerNode, "Uniform Float Sampler", "Samplers", "")

public:
	FDataflowUniformFloatSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	UPROPERTY(meta = (DataflowOutput))
	FDataflowFloatSampler Sampler;

	UPROPERTY(EditAnywhere, Category = Sampler, meta = (ShowOnlyInnerProperties, SkipInDisplayNameChain))
	FDataflowUniformFloatSampler UniformSampler;

	void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

USTRUCT()
struct FDataflowUniformVectorSampler : public FDataflowVectorSamplerBase
{
	GENERATED_BODY();

	virtual ~FDataflowUniformVectorSampler() override = default;
	virtual void Sample(TArrayView<const FVector3f> Positions, TArrayView<FVector3f> OutValues) const override;
	virtual FBox GetRenderBounds() const override;

public:
	// Uniform vector value to set on all sampled points
	UPROPERTY(EditAnywhere, Category = "Uniform", meta = (DataflowInput))
	FVector3f Value = FVector3f(0.f);

	UPROPERTY(EditAnywhere, Category = RenderBounds, meta = (DataflowInput))
	FBox RenderBounds = FBox(FVector(-50.0), FVector(50.0));
};

/**
 * Output a uniform vector sampler
 */
USTRUCT(meta = (Icon = "Icons.EyeDropper"))
struct FDataflowUniformVectorSamplerNode : public FDataflowNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowUniformVectorSamplerNode, "Uniform Vector Sampler", "Samplers", "")

public:
	FDataflowUniformVectorSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	UPROPERTY(meta = (DataflowOutput))
	FDataflowVectorSampler Sampler;

	UPROPERTY(EditAnywhere, Category = Sampler, meta = (ShowOnlyInnerProperties, SkipInDisplayNameChain))
	FDataflowUniformVectorSampler UniformSampler;

	void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

