// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/SamplerNodes/DataflowVectorDeviationSamplerNode.h"
#include "Dataflow/SamplerNodes/DataflowSamplerFunctions.h"
#include "Dataflow/DataflowNodeFactory.h"

FBox FDataflowVectorDeviationFloatSampler::GetRenderBounds() const
{
	if (VectorSampler.IsValid())
	{
		return VectorSampler->GetRenderBounds();
	}

	return FBox(FVector(-50.0), FVector(50.0));
}

void FDataflowVectorDeviationFloatSampler::Sample(TArrayView<const FVector3f> Positions, TArrayView<float> OutValues) const
{
	if (VectorSampler.IsValid() && Positions.Num() == OutValues.Num() && ReferenceVector.Length() > 0.0)
	{
		TArray<FVector3f> OutValueFromInputSampler;
		OutValueFromInputSampler.SetNumUninitialized(OutValues.Num());
	
		VectorSampler->Sample(Positions, OutValueFromInputSampler);
	
		for (int32 Idx = 0; Idx < OutValues.Num(); ++Idx)
		{
			FVector SampledVector = FVector(OutValueFromInputSampler[Idx]);
			SampledVector.Normalize();

			FVector RefVector = ReferenceVector;
			RefVector.Normalize();

			// Dot product
			const float DP = FMath::Clamp(FVector::DotProduct(RefVector, SampledVector), -1.f, 1.f);

			float AngleInDegrees = FMath::RadiansToDegrees(FMath::Acos(DP));

			OutValues[Idx] = UE::Dataflow::Sampler::Remap(AngleInDegrees, 0.f, 180.f, 0.f, 1.f, 1.0, false);
		}
	}
	else
	{
		for (int32 Idx = 0; Idx < OutValues.Num(); ++Idx)
		{
			OutValues[Idx] = 0.f;
		}
	}
}

FDataflowVectorDeviationSamplerNode::FDataflowVectorDeviationSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Sampler);
	RegisterInputConnection(&ReferenceVector);
	RegisterOutputConnection(&VectorDeviationSampler);
}

void FDataflowVectorDeviationSamplerNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&VectorDeviationSampler))
	{
		const FVector InReferenceVector = GetValue(Context, &ReferenceVector);

		const FDataflowVectorSampler& InSampler = GetValue(Context, &Sampler);

		TSharedRef<FDataflowVectorDeviationFloatSampler> Impl = MakeShared<FDataflowVectorDeviationFloatSampler>();
		
		Impl->VectorSampler = InSampler.GetImpl();
		Impl->ReferenceVector = InReferenceVector;

		FDataflowFloatSampler OutSampler(Impl.ToSharedPtr());

		SetValue(Context, OutSampler, &VectorDeviationSampler);
	}
}

