// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowNode.h"
#include "Dataflow/SamplerNodes/DataflowSamplerTypes.h"

#include "DataflowVectorDeviationSamplerNode.generated.h"

USTRUCT()
struct FDataflowVectorDeviationFloatSampler : public FDataflowFloatSamplerBase
{
	GENERATED_BODY();

	virtual ~FDataflowVectorDeviationFloatSampler() override = default;
	virtual void Sample(TArrayView<const FVector3f> Positions, TArrayView<float> OutValues) const override;
	virtual FBox GetRenderBounds() const override;

public:
	UPROPERTY()
	FVector ReferenceVector =FVector::UpVector;

	/**  */
	TSharedPtr<const FDataflowVectorSamplerBase> VectorSampler;
};

/**
 * 
 */
USTRUCT(meta = (Icon = "Icons.EyeDropper"))
struct FDataflowVectorDeviationSamplerNode : public FDataflowNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowVectorDeviationSamplerNode, "Vector Deviation Sampler", "Samplers", "")

public:
	FDataflowVectorDeviationSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	FDataflowVectorSampler Sampler;

	UPROPERTY(meta = (DataflowOutput))
	FDataflowFloatSampler VectorDeviationSampler;

	/** If BlendSampler is not connected this value controls the lerp */
	UPROPERTY(EditAnywhere, Category = Deviation, meta = (DataflowInput))
	FVector ReferenceVector = FVector::UpVector;

	void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};
