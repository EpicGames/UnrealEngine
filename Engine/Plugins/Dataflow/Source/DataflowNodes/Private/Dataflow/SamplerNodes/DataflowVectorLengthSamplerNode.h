// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowNode.h"
#include "Dataflow/SamplerNodes/DataflowSamplerTypes.h"

#include "DataflowVectorLengthSamplerNode.generated.h"

USTRUCT()
struct FDataflowVectorLengthFloatSampler : public FDataflowFloatSamplerBase
{
	GENERATED_BODY();

	virtual ~FDataflowVectorLengthFloatSampler() override = default;
	virtual void Sample(TArrayView<const FVector3f> Positions, TArrayView<float> OutValues) const override;
	virtual FBox GetRenderBounds() const override;

public:
	/**  */
	TSharedPtr<const FDataflowVectorSamplerBase> VectorSampler;
};

/**
 * 
 */
USTRUCT(meta = (Icon = "Icons.EyeDropper"))
struct FDataflowVectorLengthSamplerNode : public FDataflowNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowVectorLengthSamplerNode, "Vector Length Sampler", "Samplers", "")

public:
	FDataflowVectorLengthSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	FDataflowVectorSampler Sampler;

	UPROPERTY(meta = (DataflowOutput))
	FDataflowFloatSampler VectorLengthSampler;

	void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};
