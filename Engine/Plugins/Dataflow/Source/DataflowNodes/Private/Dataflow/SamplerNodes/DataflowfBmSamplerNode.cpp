// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/SamplerNodes/DataflowfBmSamplerNode.h"
#include "Dataflow/SamplerNodes/DataflowSamplerFunctions.h"
#include "Dataflow/DataflowNodeFactory.h"

FBox FDataflowfBmFloatSampler::GetRenderBounds() const
{
	return RenderBounds;
}

void FDataflowfBmFloatSampler::Sample(TArrayView<const FVector3f> Positions, TArrayView<float> OutValues) const
{
	if (Positions.Num() == OutValues.Num())
	{
		for (int32 Index = 0; Index < OutValues.Num(); ++Index)
		{
			OutValues[Index] = UE::Dataflow::Sampler::fBm(FVector(Positions[Index]),
				Amplitude,
				Frequency,
				Offset,
				Octaves,
				Lacunarity,
				Gain);
		}
	}
	else	
	{
		for (int32 Index = 0; Index < OutValues.Num(); ++Index)
		{
			OutValues[Index] = 0;
		}
	}
}


FDataflowfBmFloatSamplerNode::FDataflowfBmFloatSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&fBmSampler.Amplitude, GET_MEMBER_NAME_CHECKED(FDataflowfBmFloatSampler, Amplitude));
	RegisterInputConnection(&fBmSampler.Frequency, GET_MEMBER_NAME_CHECKED(FDataflowfBmFloatSampler, Frequency));
	RegisterInputConnection(&fBmSampler.Offset, GET_MEMBER_NAME_CHECKED(FDataflowfBmFloatSampler, Offset));
	RegisterInputConnection(&fBmSampler.Octaves, GET_MEMBER_NAME_CHECKED(FDataflowfBmFloatSampler, Octaves));
	RegisterInputConnection(&fBmSampler.Lacunarity, GET_MEMBER_NAME_CHECKED(FDataflowfBmFloatSampler, Lacunarity));
	RegisterInputConnection(&fBmSampler.Gain, GET_MEMBER_NAME_CHECKED(FDataflowfBmFloatSampler, Gain));
	RegisterInputConnection(&fBmSampler.RenderBounds, GET_MEMBER_NAME_CHECKED(FDataflowfBmFloatSampler, RenderBounds))
		.SetCanHidePin(true)
		.SetPinIsHidden(true);

	RegisterOutputConnection(&Sampler);
}

void FDataflowfBmFloatSamplerNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Sampler))
	{
		TSharedRef<FDataflowfBmFloatSampler> Impl = MakeShared<FDataflowfBmFloatSampler>(fBmSampler);
		Impl->Amplitude = GetValue(Context, &fBmSampler.Amplitude);
		Impl->Frequency = GetValue(Context, &fBmSampler.Frequency);
		Impl->Offset = GetValue(Context, &fBmSampler.Offset);
		Impl->Octaves = GetValue(Context, &fBmSampler.Octaves);
		Impl->Lacunarity = GetValue(Context, &fBmSampler.Lacunarity);
		Impl->Gain = GetValue(Context, &fBmSampler.Gain);
		Impl->RenderBounds = GetValue(Context, &fBmSampler.RenderBounds);

		FDataflowFloatSampler OutSampler(Impl.ToSharedPtr());
		SetValue(Context, OutSampler, &Sampler);
	}
}

FBox FDataflowfBmVectorSampler::GetRenderBounds() const
{
	return RenderBounds;
}

void FDataflowfBmVectorSampler::Sample(TArrayView<const FVector3f> Positions, TArrayView<FVector3f> OutValues) const
{
	if (Positions.Num() == OutValues.Num())
	{
		for (int32 Index = 0; Index < OutValues.Num(); ++Index)
		{
			OutValues[Index] = FVector3f(UE::Dataflow::Sampler::VfBm(FVector(Positions[Index]),
				Amplitude,
				Frequency,
				Offset,
				Octaves,
				Lacunarity,
				Gain));
		}
	}
	else
	{
		for (int32 Index = 0; Index < OutValues.Num(); ++Index)
		{
			OutValues[Index] = FVector3f(0.f);
		}
	}
}

FDataflowfBmVectorSamplerNode::FDataflowfBmVectorSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&fBmSampler.Amplitude, GET_MEMBER_NAME_CHECKED(FDataflowfBmVectorSampler, Amplitude));
	RegisterInputConnection(&fBmSampler.Frequency, GET_MEMBER_NAME_CHECKED(FDataflowfBmVectorSampler, Frequency));
	RegisterInputConnection(&fBmSampler.Offset, GET_MEMBER_NAME_CHECKED(FDataflowfBmVectorSampler, Offset));
	RegisterInputConnection(&fBmSampler.Octaves, GET_MEMBER_NAME_CHECKED(FDataflowfBmVectorSampler, Octaves));
	RegisterInputConnection(&fBmSampler.Lacunarity, GET_MEMBER_NAME_CHECKED(FDataflowfBmVectorSampler, Lacunarity));
	RegisterInputConnection(&fBmSampler.Gain, GET_MEMBER_NAME_CHECKED(FDataflowfBmVectorSampler, Gain));
	RegisterInputConnection(&fBmSampler.RenderBounds, GET_MEMBER_NAME_CHECKED(FDataflowfBmVectorSampler, RenderBounds))
		.SetCanHidePin(true)
		.SetPinIsHidden(true);

	RegisterOutputConnection(&Sampler);
}

void FDataflowfBmVectorSamplerNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Sampler))
	{
		TSharedRef<FDataflowfBmVectorSampler> Impl = MakeShared<FDataflowfBmVectorSampler>(fBmSampler);
		Impl->Amplitude = GetValue(Context, &fBmSampler.Amplitude);
		Impl->Frequency = GetValue(Context, &fBmSampler.Frequency);
		Impl->Offset = GetValue(Context, &fBmSampler.Offset);
		Impl->Octaves = GetValue(Context, &fBmSampler.Octaves);
		Impl->Lacunarity = GetValue(Context, &fBmSampler.Lacunarity);
		Impl->Gain = GetValue(Context, &fBmSampler.Gain);
		Impl->RenderBounds = GetValue(Context, &fBmSampler.RenderBounds);

		FDataflowVectorSampler OutSampler(Impl.ToSharedPtr());
		SetValue(Context, OutSampler, &Sampler);
	}
}

