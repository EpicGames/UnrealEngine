// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowNode.h"
#include "Dataflow/SamplerNodes/DataflowSamplerTypes.h"

#include "DataflowfBmSamplerNode.generated.h"

USTRUCT()
struct FDataflowfBmFloatSampler : public FDataflowFloatSamplerBase
{
	GENERATED_BODY();

	virtual ~FDataflowfBmFloatSampler() override = default;
	virtual void Sample(TArrayView<const FVector3f> Positions, TArrayView<float> OutValues) const override;
	virtual FBox GetRenderBounds() const override;

public:
	// 
	UPROPERTY(EditAnywhere, Category = "fBm", meta = (DataflowInput, UIMin = 0, ClampMin = 0))
	float Amplitude = 1.f;
	
	// 
	UPROPERTY(EditAnywhere, Category = "fBm", meta = (DataflowInput, UIMin = 0, ClampMin = 0))
	float Frequency = 1.f;
	
	// 
	UPROPERTY(EditAnywhere, Category = "fBm", meta = (DataflowInput, UIMin = 0, ClampMin = 0))
	FVector Offset = FVector(0.f);
	
	// Number of layers of the summed instances of noise()
	UPROPERTY(EditAnywhere, Category = "fBm", meta = (DataflowInput, UIMin = 0, ClampMin = 0))
	int32 Octaves = 1;
	
	// Controls the uniform irregularity exponent of the series
	UPROPERTY(EditAnywhere, Category = "fBm", meta = (DataflowInput, UIMin = 0, ClampMin = 0))
	float Lacunarity = 2.f;
	
	// Gain
	UPROPERTY(EditAnywhere, Category = "fBm", meta = (DataflowInput, UIMin = 0, ClampMin = 0))
	float Gain = 0.5f;

	UPROPERTY(EditAnywhere, Category = RenderBounds, meta = (DataflowInput))
	FBox RenderBounds = FBox(FVector(-50.0), FVector(50.0));
};

/**
 * Output a fBm Noise float sampler
 */
USTRUCT(meta = (Icon = "Icons.EyeDropper"))
struct FDataflowfBmFloatSamplerNode : public FDataflowNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowfBmFloatSamplerNode, "fBm Float Sampler", "Samplers", "")

public:
	FDataflowfBmFloatSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	UPROPERTY(meta = (DataflowOutput))
	FDataflowFloatSampler Sampler;

	UPROPERTY(EditAnywhere, Category = Sampler, meta = (ShowOnlyInnerProperties, SkipInDisplayNameChain))
	FDataflowfBmFloatSampler fBmSampler;

	void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

USTRUCT()
struct FDataflowfBmVectorSampler : public FDataflowVectorSamplerBase
{
	GENERATED_BODY();

	virtual ~FDataflowfBmVectorSampler() override = default;
	virtual void Sample(TArrayView<const FVector3f> Positions, TArrayView<FVector3f> OutValues) const override;
	virtual FBox GetRenderBounds() const override;

public:
	// 
	UPROPERTY(EditAnywhere, Category = "fBm", meta = (DataflowInput, UIMin = 0, ClampMin = 0))
	float Amplitude = 1.f;

	// 
	UPROPERTY(EditAnywhere, Category = "fBm", meta = (DataflowInput, UIMin = 0, ClampMin = 0))
	float Frequency = 1.f;

	// 
	UPROPERTY(EditAnywhere, Category = "fBm", meta = (DataflowInput, UIMin = 0, ClampMin = 0))
	FVector Offset = FVector(0.f);

	// Number of layers of the summed instances of noise()
	UPROPERTY(EditAnywhere, Category = "fBm", meta = (DataflowInput, UIMin = 0, ClampMin = 0))
	int32 Octaves = 1;

	// Controls the uniform irregularity exponent of the series
	UPROPERTY(EditAnywhere, Category = "fBm", meta = (DataflowInput, UIMin = 0, ClampMin = 0))
	float Lacunarity = 2.f;

	// Gain
	UPROPERTY(EditAnywhere, Category = "fBm", meta = (DataflowInput, UIMin = 0, ClampMin = 0))
	float Gain = 0.5f;

	UPROPERTY(EditAnywhere, Category = RenderBounds, meta = (DataflowInput))
	FBox RenderBounds = FBox(FVector(-50.0), FVector(50.0));
};

/**
 * Output a fBm vector sampler
 */
USTRUCT(meta = (Icon = "Icons.EyeDropper"))
struct FDataflowfBmVectorSamplerNode : public FDataflowNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowfBmVectorSamplerNode, "fBm Vector Sampler", "Samplers", "")

public:
	FDataflowfBmVectorSamplerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	UPROPERTY(meta = (DataflowOutput))
	FDataflowVectorSampler Sampler;

	UPROPERTY(EditAnywhere, Category = Sampler, meta = (ShowOnlyInnerProperties, SkipInDisplayNameChain))
	FDataflowfBmVectorSampler fBmSampler;

	void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

