// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/Transfer/DataflowMeshAttributesTransferNodes.h"

#include "Dataflow/DataflowCore.h"
#include "Dataflow/DataflowEngine.h"
#include "Dataflow/DataflowMesh.h"
#include "DynamicMesh/DynamicBoneAttribute.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowMeshAttributesTransferNodes)

using namespace UE::Dataflow;
using namespace UE::Geometry;

FTransferMeshAttributesDataflowNode::FTransferMeshAttributesDataflowNode(const FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
	, All(FDataflowFunctionProperty::FDelegate::CreateRaw(this, &FTransferMeshAttributesDataflowNode::AddAllAttributes))
	, None(FDataflowFunctionProperty::FDelegate::CreateRaw(this, &FTransferMeshAttributesDataflowNode::RemoveAllAttributes))
{
	RegisterInputConnection(&Mesh);
	RegisterOutputConnection(&Mesh, &Mesh);
	RegisterInputConnection(&SourceMesh);
}

void FTransferMeshAttributesDataflowNode::Evaluate(FContext& Context, const FDataflowOutput* Out) const
{
	if (!Out->IsA(&Mesh))
	{
		return;
	}
	
	if (AttributeProxies.IsEmpty())
	{
		return;
	}
	
	const TObjectPtr<const UDataflowMesh> SrcMeshPtr = GetValue(Context, &SourceMesh);
	TObjectPtr<UDataflowMesh> DstMeshPtr = GetValue(Context, &Mesh);
	if (!SrcMeshPtr || !DstMeshPtr)
	{
		return;
	}
		
	const FDynamicMesh3& SrcDynaMesh = SrcMeshPtr->GetDynamicMeshRef();
	if (!SrcDynaMesh.HasAttributes())
	{
		return;
	}
	
	FDynamicMesh3 DstDynaMesh = DstMeshPtr->GetDynamicMeshRef();
	
	// do transfer
	TransferProxies(Context, SrcDynaMesh, DstDynaMesh);
	
	TObjectPtr<UDataflowMesh> NewMesh = NewObject<UDataflowMesh>();
	NewMesh->SetDynamicMesh(MoveTemp(DstDynaMesh));
	NewMesh->SetMaterials(DstMeshPtr->GetMaterials());
	SetValue(Context, NewMesh, &Mesh);
}

void FTransferMeshAttributesDataflowNode::AddAllAttributes(FContext& Context)
{
	AttributeProxies.Reset();
	AttributeProxies.Add(FInstancedStruct::Make<FDataflowSkeletonProxy>());
	AttributeProxies.Add(FInstancedStruct::Make<FDataflowSkinWeightsProxy>());
	AttributeProxies.Add(FInstancedStruct::Make<FDataflowMorphTargetProxy>());
	AttributeProxies.Add(FInstancedStruct::Make<FDataflowTriangleLabelsProxy>());
	AttributeProxies.Add(FInstancedStruct::Make<FDataflowPolygroupProxy>());
}

void FTransferMeshAttributesDataflowNode::RemoveAllAttributes(FContext& Context)
{
	AttributeProxies.Reset();
}

void FTransferMeshAttributesDataflowNode::TransferProxies(FContext& Context, const FDynamicMesh3& InSrcDynaMesh, FDynamicMesh3& InOutDstDynaMesh) const
{
	const FDynamicMeshAttributeSet* SrcAttributes = InSrcDynaMesh.Attributes();
	
	if (!InOutDstDynaMesh.HasAttributes())
	{
		InOutDstDynaMesh.EnableAttributes();
	}
	FDynamicMeshAttributeSet* DstAttributes = InOutDstDynaMesh.Attributes();

	// skeleton & skin weight
	{
		const bool bTransferSkeleton = SrcAttributes->HasBones() ? AttributeProxies.ContainsByPredicate([](const FInstancedStruct& Proxy)
		{
			return Proxy.GetPtr<FDataflowSkeletonProxy>() != nullptr;
		}) : false;
		
		if (bTransferSkeleton)
		{
			if (!InOutDstDynaMesh.HasAttributes())
			{
				InOutDstDynaMesh.EnableAttributes();
			}
			if (!DstAttributes->HasBones())
			{
				DstAttributes->EnableBones(SrcAttributes->GetNumBones());
			}
			DstAttributes->CopyBoneAttributes(*SrcAttributes);
		}
	}

	// "transferable" attributes
	{
		FTransferAttributes TransferAttributes(&InSrcDynaMesh);
		
		BuildProxies(TransferAttributes, SrcAttributes, DstAttributes);

		if (TransferAttributes.Validate() == EOperationValidationResult::Ok)
		{
			// should this result be tested?
			TransferAttributes.TransferAttributesToMesh(InOutDstDynaMesh);
		}
	}
}

void FTransferMeshAttributesDataflowNode::BuildProxies(FTransferAttributes& TransferAttributes,
	const FDynamicMeshAttributeSet* SrcAttributes,
	FDynamicMeshAttributeSet* DstAttributes) const
{
	using namespace UE::Geometry;
	
	auto TransferAll = []<typename ProxyType>(const TArray<const ProxyType*> Proxies)
	{
		return Proxies.ContainsByPredicate([](const ProxyType* Proxy)
		{
			return Proxy->SourceType == EProxySourceType::All;
		});
	};
	
	// skin weights
	{
		const TArray<const FDataflowSkinWeightsProxy*> SkinWeights = GetProxies<FDataflowSkinWeightsProxy>();
		
		if (!SkinWeights.IsEmpty())
		{
			const bool bTransferSkeleton = AttributeProxies.ContainsByPredicate([](const FInstancedStruct& Proxy)
			{
				return Proxy.GetPtr<FDataflowSkeletonProxy>() != nullptr;
			});

			FSkinWeightsProxy::FSkinWeightsProxyOptions SkinWeightsOptions;
			if (!bTransferSkeleton && SrcAttributes->HasBones())
			{	
				SkinWeightsOptions.SourceIndexToBone = SrcAttributes->GetBoneNames()->GetAttribValues();
				if (!DstAttributes->HasBones())
				{
					DstAttributes->EnableBones(SrcAttributes->GetNumBones());
				}
				
				const TArray<FName>& SourceBoneNames = SrcAttributes->GetBoneNames()->GetAttribValues();
				const TArray<FName>& TargetBoneNames = DstAttributes->GetBoneNames()->GetAttribValues();
				if (SourceBoneNames != TargetBoneNames)
				{
					SkinWeightsOptions.TargetBoneToIndex.Reserve(TargetBoneNames.Num());
					for (int32 BoneID = 0; BoneID < TargetBoneNames.Num(); ++BoneID)
					{
						const FName& BoneName = TargetBoneNames[BoneID];
						if (SkinWeightsOptions.TargetBoneToIndex.Contains(BoneName))
						{
							checkSlow(false);
						}
						else
						{
							SkinWeightsOptions.TargetBoneToIndex.Add(BoneName, static_cast<uint16>(BoneID));
						}
					}
				}
			}
		
			auto AddSkinWeightsProxy = [&TransferAttributes, &Options = SkinWeightsOptions](const FDynamicMeshVertexSkinWeightsAttribute* SrcAttribute, const FName DstName)
				{
					if (SrcAttribute && DstName != NAME_None)
					{
						TransferAttributes.AddVertexProxy<FSkinWeightsProxy>(SrcAttribute, DstName, Options);
					}
				};

			if (TransferAll(SkinWeights))
			{
				for (const auto &[AttrName, Attr]: SrcAttributes->GetSkinWeightsAttributes())
				{
					AddSkinWeightsProxy(Attr.Get(), AttrName);
				}
			}
			else
			{
				for (const FDataflowSkinWeightsProxy* Proxy: SkinWeights)
				{
					if (Proxy->Source != NAME_None)
					{
						AddSkinWeightsProxy(SrcAttributes->GetSkinWeightsAttribute(Proxy->Source), Proxy->Destination);
					}
				}
			}
		}
	}
	
	// morph targets
	{
		auto AddMorphTargetProxy = [&TransferAttributes](const FDynamicMeshMorphTargetAttribute* SrcAttribute, const FName DstName)
		{
			if (SrcAttribute && DstName != NAME_None)
			{
				TransferAttributes.AddVertexProxy<FMorphTargetProxy>(SrcAttribute, DstName);
			}
		};

		const TArray<const FDataflowMorphTargetProxy*> MorphTargets = GetProxies<FDataflowMorphTargetProxy>();
		if (TransferAll(MorphTargets))
		{
			for (const auto &[AttrName, Attr]: SrcAttributes->GetMorphTargetAttributes())
			{
				AddMorphTargetProxy(Attr.Get(), AttrName);
			}
		}
		else
		{
			for (const FDataflowMorphTargetProxy* Proxy: MorphTargets)
			{
				if (Proxy->Source != NAME_None)
				{
					AddMorphTargetProxy(SrcAttributes->GetMorphTargetAttribute(Proxy->Source), Proxy->Destination);
				}
			}
		}
	}
	
	// polygroups
	{
		auto AddPolygroupProxy = [&TransferAttributes](const FDynamicMeshPolygroupAttribute* SrcAttribute, const FName DstName)
		{
			if (SrcAttribute && DstName != NAME_None)
			{
				TransferAttributes.AddTriangleProxy<FPolygroupProxy>(SrcAttribute, DstName);
			}
		};
		
		const TArray<const FDataflowPolygroupProxy*> Polygroups = GetProxies<FDataflowPolygroupProxy>();
		if (TransferAll(Polygroups))
		{
			for (int32 Index = 0; Index < SrcAttributes->NumPolygroupLayers(); Index++)
			{
				if (const FDynamicMeshPolygroupAttribute* Polygroup = SrcAttributes->GetPolygroupLayer(Index))
				{
					AddPolygroupProxy(Polygroup, Polygroup->GetName());
				}
			}
		}
		else
		{
			for (const FDataflowPolygroupProxy* Proxy: Polygroups)
			{
				if (Proxy->Source != NAME_None)
				{
					for (int32 Index = 0; Index < SrcAttributes->NumPolygroupLayers(); Index++)
					{
						const FDynamicMeshPolygroupAttribute* Polygroup = SrcAttributes->GetPolygroupLayer(Index);
						if (Polygroup && Polygroup->GetName() == Proxy->Source)
						{
							AddPolygroupProxy(Polygroup, Proxy->Destination);
						}
					}					
				}
			}
		}
	}
	
	// triangle labels
	{
		auto AddTriangleLabelProxy = [&TransferAttributes](const FDynamicMeshTriangleLabelAttribute* SrcAttribute, const FName DstName)
		{
			if (SrcAttribute && DstName != NAME_None)
			{
				TransferAttributes.AddTriangleProxy<FTriangleLabelProxy>(SrcAttribute, DstName);
			}
		};
		
		const TArray<const FDataflowTriangleLabelsProxy*> TriangleLabels = GetProxies<FDataflowTriangleLabelsProxy>();
		if (TransferAll(TriangleLabels))
		{
			for (const auto &[AttrName, Attr]: SrcAttributes->GetTriangleLabelAttributes())
			{
				AddTriangleLabelProxy(Attr.Get(), AttrName);
			}
		}
		else
		{
			for (const FDataflowTriangleLabelsProxy* Proxy: TriangleLabels)
			{
				if (Proxy->Source != NAME_None)
				{
					AddTriangleLabelProxy(SrcAttributes->FindTriangleLabelAttribute(Proxy->Source), Proxy->Destination);
				}
			}
		}
	}
}