// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/Transfer/TransferAttributes.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "DynamicMesh/MeshNormals.h"
#include "Util/ProgressCancel.h"
#include "Async/ParallelFor.h"
#include "TransformTypes.h"

using namespace UE::Geometry;

namespace UE::Transfer::Private 
{
	static FVector3f ToVector3f(const FVector3d& InVector)
	{
		return FVector3f(static_cast<float>(InVector.X), static_cast<float>(InVector.Y), static_cast<float>(InVector.Z));
	}

	struct FTaskContext
	{
		TArray<int32> ElementIDs;
	};
}

FTransferAttributes::FTransferAttributes(const FDynamicMesh3* InSourceMesh, const FDynamicMeshAABBTree3* InSourceBVH)
	: SourceMesh(InSourceMesh)
	, SourceBVH(InSourceBVH)
	, InternalSourceBVH(SourceBVH ? nullptr : MakeUnique<FDynamicMeshAABBTree3>(SourceMesh))
{}

FTransferAttributes::~FTransferAttributes() 
{}

bool FTransferAttributes::Cancelled()
{
	return (Progress == nullptr) ? false : Progress->Cancelled();
}

EOperationValidationResult FTransferAttributes::Validate()
{	
	if (SourceMesh == nullptr) 
	{
		return EOperationValidationResult::Failed_UnknownReason;
	}

	if (SourceBVH == nullptr && InternalSourceBVH.IsValid() == false) 
	{
		return EOperationValidationResult::Failed_UnknownReason;
	}

	if (!SourceMesh->HasAttributes()) 
	{
		return EOperationValidationResult::Failed_UnknownReason;
	}

	if (VertexProxies.IsEmpty() && TriangleProxies.IsEmpty())
	{
		return EOperationValidationResult::Failed_UnknownReason;
	}
	
	return EOperationValidationResult::Ok;
}

namespace UE::Transfer::Private
{
	struct FGetOrCreateAttribute
	{
		FGetOrCreateAttribute(FDynamicMesh3& InDstMesh)
			: DstMesh( InDstMesh )
		{
			check(DstMesh.Attributes());
		}

		bool operator()(FSkinWeightsProxy& Proxy) const
		{
			return Proxy.GetOrCreateDestAttribute(DstMesh);
		}

		bool operator()(FMorphTargetProxy& Proxy) const
		{
			return Proxy.GetOrCreateDestAttribute(DstMesh);
		}
		
		bool operator()(FPolygroupProxy& Proxy) const
		{
			return Proxy.GetOrCreateDestAttribute(DstMesh);
		}
		
		bool operator()(FTriangleLabelProxy& Proxy) const
		{
			return Proxy.GetOrCreateDestAttribute(DstMesh);
		}
		
		FDynamicMesh3& DstMesh;
	};
}

bool FTransferAttributes::TransferAttributesToMesh(FDynamicMesh3& InOutTargetMesh)
{	
	using namespace Transfer::Private;
 
	if (Validate() != EOperationValidationResult::Ok) 
	{
		return false;
	}
 
	if (!InOutTargetMesh.HasAttributes())
	{
		InOutTargetMesh.EnableAttributes(); 
	}
 
	// If we need to compare normals, make sure both the target and the source meshes have per-vertex normals data
	TUniquePtr<FMeshNormals> InternalTargetMeshNormals;
	if (NormalThreshold >= 0)
	{
		if (!SourceMesh->HasVertexNormals() && !InternalSourceMeshNormals)
		{
			// only do this once for the source mesh in case of subsequent calls to the method
			InternalSourceMeshNormals = MakeUnique<FMeshNormals>(SourceMesh);
			InternalSourceMeshNormals->ComputeVertexNormals();
		}
 
		if (!InOutTargetMesh.HasVertexNormals())
		{
			InternalTargetMeshNormals = MakeUnique<FMeshNormals>(&InOutTargetMesh);
			InternalTargetMeshNormals->ComputeVertexNormals();
		}
	}
	
	// create destination attributes
	{
		FGetOrCreateAttribute GetOrCreateAttribute(InOutTargetMesh);
		for (FVertexProxy& Proxy: VertexProxies)
		{
			Visit(GetOrCreateAttribute, Proxy);
		}
		for (FTriangleProxy& Proxy: TriangleProxies)
		{
			Visit(GetOrCreateAttribute, Proxy);
		}
	}
 
	bool bFailed = false;
 
	if (TransferMethod == ETransferMethod::ClosestPointOnSurface)
	{
		bFailed = TransferUsingClosestPoint(InOutTargetMesh, InternalTargetMeshNormals);
	} 
	else if (TransferMethod == ETransferMethod::Inpaint)
	{
		ensure(false);
		// TODO
	}
	else 
	{
		checkNoEntry(); // unsupported method
	}
 
	if (Cancelled() || bFailed) 
	{
		return false;
	}
		
	return true;
}

FClosestSample FTransferAttributes::GetClosestSample(const FVector3d& InPoint, const FVector3f& InNormal) const
{	
	using namespace Transfer::Private;

	FClosestSample Sample;

	// Find the containing triangle and the barycentric coordinates of the closest point	
	if (FindClosestPointOnSourceSurface(InPoint, TargetToWorld, Sample.TriID, Sample.Bary))
	{
		if (SearchRadius < 0 && NormalThreshold < 0)
		{
			// If the radius and normals are ignored, simply interpolate the values and return the result
			return Sample;
		}
	
		bool bPassedRadiusCheck = true;
		if (SearchRadius >= 0)
		{
			const FVector3d MatchedPoint = SourceMesh->GetTriBaryPoint(Sample.TriID,  Sample.Bary[0],  Sample.Bary[1],  Sample.Bary[2]);
			bPassedRadiusCheck = (InPoint - MatchedPoint).Length() <= SearchRadius;
		}

		bool bPassedNormalsCheck = true;
		if (NormalThreshold >= 0)
		{
			FVector3f Normal0 = FVector3f::UnitY(), Normal1 = FVector3f::UnitY(), Normal2 = FVector3f::UnitY();
			const bool bHasSourceNormals = SourceMesh->HasVertexNormals();
			if (ensure(bHasSourceNormals || InternalSourceMeshNormals.IsValid()))
			{
				const FIndex3i TriVertices = SourceMesh->GetTriangle(Sample.TriID);
				Normal0 = bHasSourceNormals ? SourceMesh->GetVertexNormal(TriVertices[0]) : ToVector3f(InternalSourceMeshNormals->GetNormals()[TriVertices[0]]);
				Normal1 = bHasSourceNormals ? SourceMesh->GetVertexNormal(TriVertices[1]) : ToVector3f(InternalSourceMeshNormals->GetNormals()[TriVertices[1]]);
				Normal2 = bHasSourceNormals ? SourceMesh->GetVertexNormal(TriVertices[2]) : ToVector3f(InternalSourceMeshNormals->GetNormals()[TriVertices[2]]);
			}

			const FVector3f BaryF = ToVector3f(Sample.Bary);
			const FVector3f MatchedNormal = Normalized(BaryF[0] * Normal0 + BaryF[1] * Normal1 + BaryF[2] * Normal2);
			const FVector3f InNormalNormalized = Normalized(InNormal);
			const double NormalAngle = static_cast<double>(FMathf::ACos(InNormalNormalized.Dot(MatchedNormal)));
			bPassedNormalsCheck = NormalAngle <= NormalThreshold;

			if (!bPassedNormalsCheck && LayeredMeshSupport)
			{
				// try again with a flipped normal
				bPassedNormalsCheck = (TMathUtil<double>::Pi - NormalAngle) <= NormalThreshold;
			}
		}
	
		if (!bPassedRadiusCheck || !bPassedNormalsCheck)
		{
			Sample.TriID = IndexConstants::InvalidID;
		}
	}
	
	return Sample;
}

bool FTransferAttributes::FindClosestPointOnSourceSurface(const FVector3d& InPoint, const FTransformSRT3d& InToWorld, int32& NearTriID, FVector3d& Bary) const
{
	IMeshSpatial::FQueryOptions Options;
	double NearestDistSqr;
	
	const FVector3d WorldPoint = InToWorld.TransformPosition(InPoint);
	if (SourceBVH != nullptr) 
	{ 
		NearTriID = SourceBVH->FindNearestTriangle(WorldPoint, NearestDistSqr, Options);
	}
	else 
	{
		NearTriID = InternalSourceBVH->FindNearestTriangle(WorldPoint, NearestDistSqr, Options);
	}

	if (!ensure(NearTriID != IndexConstants::InvalidID))
	{
		return false;
	}

	const FDistPoint3Triangle3d Query = TMeshQueries<FDynamicMesh3>::TriangleDistance(*SourceMesh, NearTriID, WorldPoint);
	const FVector3d NearestPnt = Query.ClosestTrianglePoint;
	const FIndex3i TriVertex = SourceMesh->GetTriangle(NearTriID);

	Bary = VectorUtil::BarycentricCoords(NearestPnt, SourceMesh->GetVertexRef(TriVertex.A),
													 SourceMesh->GetVertexRef(TriVertex.B),
													 SourceMesh->GetVertexRef(TriVertex.C));

	return true;
}

namespace UE::Transfer::Private
{
	struct FSample
	{
		int32 TriID; 
		FVector3d Bary;
	};
	
	struct FClosestTransfer
	{
		FClosestTransfer(const FDynamicMesh3& InSrcMesh, const int32 InElementID, const FClosestSample& InSample)
			: ElementID(InElementID)
			, Sample(InSample)
			, Triangle(InSample.TriID != IndexConstants::InvalidID ? InSrcMesh.GetTriangle(InSample.TriID) : FIndex3i::Invalid())
			, BaryWeights(ToVector3f(InSample.Bary))
		{
			ensure(InElementID != IndexConstants::InvalidID);
		}

		void operator()(const FSkinWeightsProxy& Proxy) const
		{
			Proxy.Transfer(ElementID, Triangle, BaryWeights);
		}

		void operator()(const FMorphTargetProxy& Proxy) const
		{
			Proxy.Transfer(ElementID, Triangle, BaryWeights);
		}

		void operator()(const FPolygroupProxy& Proxy) const
		{
			Proxy.Transfer(Sample.TriID, ElementID);
		}
		
		void operator()(const FTriangleLabelProxy& Proxy) const
		{
			Proxy.Transfer(Sample.TriID, ElementID);
		}
		
		const int32 ElementID = IndexConstants::InvalidID;
		const FClosestSample& Sample;
		const FIndex3i Triangle;
		const FVector3f BaryWeights;
	};
}

bool FTransferAttributes::TransferUsingClosestPoint(FDynamicMesh3& InOutTargetMesh, const TUniquePtr<FMeshNormals>& InTargetMeshNormals)
{
	if (!ensure(TransferMethod == ETransferMethod::ClosestPointOnSurface))
	{
		return true;
	}
	
	bool bFailed = false;
	
	if (!VertexProxies.IsEmpty())
	{
		// compute the transfer only for the subset of vertices if necessary  
		const bool bUseSubset = !TargetVerticesSubset.IsEmpty();
		const int32 NumVerticesToTransfer = bUseSubset ? TargetVerticesSubset.Num() : InOutTargetMesh.MaxVertexID();
		
		const int32 NumMatched = TransferVerticesUsingClosestPoint(InOutTargetMesh, InTargetMeshNormals);
			
		// If the caller requested to simply find the closest point for all vertices then the number of matched vertices
		// must be equal to the target mesh vertex count
		if (SearchRadius < 0 && NormalThreshold < 0)
		{
			bFailed |= NumMatched != NumVerticesToTransfer;
		}
	}
		
	if (!TriangleProxies.IsEmpty())
	{
		// compute the transfer only for the subset of triangles if necessary  
		const bool bUseSubset = !TargetTrianglesSubset.IsEmpty();
		const int32 NumTrianglesToTransfer = bUseSubset ? TargetTrianglesSubset.Num() : InOutTargetMesh.MaxTriangleID();
		
		const int32 NumMatched = TransferTrianglesUsingClosestPoint(InOutTargetMesh);
			
		// If the caller requested to simply find the closest point for all triangles then the number of matched triangles
		// must be equal to the target mesh triangle count
		if (SearchRadius < 0 && NormalThreshold < 0)
		{
			bFailed |= NumMatched != NumTrianglesToTransfer;
		}
	}
	
	return bFailed;
}

int32 FTransferAttributes::TransferVerticesUsingClosestPoint(FDynamicMesh3& InOutTargetMesh, const TUniquePtr<FMeshNormals>& InTargetMeshNormals)
{
	using namespace Transfer::Private;
	
	// compute the transfer only for the subset of vertices if necessary  
	const bool bUseSubset = !TargetVerticesSubset.IsEmpty();
	const int32 NumVerticesToTransfer = bUseSubset ? TargetVerticesSubset.Num() : InOutTargetMesh.MaxVertexID();
	MatchedVertices.Init(false, NumVerticesToTransfer);
	
	TFunction<FVector3f(const int32)> GetNormal = [](const int32 VertexID)
	{
		return FVector3f::UnitY();
	};
	
	if (NormalThreshold >= 0) 
	{
		const bool bHasNormals = InOutTargetMesh.HasVertexNormals();
		if (ensure(bHasNormals || InTargetMeshNormals))
		{
			if (bHasNormals)
			{
				GetNormal = [&InOutTargetMesh](const int32 VertexID)
				{
					return InOutTargetMesh.GetVertexNormal(VertexID);
				};
			}
			else
			{
				GetNormal = [&InTargetMeshNormals](const int32 VertexID)
				{
					return ToVector3f(InTargetMeshNormals->GetNormals()[VertexID]);
				};
			}
		}
	}

	ParallelFor(NumVerticesToTransfer, [this, &InOutTargetMesh, &GetNormal, bUseSubset](int32 InVertexID)
	{
		if (Cancelled()) 
		{
			return;
		}

		const int32 VertexID = bUseSubset ? TargetVerticesSubset[InVertexID] : InVertexID;
		if (InOutTargetMesh.IsVertex(VertexID)) 
		{
			const FVector3d Point = InOutTargetMesh.GetVertexRef(VertexID);
			const FVector3f Normal = GetNormal(VertexID);
			
			const FClosestSample Sample = GetClosestSample(Point, Normal);
			if (Sample.TriID != IndexConstants::InvalidID)
			{
				FClosestTransfer ClosestTransfer(*SourceMesh, VertexID, Sample);
				for (FVertexProxy& Proxy: VertexProxies)
				{
					 Visit(ClosestTransfer, Proxy);
				}	
				MatchedVertices[VertexID] = true;
			}
		}

	}, bUseParallel ? EParallelForFlags::None : EParallelForFlags::ForceSingleThread);

	return MatchedVertices.CountSetBits();
}

int32 FTransferAttributes::TransferTrianglesUsingClosestPoint(FDynamicMesh3& InOutTargetMesh)
{
	using namespace Transfer::Private;
	
	// compute the transfer only for the subset of triangles if necessary  
	const bool bUseSubset = !TargetTrianglesSubset.IsEmpty();
	const int32 NumTrianglesToTransfer = bUseSubset ? TargetTrianglesSubset.Num() : InOutTargetMesh.MaxTriangleID();
	MatchedTriangles.Init(false, NumTrianglesToTransfer);

	ParallelFor(NumTrianglesToTransfer, [this, &InOutTargetMesh, bUseSubset](int32 InTriangleID)
	{
		if (Cancelled()) 
		{
			return;
		}

		const int32 TriangleID = bUseSubset ? TargetTrianglesSubset[InTriangleID] : InTriangleID;
		if (InOutTargetMesh.IsTriangle(TriangleID)) 
		{
			FVector3d Normal;
			double Area;
			FVector3d Centroid;
			InOutTargetMesh.GetTriInfo(TriangleID, Normal, Area, Centroid);
			
			const FClosestSample Sample = GetClosestSample(Centroid, ToVector3f(Normal));
			if (Sample.TriID != IndexConstants::InvalidID)
			{
				FClosestTransfer ClosestTransfer(*SourceMesh, TriangleID, Sample);
				for (FTriangleProxy& Proxy: TriangleProxies)
				{
					 Visit(ClosestTransfer, Proxy);
				}	
				MatchedTriangles[TriangleID] = true;
			}
		}

	}, bUseParallel ? EParallelForFlags::None : EParallelForFlags::ForceSingleThread);

	return MatchedTriangles.CountSetBits();
}