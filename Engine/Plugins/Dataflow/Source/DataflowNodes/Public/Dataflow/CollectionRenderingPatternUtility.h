// Copyright Epic Games, Inc. All Rights Reserved. 
#pragma once

#include "CoreMinimal.h"

namespace UE::Geometry { class FDynamicMesh3; }
namespace GeometryCollection::Facades { class FRenderingFacade; }
class UDataflow;
class UObject;

namespace UE::Dataflow
{
	namespace Conversion
	{
		// Convert a rendering facade to a dynamic mesh. if InMeshIndex==INDEX_NONE then all Geometry is converted. 
		void DATAFLOWNODES_API RenderingFacadeToDynamicMesh(const GeometryCollection::Facades::FRenderingFacade& Facade, int32 InMeshIndex, UE::Geometry::FDynamicMesh3& DynamicMesh, const bool bBuildRemapping = true);

		// Convert a dynamic mesh to a rendering facade
		void DATAFLOWNODES_API DynamicMeshToRenderingFacade(const UE::Geometry::FDynamicMesh3& DynamicMesh, GeometryCollection::Facades::FRenderingFacade& Facade);
	}

}	// namespace UE::Dataflow