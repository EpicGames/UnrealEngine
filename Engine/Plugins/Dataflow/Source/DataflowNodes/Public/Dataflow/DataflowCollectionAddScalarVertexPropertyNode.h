// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowNode.h"
#include "Dataflow/DataflowVertexAttributeEditableNode.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "Dataflow/DataflowCollectionAttributeKeyNodes.h"
#include "Dataflow/DataflowVertexPropertyGroup.h"

#include "DataflowCollectionAddScalarVertexPropertyNode.generated.h"

#define UE_API DATAFLOWNODES_API

/** How the map stored on the AddWeightMapNode should be applied to an existing map. If no map exists, it is treated as zero.*/
UENUM()
enum class EDataflowWeightMapOverrideType : uint8
{
	
	/** Replace all the values.*/
	ReplaceAll,
	/** Add the values difference to the input one. */
	AddDifference,
	/** Replace only the values that has changed. */
	ReplaceChanged,
};

/** Scalar vertex properties. */
USTRUCT(Meta = (DataflowCollection))
struct FDataflowCollectionAddScalarVertexPropertyNode : public FDataflowVertexAttributeEditableNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowCollectionAddScalarVertexPropertyNode, "PaintWeightMap", "Collection", "Paint a weight map and save it to collection")

	UE_API virtual TArray<UE::Dataflow::FRenderingParameter> GetRenderParametersImpl() const override;

	
public:
	UPROPERTY(Meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection"))
	FManagedArrayCollection Collection;

	/** The name to be set as a weight map attribute. */
	UPROPERTY(EditAnywhere, Category = "Vertex Attributes", meta = (DisplayName = "Attribute Name"))
	FString Name;

	UPROPERTY(meta = (DisplayName = "Attribute Key", DataflowInput, DataflowOutput, DataflowPassthrough = "AttributeKey"))
	FCollectionAttributeKey AttributeKey;

	// This property should be technically deprecated but we need to keep it because we cannot post serialize fix it as we need a dataflow Context to create a snapshot 
	// deprecated methods are still using it and we still use it to load the data when there's no pre-existing snapshot 
	UPROPERTY()
	TArray<float> VertexWeights; 

	UPROPERTY(EditAnywhere, Category = "Vertex Attributes", meta = (DisplayName = "Vertex Group"))
	FScalarVertexPropertyGroup TargetGroup;
	
	/** This type will define how the data are applied to the input data */
	UPROPERTY(EditAnywhere, Category = "Weight Map")
	EDataflowWeightMapOverrideType OverrideType = EDataflowWeightMapOverrideType::ReplaceAll;

	UE_API FDataflowCollectionAddScalarVertexPropertyNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

	/** Report the vertex weights back onto the property ones */
	UE_DEPRECATED(5.8, "ReportVertexWeights is now deprecated, use the FDataflowVertexAttributeEditableNode API instead")
	UE_API void ReportVertexWeights(const TArray<float>& SetupWeights, const TArray<float>& FinalWeights, const TArray<int32>& WeightIndices);

	/** Extract the vertex weights back from the property ones */
	UE_DEPRECATED(5.8, "ExtractVertexWeights is now deprecated, use the FDataflowVertexAttributeEditableNode API instead")
	UE_API void ExtractVertexWeights(const TArray<float>& SetupWeights, TArrayView<float> FinalWeights) const;

	/** Fill the weight attribute values from the collection */
	UE_DEPRECATED(5.8, "FillAttributeWeights is now deprecated, use the FDataflowVertexAttributeEditableNode API instead")
	UE_API bool FillAttributeWeights(const TSharedPtr<const FManagedArrayCollection> SelectedCollection, const FCollectionAttributeKey& InAttributeKey, TArray<float>& OutAttributeValues) const;

	/** Get the weights attribute key to retrieve/set the weight values*/
	UE_API FCollectionAttributeKey GetWeightAttributeKey(UE::Dataflow::FContext& Context) const;

private:

	/** Pass through value to skip replacing the weight map value if nothing has changed */
	static constexpr float ReplaceChangedPassthroughValue = UE_BIG_NUMBER;
	
	UE_API virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	virtual FAttributeKey GetVertexAttributeToVisualize(UE::Dataflow::FContext& Context, const FName OutputName, const FName RenderGroup) const override;

	//~ Begin - FDataflowVertexAttributeEditableNode API
	virtual void GetSupportedViewModes(UE::Dataflow::FContext& Context, TArray<FName>& OutViewModeNames) const override;
	virtual void GetVertexAttributeValues(UE::Dataflow::FContext& Context, TArray<float>& OutValues) const override;
	virtual void SetVertexAttributeValues(UE::Dataflow::FContext& Context, const TArray<float>& InValues, const TArray<int32>& InWeightIndices) override;
	virtual void GetExtraVertexMapping(UE::Dataflow::FContext& Context, FName SelectedViewMode, TArray<int32>& OutMappingToWeight, TArray<TArray<int32>>& OutMappingFromWeight) const override;
	//~ End - FDataflowVertexAttributeEditableNode API

	void GetWeightsFromActiveSnapshot(const FManagedArrayCollection& Collection, FName TargetGroupName, TArrayView<float> OutValues) const;
	void GetWeightsFromSnapshot(const FManagedArrayCollection& Collection, const FDataflowToolNodeSnapshot& Snapshot, FName TargetGroupName, TArrayView<float> OutValues) const;
	void StoreWeightsInSnapshot(const FManagedArrayCollection& Collection, TConstArrayView<float> InValues, FName TargetGroupName, FDataflowToolNodeSnapshot& OutSnapshot) const;

	/** use the input and stored weights and process final weights based on the selected override type */
	void ComputeFinalWeights(TConstArrayView<float> InputWeights, TConstArrayView<float> StoredWeights, TArrayView<float> FinalWeights) const;

	/** 
	* Compute weights to store from the final weights, input weights and the selected override type
	* Only processed the indices contained in WeightIndices, if non-empty
	*/
	void ComputeWeightsToStore(TConstArrayView<float> InputWeights, TConstArrayView<float> FinalWeights, TConstArrayView<int32> WeightIndices, TArrayView<float> WeightsToStore);


};

#undef UE_API
