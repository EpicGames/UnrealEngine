// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Dataflow/DataflowEngine.h"
#include "Dataflow/DataflowConnectionTypes.h"

#include "DataflowCollectionAttributeKeyNodes.generated.h"

class USkeletalMesh;


USTRUCT(meta = (Dataflow))
struct FMakeAttributeKeyDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMakeAttributeKeyDataflowNode, "MakeAttributeKey", "GeometryCollection", "")

public:

	UPROPERTY(EditAnywhere, Category = "Collection Key", meta = (DataflowInput, DisplayName = "Group"))
	FString GroupIn = "";

	UPROPERTY(EditAnywhere, Category = "Collection Key", meta = (DataflowInput, DisplayName = "Attribute"))
	FString AttributeIn = "";

	UPROPERTY(meta = (DataflowOutput, DisplayName = "AttributeKey"))
	FCollectionAttributeKey AttributeKeyOut;

	FMakeAttributeKeyDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&GroupIn);
		RegisterInputConnection(&AttributeIn);
		RegisterOutputConnection(&AttributeKeyOut);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

USTRUCT(meta = (DataflowGeometryCollection))
struct FBreakAttributeKeyDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FBreakAttributeKeyDataflowNode, "BreakAttributeKey", "Dataflow", "")

public:

	UPROPERTY(meta = (DataflowInput, DisplayName = "AttributeKey"))
	FCollectionAttributeKey AttributeKeyIn;

	UPROPERTY(EditAnywhere, Category = "Attribute Key", meta = (DataflowOutput, DisplayName = "Attribute"))
	FString AttributeOut = "";

	UPROPERTY(EditAnywhere, Category = "Attribute Key", meta = (DataflowOutput, DisplayName = "Group"))
	FString GroupOut = "";



	FBreakAttributeKeyDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&AttributeKeyIn);
		RegisterOutputConnection(&AttributeOut);
		RegisterOutputConnection(&GroupOut);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

namespace UE::Dataflow
{
	void DataflowCollectionAttributeKeyNodes();
}

