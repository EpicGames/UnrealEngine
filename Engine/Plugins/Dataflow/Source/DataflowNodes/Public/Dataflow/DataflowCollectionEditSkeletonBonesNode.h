// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "DataflowCollectionAddScalarVertexPropertyNode.h"
#include "DataflowPrimitiveNode.h"
#include "DataflowSkeletalMeshNodes.h"
#include "Dataflow/DataflowNode.h"
#include "GeometryCollection/ManagedArrayCollection.h"

#include "DataflowCollectionEditSkeletonBonesNode.generated.h"

#define UE_API DATAFLOWNODES_API

/** Edit skeleton bones properties. */
USTRUCT(Meta = (Experimental,DataflowCollection))
struct FDataflowCollectionEditSkeletonBonesNode : public FDataflowPrimitiveNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowCollectionEditSkeletonBonesNode, "EditSkeletonBones", "Collection", "Edit skeleton bones")

public:

	UE_API FDataflowCollectionEditSkeletonBonesNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

	UPROPERTY(Meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection"))
	FManagedArrayCollection Collection;
	
	/** Target group in which the attributes are stored */
	UPROPERTY(EditAnywhere, Category = "Vertex Attributes", meta = (DisplayName = "Vertex Group"))
	FScalarVertexPropertyGroup VertexGroup;

	/** Skeleton to be edited */
	UPROPERTY(EditAnywhere, Category = "Skeleton Binding", Meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Skeleton"))
	TObjectPtr<USkeleton> Skeleton = nullptr;

	/** Skeleton used to store the tool result */
	UPROPERTY()
	TObjectPtr<USkeleton> ToolSkeleton = nullptr;
	
	/** Validate the skeletal mesh construction */
	void ValidateSkeletalMeshes() {bValidSkeletalMeshes = true;}

	/** Delegate to transfer the bone selection to the tool*/
	FDataflowBoneSelectionChangedNotifyDelegate OnBoneSelectionChanged;
	
private:

	/** Update the tool skeleton */
	TObjectPtr<USkeleton> UpdateToolSkeleton(UE::Dataflow::FContext& Context);

	//~ Begin FDataflowPrimitiveNode interface
	virtual TArray<UE::Dataflow::FRenderingParameter> GetRenderParametersImpl() const override;
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	virtual void AddPrimitiveComponents(UE::Dataflow::FContext& Context, const TSharedPtr<const FManagedArrayCollection> RenderCollection,
		TObjectPtr<UObject> NodeOwner, TObjectPtr<AActor> RootActor, TArray<TObjectPtr<UPrimitiveComponent>>& PrimitiveComponents) override;
	virtual void OnInvalidate() override;
#if WITH_EDITOR
	virtual bool CanDebugDraw() const override { return true; }
	virtual bool CanDebugDrawViewMode(const FName& ViewModeName) const override;
	virtual void DebugDraw(UE::Dataflow::FContext& Context,
		IDataflowDebugDrawInterface& DataflowRenderingInterface,
		const FDebugDrawParameters& DebugDrawParameters) const override;
#endif
	//~ End FDataflowPrimitiveNode interface

	/** Transient skeletal mesh built from dataflow render collection */
	UPROPERTY(Transient)
	TArray<TObjectPtr<USkeletalMesh>> SkeletalMeshes;

	/** Valid skeletal mesh boolean to trigger the construction */
	bool bValidSkeletalMeshes = false;
};

#undef UE_API
