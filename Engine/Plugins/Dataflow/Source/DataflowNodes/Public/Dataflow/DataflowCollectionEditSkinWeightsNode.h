// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "DataflowCollectionAddScalarVertexPropertyNode.h"

#include "DataflowPrimitiveNode.h"
#include "DataflowSkeletalMeshNodes.h"
#include "Dataflow/DataflowCollectionAttributeKeyNodes.h"
#include "Dataflow/DataflowNode.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "GPUSkinPublicDefs.h"

#include "DataflowCollectionEditSkinWeightsNode.generated.h"

#define UE_API DATAFLOWNODES_API

class UDynamicMesh;

/** Dataflow skin weights data */
USTRUCT()
struct FDataflowSkinWeightData
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<float> BoneWeights;

	UPROPERTY()
	TArray<int32> BoneIndices;
};

/**
* FDataflowVertexSkinWeightData is a replacement for FDataflowSkinWeightData in FDataflowCollectionEditSkinWeightsNode
* This allow storing all the data linearly and enable bulk serialization for speed
* This solve the problem when we have million vertices meshes that was causing the serialization of the node to take a very long time
* this is now an order of magnitude faster that using FDataflowSkinWeightData
*/
USTRUCT()
struct FDataflowVertexSkinWeightData 
{
	GENERATED_BODY()

public:
	static constexpr const int32 MaxNumInflences = MAX_TOTAL_INFLUENCES;

	int32 Num() const;
	void SetNum(int32 VertexNum);
	void Reset();

	void FromArrays(int32 VertexIndex, const TArray<int32>& InIndices, const TArray<float>& InWeights);
	void ToArrays(int32 VertexIndex, TArray<int32>& OutIndices, TArray<float>& OutWeights) const;

	bool Serialize(FArchive& Ar);

	friend FArchive& operator<<(FArchive& Ar, FDataflowVertexSkinWeightData& Data);

private:
	TArray<int16> Bones;

	TArray<float> Weights;

	int32 ArrayNum = 0;
};

template<>
struct TStructOpsTypeTraits<FDataflowVertexSkinWeightData> : public TStructOpsTypeTraitsBase2<FDataflowVertexSkinWeightData>
{
	enum
	{
		WithSerializer = true,
	};
	static constexpr EPropertyObjectReferenceType WithSerializerObjectReferences = EPropertyObjectReferenceType::None;
};

/** Edit skin weights vertex properties. */
USTRUCT(Meta = (Experimental,DataflowCollection))
struct FDataflowCollectionEditSkinWeightsNode : public FDataflowPrimitiveNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowCollectionEditSkinWeightsNode, "EditSkinWeights", "Collection", "Edit skin weights and save it to collection")

public:

	UE_API FDataflowCollectionEditSkinWeightsNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

	UPROPERTY(Meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection"))
	FManagedArrayCollection Collection;

	/** The name to be set as a weight map attribute. */
	UPROPERTY(EditAnywhere, Category = "Vertex Attributes", meta = (DisplayName = "Bone Indices"))
	FString BoneIndicesName;

	/** The name to be set as a weight map attribute. */
	UPROPERTY(EditAnywhere, Category = "Vertex Attributes", meta = (DisplayName = "Bone Weights"))
	FString BoneWeightsName;

	/** Target group in which the attributes are stored */
	UPROPERTY(EditAnywhere, Category = "Vertex Attributes", meta = (DisplayName = "Vertex Group"))
	FScalarVertexPropertyGroup VertexGroup;

	/** Bone indices key to be used in other nodes if necessary */
	UPROPERTY(meta = (DisplayName = "Bone Indices", DataflowInput, DataflowOutput, DataflowPassthrough = "BoneIndicesKey"))
	FCollectionAttributeKey BoneIndicesKey;

	/** Bone weights key to be used in other nodes if necessary */
	UPROPERTY(meta = (DisplayName = "Bone Weights", DataflowInput, DataflowOutput, DataflowPassthrough = "BoneWeightsKey"))
	FCollectionAttributeKey BoneWeightsKey;

	/** Skeletal mesh to extract the skeleton from for the skinning */
	UPROPERTY(EditAnywhere, Category = "Skeleton Binding", meta = (DataflowInput))
	TObjectPtr<USkeletalMesh> SkeletalMesh = nullptr;

	/** Boolean to use a compressed format (FVector4f, FIntVector) to store the skin weights */
	UPROPERTY(EditAnywhere, Category = "Skeleton Binding")
	bool bCompressSkinWeights = false;

	/** List of skin weights */
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use VertexSkinWeights instead"))
	TArray<FDataflowSkinWeightData> SkinWeights_DEPRECATED;

	UPROPERTY()
	FDataflowVertexSkinWeightData VertexSkinWeights;
	
	/** 
	* Hash of the ref skeleton the stored skin weight are based on 
	* This allow invalidating the SkinWeights when the ref skeleton does not match anymore instead of having broken weights
	*/
	UPROPERTY()
	int32 SkinWeightsRefSkeletonHash = 0;

	/** Delegate to transfer the bone selection to the tool*/
	FDataflowBoneSelectionChangedNotifyDelegate OnBoneSelectionChanged;

	/** Report the vertex weights back onto the property ones */
	UE_API void ReportVertexWeights(UE::Dataflow::FContext& Context, const TArray<TArray<int32>>&SetupIndices, const TArray<TArray<float>>& SetupWeights, const TArray<TArray<int32>>& FinalIndices, const TArray<TArray<float>>& FinalWeights);

	/** Extract the vertex weights back from the property ones */
	UE_API void ExtractVertexWeights(UE::Dataflow::FContext& Context, const TArray<TArray<int32>>& SetupIndices, const TArray<TArray<float>>& SetupWeights, TArrayView<TArray<int32>> FinalIndices, TArrayView<TArray<float>> FinalWeights) const;

	/** Fill the weight attribute values from the collection */
	UE_API static bool FillAttributeWeights(const FManagedArrayCollection& SelectedCollection,
		const FCollectionAttributeKey& IndicesAttributeKey, const FCollectionAttributeKey& WeightsAttributeKey, TArray<TArray<int32>>& AttributeIndices, TArray<TArray<float>>& AttributeWeights);

	/** Get the weight attribute values and add them if not there*/
	UE_API static bool GetAttributeWeights(FManagedArrayCollection& SelectedCollection,
			const FCollectionAttributeKey& InBoneIndicesKey, const FCollectionAttributeKey& InBoneWeightsKey,
			TArray<TArray<int32>>& AttributeIndices, TArray<TArray<float>>& AttributeWeights, const bool bCanCompressSkinWeights);

	/** Set back the attribute values into the collection */
	UE_API static bool SetAttributeWeights(FManagedArrayCollection& SelectedCollection,
			const FCollectionAttributeKey& InBoneIndicesKey, const FCollectionAttributeKey& InBoneWeightsKey,
			const TArray<TArray<int32>>& AttributeIndices, const TArray<TArray<float>>& AttributeWeights);

	/** Get the weights attribute key to retrieve/set the weight values*/
	UE_API FCollectionAttributeKey GetBoneIndicesKey(UE::Dataflow::FContext& Context) const;

	/** Get the weights attribute key to retrieve/set the weight values*/
    UE_API FCollectionAttributeKey GetBoneWeightsKey(UE::Dataflow::FContext& Context) const;

	TRefCountPtr<FDataflowDebugDrawSkeletonObject> GetDebugDrawSkeleton() const { return SkeletonObject; }

	/** Validate the skeletal mesh construction */
	void ValidateSkeletalMeshes() {bValidSkeletalMeshes = true;}

	/** Return the collection offset given a skeletal mesh*/
	UE_API int32 GetSkeletalMeshOffset(const TObjectPtr<USkeletalMesh>& SkeletalMesh) const;

	class FEditNodeToolChange;
	UE_API TUniquePtr<class FToolCommandChange> MakeEditNodeToolChange();

private:

	//~ Begin FDataflowPrimitiveNode interface
	UE_API virtual TArray<UE::Dataflow::FRenderingParameter> GetRenderParametersImpl() const override;
	UE_API virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	UE_API virtual void OnPropertyChanged(UE::Dataflow::FContext& Context, const FPropertyChangedEvent& InPropertyChangedEvent) override;
	UE_API virtual void AddPrimitiveComponents(UE::Dataflow::FContext& Context, const TSharedPtr<const FManagedArrayCollection> RenderCollection,
		TObjectPtr<UObject> NodeOwner, TObjectPtr<AActor> RootActor, TArray<TObjectPtr<UPrimitiveComponent>>& PrimitiveComponents) override;
	UE_API virtual void OnInvalidate() override;
#if WITH_EDITOR
	virtual bool CanDebugDraw() const override { return true; }
	UE_API virtual bool CanDebugDrawViewMode(const FName& ViewModeName) const override;
	UE_API virtual void DebugDraw(UE::Dataflow::FContext& Context,
		IDataflowDebugDrawInterface& DataflowRenderingInterface,
		const FDebugDrawParameters& DebugDrawParameters) const override;
#endif
	//~ End FDataflowPrimitiveNode interface

	virtual void PostSerialize(const FArchive& Ar) override;

	int32 ComputeInputRefSkeletonHashFromInput(UE::Dataflow::FContext& Context) const;

	/** Transient skeletal mesh built from dataflow render collection */
	UPROPERTY(Transient)
	TArray<TObjectPtr<USkeletalMesh>> SkeletalMeshes;

	/** Valid skeletal mesh boolean to trigger the construction */
	bool bValidSkeletalMeshes = false;

	/** Transient dynamic meshes built from dataflow render collection */
	UPROPERTY(Transient)
	TArray<TObjectPtr<UDynamicMesh>> DynamicMeshes;

	/** 
	* not ideal but for now this is the only way we can update the displayed skeleton 
	* todo(dataflow) : find a better way to handle skeleton drawing and manipulation post 5.7
	*/
	mutable TRefCountPtr<FDataflowDebugDrawSkeletonObject> SkeletonObject;
};

#undef UE_API
