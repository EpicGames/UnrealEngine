// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Dataflow/DataflowNode.h"

#include "DataflowPrimitiveNode.generated.h"

/**
* FDataflowPrimitiveNode
*	Base class for nodes that could add a primitive component to the construction scene within the Dataflow graph. 
*/
USTRUCT(Meta = (Experimental))
struct FDataflowPrimitiveNode : public FDataflowNode
{
	GENERATED_BODY()

	FDataflowPrimitiveNode()
		: Super() { }

	FDataflowPrimitiveNode(const UE::Dataflow::FNodeParameters& Param, FGuid InGuid = FGuid::NewGuid())
		: Super(Param,InGuid) {
	}

	/** FDataflowNode interface */
	virtual ~FDataflowPrimitiveNode() = default;
	
	static FName StaticType() { return FName("FDataflowPrimitiveNode"); }

	virtual bool IsA(FName InType) const override 
	{ 
		return InType.ToString().Equals(StaticType().ToString()) 
			|| Super::IsA(InType); 
	}

	virtual bool HasRenderCollectionPrimitives() const override
	{
		return true;
	}

	/** Add primitive components to the construction scene */
	virtual void AddPrimitiveComponents(UE::Dataflow::FContext& Context, const TSharedPtr<const FManagedArrayCollection> RenderCollection, TObjectPtr<UObject> NodeOwner,
		TObjectPtr<AActor> RootActor, TArray<TObjectPtr<UPrimitiveComponent>>& PrimitiveComponents) {}
};


