// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Dataflow/DataflowCollectionAddScalarVertexPropertyNode.h"
#include "Dataflow/DataflowNode.h"
#include "Dataflow/DataflowToolNode.h"
#include "Dataflow/DataflowSelection.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "HAL/Platform.h"

#include "DataflowSelectionToolNode.generated.h"

#define UE_API DATAFLOWNODES_API

struct FDataflowSelectionToolNodeData
{
public:
	UE_API void DeselectVertices(TConstArrayView<int32> Vertices);
	UE_API void DeselectVertices(const TSet<int32>& Vertices);
	UE_API void DeselectAllVertices();
	UE_API void SelectVertices(TConstArrayView<int32> Vertices);
	UE_API void SelectVertices(const TSet<int32>& Vertices);

	UE_API void GetSelectedVertices(TSet<int32>& OutSelection) const;

	UE_API const FDataflowVertexSelection& GetVertexSelection() const;

private:
	void Init(const FManagedArrayCollection& InCollection);
	void LoadFromSnapshot(const FDataflowToolNodeSnapshot& InSnapshot);
	void SaveToSnapshot(FDataflowToolNodeSnapshot& OutSnapshot) const;

	friend struct FDataflowSelectionToolNode;
	FDataflowVertexSelection Selection;
};

/**
* Dataflow selection tool node
* When selected this node opens a selection tool to select vertex or face selection 
*/
USTRUCT(meta=(icon="LevelEditor.StrictBoxSelect"))
struct FDataflowSelectionToolNode : public FDataflowToolNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowSelectionToolNode, "DataflowSelectionToolNode", "Tools", "Vertex Face Collection")
	DATAFLOW_NODE_RENDER_TYPE("SurfaceRender", FGeometryCollection::StaticType(), "Collection")

public:
	FDataflowSelectionToolNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

	UE_API void LoadData(UE::Dataflow::FContext& Context, FDataflowSelectionToolNodeData& OutData) const;
	UE_API void SaveData(const FDataflowSelectionToolNodeData& InData);

private:
	/** Collection to use for selecting on */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = Collection))
	FManagedArrayCollection Collection;

	/** Output selection - this supports many types and will convert the internal vertex selection to the concrete type of the output */
	UPROPERTY(meta = (DataflowOutput))
	FDataflowSelectionTypes Selection;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

namespace UE::Dataflow
{
	void RegisterSelectionToolNode();
}

#undef UE_API
