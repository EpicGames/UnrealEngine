// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Dataflow/DataflowCore.h"
#include "Dataflow/DataflowDebugDraw.h"
#include "Dataflow/DataflowDebugDrawComponent.h"
#include "Dataflow/DataflowDebugDrawObject.h"
#include "Dataflow/DataflowEngine.h"

#include "DataflowSkeletalMeshNodes.generated.h"

#define UE_API DATAFLOWNODES_API

DEFINE_LOG_CATEGORY_STATIC(LogDataflowSkeletalMeshNodes, Log, All);

class USkeletalMesh;
class UPhysicsAsset;

// A delegate for monitoring dataflow skeleton selection changes.
DECLARE_MULTICAST_DELEGATE_OneParam(FDataflowBoneSelectionChangedNotifyDelegate, const TArray<FName>& /*InBoneNames*/);

/** Debug draw skeleton object to debug skeletal meshes */
struct FDataflowDebugDrawSkeletonObject : public FDataflowDebugDrawBaseObject
{
	FDataflowDebugDrawSkeletonObject(IDataflowDebugDrawInterface::FDataflowElementsType& InDataflowElements, const FReferenceSkeleton& InReferenceSkeleton, const bool bBonesVisible = true) :
		FDataflowDebugDrawBaseObject(InDataflowElements), ReferenceSkeleton(InReferenceSkeleton), bBonesVisible(bBonesVisible)
	{}

	/** Populate dataflow elements */
	UE_API virtual void PopulateDataflowElements() override;

	/** Debug draw dataflow element */
	UE_API virtual void DrawDataflowElements(FPrimitiveDrawInterface* PDI) override;

	/** Compute the dataflow elements bounding box */
	UE_API virtual FBox ComputeBoundingBox() const override;

	/** override the bone transforms */
	UE_API void OverrideBoneTransforms(const TArray<FTransform>& InBoneTransforms);

	UE_API const FReferenceSkeleton& GetReferenceSkeleton() const;

	/** Delegate to broadcast bones selection changes */
	FDataflowBoneSelectionChangedNotifyDelegate OnBoneSelectionChanged;

private :
	FCriticalSection TransformOverridesLockGuard;
	TArray<FTransform> TransformOverrides;

	/** Skeletal mesh to use to populate/draw scene elements */
	const FReferenceSkeleton ReferenceSkeleton;
		
	/** Previous element selection */
	TArray<bool> PreviousSelection;

	/** Boolean to make the bones visible or not */
	bool bBonesVisible = true;
};

USTRUCT()
struct FGetSkeletalMeshDataflowNode: public FDataflowNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FGetSkeletalMeshDataflowNode, "SkeletalMesh", "General", "Skeletal Mesh")

public:
	
	UPROPERTY(EditAnywhere, Category = "Dataflow", meta = (DataflowOutput, DisplayName = "SkeletalMesh"))
	TObjectPtr<const USkeletalMesh> SkeletalMesh = nullptr;

	UPROPERTY(EditAnywhere, Category = "Dataflow" )
	FName PropertyName = "SkeletalMesh";

	FGetSkeletalMeshDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterOutputConnection(&SkeletalMesh);
	}

	UE_API virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

private:
	UE_API virtual bool SupportsAssetProperty(UObject* Asset) const override;
	UE_API virtual void SetAssetProperty(UObject* Asset) override;
	
#if WITH_EDITOR
	virtual bool CanDebugDraw() const override { return true; }
	UE_API virtual bool CanDebugDrawViewMode(const FName& ViewModeName) const override;
	UE_API virtual void DebugDraw(UE::Dataflow::FContext& Context,
		IDataflowDebugDrawInterface& DataflowRenderingInterface,
		const FDebugDrawParameters& DebugDrawParameters) const override;
#endif
};

USTRUCT()
struct FGetSkeletonDataflowNode : public FDataflowNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FGetSkeletonDataflowNode, "Skeleton", "General", "Skeletal Mesh")

public:

	UPROPERTY(EditAnywhere, Category = "Dataflow", meta = (DataflowOutput, DisplayName = "Skeleton"))
	TObjectPtr<const USkeleton> Skeleton = nullptr;

	UPROPERTY(EditAnywhere, Category = "Dataflow")
	FName PropertyName = "Skeleton";

	FGetSkeletonDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterOutputConnection(&Skeleton);
	}

	UE_API virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

private:
	UE_API virtual bool SupportsAssetProperty(UObject* Asset) const override;
	UE_API virtual void SetAssetProperty(UObject* Asset) override;

#if WITH_EDITOR
	virtual bool CanDebugDraw() const override { return true; }
	UE_API virtual bool CanDebugDrawViewMode(const FName& ViewModeName) const override;
	UE_API virtual void DebugDraw(UE::Dataflow::FContext& Context,
		IDataflowDebugDrawInterface& DataflowRenderingInterface,
		const FDebugDrawParameters& DebugDrawParameters) const override;
#endif
};


USTRUCT()
struct FSkeletalMeshBoneDataflowNode : public FDataflowNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FSkeletalMeshBoneDataflowNode, "SkeletalMeshBone", "General", "Skeletal Mesh")

public:
	UPROPERTY(EditAnywhere, Category = "Dataflow")
	FName BoneName;

	UPROPERTY(meta = (DataflowInput, DisplayName = "SkeletalMesh"))
	TObjectPtr<const USkeletalMesh> SkeletalMesh = nullptr;

	UPROPERTY(meta = (DataflowOutput, DisplayName = "Index"))
	int BoneIndexOut = INDEX_NONE;

	UPROPERTY(EditAnywhere, Category = "Dataflow")
	FName PropertyName = "Overrides";

	FSkeletalMeshBoneDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&SkeletalMesh);
		RegisterOutputConnection(&BoneIndexOut);
	}


	UE_API virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};


USTRUCT()
struct FSkeletalMeshReferenceTransformDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FSkeletalMeshReferenceTransformDataflowNode, "SkeletalMeshReferenceTransform", "General", "Skeletal Mesh")

public:

	UPROPERTY(meta = (DataflowInput, DisplayName = "SkeletalMesh"))
	TObjectPtr<const USkeletalMesh> SkeletalMeshIn = nullptr;

	UPROPERTY(meta = (DataflowInput, DisplayName = "Index"))
	int32 BoneIndexIn = INDEX_NONE;

	UPROPERTY(meta = (DataflowOutput, DisplayName = "Transform"))
	FTransform TransformOut = FTransform::Identity;

	FSkeletalMeshReferenceTransformDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&SkeletalMeshIn);
		RegisterInputConnection(&BoneIndexIn);
		RegisterOutputConnection(&TransformOut);
	}


	UE_API virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/** Get the physics assets from input skeletal mesh. */
USTRUCT()
struct FGetPhysicsAssetFromSkeletalMeshDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FGetPhysicsAssetFromSkeletalMeshDataflowNode, "GetPhysicsAssetFromSkeletalMesh", "General", "Get Physics Asset Skeletal Mesh")

public:
	/** Input skeletal mesh */
	UPROPERTY(EditAnywhere, Category = "Skeletal Mesh", Meta = (DataflowInput))
	TObjectPtr<const USkeletalMesh> SkeletalMesh;

	/** Output physics asset */
	UPROPERTY(Meta = (DataflowOutput))
	TObjectPtr<const UPhysicsAsset> PhysicsAsset;

	FGetPhysicsAssetFromSkeletalMeshDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&SkeletalMesh);
		RegisterOutputConnection(&PhysicsAsset);
	}

private:
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

namespace UE::Dataflow
{
	void RegisterSkeletalMeshNodes();
}

#undef UE_API
