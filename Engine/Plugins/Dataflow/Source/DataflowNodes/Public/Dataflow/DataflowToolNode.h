// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Dataflow/DataflowNode.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "InteractiveToolChange.h"
#include "UObject/NameTypes.h"
#include "Delegates/IDelegateInstance.h"


#include "DataflowToolNode.generated.h"

#define UE_API DATAFLOWNODES_API

/**
* Represents data for tool node to store when the user press accept or explictly store a snapshot
*
*/
USTRUCT()
struct FDataflowToolNodeSnapshot
{
	GENERATED_BODY()
public:
	/** Returns the approximate amount of memory taken by the snapshot in bytes */
	uint64 GetAllocatedSize() const;

	const FDateTime& GetDate() const { return Date; }
	bool IsLocked() const { return bLocked; }

	/** Display name of the snapshot */
	UPROPERTY(EditAnywhere, Category = Snapshots)
	FString Name;

	/** description of the snapshot , not user editable */
	UPROPERTY(VisibleAnywhere, Category = Snapshots)
	FString Description;

	/** collection that stores the data of the snapshot */
	UPROPERTY()
	FManagedArrayCollection Data;

private:
	/** Date time at when the snapshot was created - Read-only */
	UPROPERTY(VisibleAnywhere, Category = Snapshots)
	FDateTime Date = FDateTime::Now();

	/** When true, this snapshot cannot be dropped if the maximum number of snapshot is reached and room needs to be made for a new one */
	UPROPERTY(EditAnywhere, Category = Snapshots)
	bool bLocked = false;
};

/**
* This represent a set of snapshots
*/
USTRUCT()
struct FDataflowToolNodeSnapshotSet
{
	GENERATED_BODY()
public:
	UE_API TConstArrayView<FDataflowToolNodeSnapshot> GetSnapshots() const;
	UE_API const FDataflowToolNodeSnapshot* GetActiveSnapshot() const;

	UE_API FDataflowToolNodeSnapshot& AddSnapshot(bool bNotify = false);
	UE_API void SetActiveSnapshot(int32 ActiveIndex, bool bNotify = false);
	UE_API bool RemoveSnapshot(int32 Index, bool bNotify = false);
	UE_API int32 FindOldestUnlockedSnapShot();

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnSnapshotSetChanged, FDataflowToolNodeSnapshotSet&);
	FOnSnapshotSetChanged OnSnapshotSetChanged;

private:
	void NotifyChanged();

	UPROPERTY(EditAnywhere, Category = Snapshots)
	int32 ActiveSnapshot = INDEX_NONE;

	UPROPERTY(EditAnywhere, Category = Snapshots, meta = (EditFixedSize, TitleProperty = "{Name} {Date}"))
	TArray<FDataflowToolNodeSnapshot> Snapshots;
};

/**
* Base class for a node that can bring up a tool 
*/
USTRUCT()
struct FDataflowToolNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()

	FDataflowToolNode() = default;

	UE_API FDataflowToolNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
	UE_API virtual ~FDataflowToolNode();

public:
	UE_API TConstArrayView<FDataflowToolNodeSnapshot> GetSnapshots() const;
	UE_API const FDataflowToolNodeSnapshot* GetActiveSnapshot() const;

protected:
	UE_API FDataflowToolNodeSnapshot& AddSnapshot();
	UE_API bool RemoveSnapshot(int32 Index);
	UE_API int32 FindOldestUnlockedSnapShot();

	class FSnapshotToolChange : public FToolCommandChange
	{
	public:
		UE_API FSnapshotToolChange(FDataflowToolNode& Node);

	protected:
		UE_API virtual FString ToString() const override;
		UE_API virtual void Apply(UObject* Object) override;
		UE_API virtual void Revert(UObject* Object) override;

		UE_API virtual void SwapApplyRevert(UObject* Object, FDataflowToolNode& Node);

	private:
		TWeakPtr<FDataflowToolNode> WeakNode;
		FDataflowToolNodeSnapshotSet SavedSnapshots;
	};

private:
	UE_API void SwapSnapshots(FDataflowToolNodeSnapshotSet& InOutSnapshots);

	void RegisterNotifyChanged();
	void UnregisterNotifyChanged();
	FDelegateHandle OnNotifyChangedHandle;
	void OnSnapshotSetChanged(FDataflowToolNodeSnapshotSet& Set);

	UPROPERTY(EditAnywhere, Category = Snapshots)
	FDataflowToolNodeSnapshotSet Snapshots;
};

#undef UE_API
