// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowToolNode.h"

#include "DataflowVertexAttributeEditableNode.generated.h"

#define UE_API DATAFLOWNODES_API

/**
* Base class for node using the vertex attribute paint tool
*/
USTRUCT()
struct FDataflowVertexAttributeEditableNode : public FDataflowToolNode
{
	GENERATED_USTRUCT_BODY()

	FDataflowVertexAttributeEditableNode() {};

	UE_API FDataflowVertexAttributeEditableNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

public:
	class FEditNodeToolChange;
	UE_API TUniquePtr<class FToolCommandChange> MakeEditNodeToolChange();

	/** Get the list of view mode supported by this node when being edited */
	virtual void GetSupportedViewModes(UE::Dataflow::FContext& Context, TArray<FName>& OutViewModeNames) const { ensure(false); }

	/** Get the absolute vertex attribute values */
	virtual void GetVertexAttributeValues(UE::Dataflow::FContext& Context, TArray<float>& OutValues) const { ensure(false); }

	/** Set the absolute vertex attribute values - InWeightIndices can be used to sepcify which indices are to be set */
	virtual void SetVertexAttributeValues(UE::Dataflow::FContext& Context, const TArray<float>& InValues, const TArray<int32>& InWeightIndices) { ensure(false); }

	/** Provide a extra indirection vertex mapping - optional will return empty by default */
	UE_API virtual void GetExtraVertexMapping(UE::Dataflow::FContext& Context, FName SelectedViewMode, TArray<int32>& OutMappingToWeight, TArray<TArray<int32>>& OutMappingFromWeight) const;
};

#undef UE_API
