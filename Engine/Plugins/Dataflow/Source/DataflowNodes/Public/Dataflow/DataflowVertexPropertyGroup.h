// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Dataflow/DataflowNode.h"
#include "GeometryCollection/GeometryCollection.h"
#include "UObject/NameTypes.h"

#include "DataflowVertexPropertyGroup.generated.h"

#define UE_API DATAFLOWNODES_API

/**
* interface for defining vertex property groups
* this is used by nodes to know how to access collection data from a specific vertex group
*/
class IDataflowAddScalarVertexPropertyCallbacks
{
public:
	using FAttributeKey = FDataflowNode::FAttributeKey;

	struct FTargetGroupInfo
	{
		FName TargetGroup;
		FAttributeKey PositionAttributeKey;
		FAttributeKey IndicesAttributeKey;
		FAttributeKey MappingFrom2DTo3DAttributeKey;
	};

	virtual ~IDataflowAddScalarVertexPropertyCallbacks() = default;
	virtual FName GetName() const = 0;
	virtual TArray<FName> GetTargetGroupNames() const = 0;
	virtual void GetTargetGroupInfos(TArray<FTargetGroupInfo>& OutInfos) const {};
	virtual TArray<UE::Dataflow::FRenderingParameter> GetRenderingParameters() const = 0;
};

/**
* Registry for implementations of IDataflowAddScalarVertexPropertyCallbacks
*/
class FDataflowAddScalarVertexPropertyCallbackRegistry
{
public:

	using FTargetGroupInfo = IDataflowAddScalarVertexPropertyCallbacks::FTargetGroupInfo;

	UE_API static FDataflowAddScalarVertexPropertyCallbackRegistry& Get();

	UE_API void RegisterCallbacks(TUniquePtr<IDataflowAddScalarVertexPropertyCallbacks>&& Callbacks);

	UE_API void DeregisterCallbacks(const FName& CallbacksName);

	UE_API TArray<FName> GetTargetGroupNames() const;

	UE_API FTargetGroupInfo GetTargetGroupInfo(FName TargetGroup) const;

	UE_API TArray<UE::Dataflow::FRenderingParameter> GetRenderingParameters() const;

	UE_API TArray<UE::Dataflow::FRenderingParameter> GetRenderingParameters(const FName& TargetGroup) const;

private:

	TMap<FName, TUniquePtr<IDataflowAddScalarVertexPropertyCallbacks>> AllCallbacks;
};

/*
* Custom type so that we can use property type customization
* Used by Dataflow nodes as a property type - displays as a combobox with the registered groups
*/
USTRUCT()
struct FScalarVertexPropertyGroup
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category = "Vertex Group")
	FName Name = FGeometryCollection::VerticesGroup;
};

#undef UE_API 
