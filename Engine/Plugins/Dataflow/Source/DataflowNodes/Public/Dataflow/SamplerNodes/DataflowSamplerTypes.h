// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowAnyType.h"
#include "StructUtils/InstancedStruct.h"
#include "Misc/TVariant.h"

#include "DataflowSamplerTypes.generated.h"

#define UE_API DATAFLOWNODES_API

/**
* A float sampler, sample at a specific 3D position  and return a single float value per position 
*/
USTRUCT()
struct FDataflowFloatSamplerBase
{
	GENERATED_BODY();

public:
	virtual ~FDataflowFloatSamplerBase() = default;
	virtual void Sample(TArrayView<const FVector3f> Positions, TArrayView<float> OutValues) const {};
	virtual FBox GetRenderBounds() const { return FBox(EForceInit::ForceInitToZero); };
};

USTRUCT()
struct FDataflowFloatSampler : public FDataflowFloatSamplerBase
{
	GENERATED_BODY()

public:
	FDataflowFloatSampler() = default;
	FDataflowFloatSampler(const FDataflowFloatSampler&) = default;
	FDataflowFloatSampler& operator=(const FDataflowFloatSampler&) = default;
	UE_API FDataflowFloatSampler(const TSharedPtr<const FDataflowFloatSamplerBase>& InImpl);

	UE_API virtual void Sample(TArrayView<const FVector3f> Positions, TArrayView<float> OutValues) const override;
	UE_API virtual FBox GetRenderBounds() const override;
	
	const TSharedPtr<const FDataflowFloatSamplerBase>& GetImpl() const { return Impl; }

private:
	TSharedPtr<const FDataflowFloatSamplerBase> Impl;
};

/////////////////////////////////////////////////////////////////////////////////////////

/**
* A vector sampler, sample at a specific 3D position  and return a FVector3f value per position
*/
USTRUCT()
struct FDataflowVectorSamplerBase
{
	GENERATED_BODY();
public:
	virtual ~FDataflowVectorSamplerBase() = default;
	virtual void Sample(TArrayView<const FVector3f> Positions, TArrayView<FVector3f> OutValues) const {};
	virtual FBox GetRenderBounds() const { return FBox(EForceInit::ForceInitToZero); };
};

USTRUCT()
struct FDataflowVectorSampler : public FDataflowVectorSamplerBase
{
	GENERATED_BODY()

public:
	FDataflowVectorSampler() = default;
	FDataflowVectorSampler(const FDataflowVectorSampler&) = default;
	FDataflowVectorSampler& operator=(const FDataflowVectorSampler&) = default;
	UE_API FDataflowVectorSampler(const TSharedPtr<const FDataflowVectorSamplerBase>& InImpl);

	UE_API virtual void Sample(TArrayView<const FVector3f> Positions, TArrayView<FVector3f> OutValues) const override;
	UE_API virtual FBox GetRenderBounds() const override;

	const TSharedPtr<const FDataflowVectorSamplerBase>& GetImpl() const { return Impl; }

private:
	TSharedPtr<const FDataflowVectorSamplerBase> Impl;
};

/////////////////////////////////////////////////////////////////////////////////////////

USTRUCT()
struct FDataflowSamplerTypesStorage
{
	GENERATED_BODY()

	FDataflowSamplerTypesStorage() = default;
	FDataflowSamplerTypesStorage(const FDataflowSamplerTypesStorage& Other) = default;
	FDataflowSamplerTypesStorage& operator=(const FDataflowSamplerTypesStorage& Other) = default;
	FDataflowSamplerTypesStorage(FDataflowSamplerTypesStorage&& Other) = default;
	FDataflowSamplerTypesStorage& operator=(FDataflowSamplerTypesStorage&& Other) = default;
	FDataflowSamplerTypesStorage(const FDataflowFloatSampler& Sampler)
		: Storage(TInPlaceType<FDataflowFloatSampler>(), Sampler)
	{
	}
	FDataflowSamplerTypesStorage(const FDataflowVectorSampler& Sampler)
		: Storage(TInPlaceType<FDataflowVectorSampler>(), Sampler)
	{
	}

	template<typename T>
	const T* TryGet() const
	{
		return Storage.TryGet<T>();
	}
	
private:
	// Currently we don't support serialization for this storage
	TVariant<FDataflowFloatSampler, FDataflowVectorSampler> Storage;
};

template <>
struct FDataflowConverter<FDataflowSamplerTypesStorage>
{
	template <typename TFromType>
	static void From(const TFromType& From, FDataflowSamplerTypesStorage& To)
	{
		To = TFromType(From);
	}
	template <typename TToType>
	static void To(const FDataflowSamplerTypesStorage& From, TToType& To)
	{
		if (const TToType* Ptr = From.TryGet<TToType>())
		{
			To = *Ptr;
		}
	}
};

/**
* Sampler types 
*/
UE_DATAFLOW_POLICY_DECLARE_TYPENAME(FDataflowFloatSampler)
UE_DATAFLOW_POLICY_DECLARE_TYPENAME(FDataflowVectorSampler)
UE_DATAFLOW_POLICY_DECLARE_TYPENAME(FDataflowSamplerTypesStorage)

struct FDataflowSamplerTypePolicy :
	public TDataflowMultiTypePolicy<FDataflowFloatSampler, FDataflowVectorSampler>
{
};

USTRUCT()
struct FDataflowSamplerTypes : public FDataflowAnyType
{
	using FPolicyType = FDataflowSamplerTypePolicy;
	using FStorageType = FDataflowSamplerTypesStorage;

	GENERATED_BODY()

	// This needs to be a UPROPERTY() because the anytype system needs to inspect it
	UPROPERTY()
	FDataflowSamplerTypesStorage Value;
};

namespace UE::Dataflow
{
	void RegisterSamplerTypes();
};

/////////////////////////////////////////////////////////////////////////////////////////

namespace UE::Dataflow
{
	void RegisterSamplerNodes();
}

#undef UE_API
