// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowCore.h"
#include "Dataflow/DataflowDebugDraw.h"
#include "Dataflow/DataflowDebugDrawComponent.h"
#include "Dataflow/DataflowEngine.h"
#include "Dataflow/DataflowFunctionProperty.h"

// this will have to be moved in the DymamicMesh module once fully implemented
#include "Dataflow/Transfer/TransferAttributes.h"

#include "DataflowMeshAttributesTransferNodes.generated.h"

#define UE_API DATAFLOWNODES_API

DEFINE_LOG_CATEGORY_STATIC(LogDataflowMeshAttributesTransferNode, Log, All);

class UDataflowMesh;
namespace UE::Geometry
{
	class FTransferAttributes;
}

/**
 * Abstract base USTRUCT for all attribute proxy descriptors used by FTransferMeshAttributesDataflowNode.
 * Subclass this to represent a specific attribute type (skin weights, morph targets, etc.) that should be transferred.
 */
USTRUCT(meta=(Abstract))
struct FDataflowAttributeProxy
{
	GENERATED_BODY()

	virtual ~FDataflowAttributeProxy() = default;
};

/**
 * Proxy descriptor representing the skeleton (bone hierarchy) to carry over during an attribute transfer.
 */
USTRUCT(meta=(DisplayName="Skeleton"))
struct FDataflowSkeletonProxy: public FDataflowAttributeProxy
{
	GENERATED_BODY()
};

/**
 * Controls whether an attribute proxy operates on all layers of a given type or only a specific named layer.
 */
UENUM()
enum class EProxySourceType : uint8
{
	/** Transfer every instance (layer) of this attribute type found on the source mesh. */
	All,
	/** Transfer only the single named attribute specified by Source, writing the result under the name Destination. */
	Specific
};

/**
 * Proxy descriptor that configures the transfer of a skin-weight attribute from a source mesh to a target mesh.
 * Add one of these to FTransferMeshAttributesDataflowNode::AttributeProxies to include skin weights in the transfer.
 */
USTRUCT(meta=(DisplayName="Skin Weights"))
struct FDataflowSkinWeightsProxy: public FDataflowAttributeProxy
{
	GENERATED_BODY()
	
	/** Determines whether all skin-weight layers are transferred or only the one named by Source. */
	UPROPERTY(EditAnywhere, Category = AttributeProxy)
	EProxySourceType SourceType = EProxySourceType::All;
	
	/** Name of the skin-weight attribute layer to read from the source mesh. */
	UPROPERTY(EditAnywhere, Category = AttributeProxy, meta = (EditCondition = "SourceType!=EProxySourceType::All"))
	FName Source = NAME_None;
	
	/** Name to assign the skin-weight attribute layer on the destination mesh. */
	UPROPERTY(EditAnywhere, Category = AttributeProxy, meta = (EditCondition = "SourceType!=EProxySourceType::All"))
	FName Destination = NAME_None;
};

/**
 * Proxy descriptor that configures the transfer of a morph-target delta attribute from a source mesh to a target mesh.
 * Add one of these to FTransferMeshAttributesDataflowNode::AttributeProxies to include morph targets in the transfer.
 */
USTRUCT(meta=(DisplayName="Morph Target"))
struct FDataflowMorphTargetProxy: public FDataflowAttributeProxy
{
	GENERATED_BODY()
	
	/** Determines whether all morph-target layers are transferred or only the one named by Source. */
	UPROPERTY(EditAnywhere, Category = AttributeProxy)
	EProxySourceType SourceType = EProxySourceType::All;
	
	/** Name of the morph-target attribute to read from the source mesh */
	UPROPERTY(EditAnywhere, Category = AttributeProxy, meta = (EditCondition = "SourceType!=EProxySourceType::All"))
	FName Source = NAME_None;
	
	/** Name to assign the morph-target attribute on the destination mesh. */
	UPROPERTY(EditAnywhere, Category = AttributeProxy, meta = (EditCondition = "SourceType!=EProxySourceType::All"))
	FName Destination = NAME_None;
};

/**
 * Proxy descriptor that configures the transfer of a polygroup layer from a source mesh to a target mesh.
 * Add one of these to FTransferMeshAttributesDataflowNode::AttributeProxies to include polygroups in the transfer.
 */
USTRUCT(meta=(DisplayName="Polygroup"))
struct FDataflowPolygroupProxy: public FDataflowAttributeProxy
{
	GENERATED_BODY()
	
	/** Determines whether all polygroup layers are transferred or only the one named by Source. */
	UPROPERTY(EditAnywhere, Category = AttributeProxy)
	EProxySourceType SourceType = EProxySourceType::All;
	
	/** Name of the polygroup layer to read from the source mesh. */
	UPROPERTY(EditAnywhere, Category = AttributeProxy, meta = (EditCondition = "SourceType!=EProxySourceType::All"))
	FName Source = NAME_None;
	
	/** Name to assign the polygroup layer on the destination mesh. */
	UPROPERTY(EditAnywhere, Category = AttributeProxy, meta = (EditCondition = "SourceType!=EProxySourceType::All"))
	FName Destination = NAME_None;
};

/**
 * Proxy descriptor that configures the transfer of a triangle-label attribute from a source mesh to a target mesh.
 * Add one of these to FTransferMeshAttributesDataflowNode::AttributeProxies to include triangle labels in the transfer.
 */
USTRUCT(meta=(DisplayName="Triangle Labels"))
struct FDataflowTriangleLabelsProxy: public FDataflowAttributeProxy
{
	GENERATED_BODY()
	
	/** Determines whether all triangle-label layers are transferred or only the one named by Source. */
	UPROPERTY(EditAnywhere, Category = AttributeProxy)
	EProxySourceType SourceType = EProxySourceType::All;
	
	/** Name of the triangle-label attribute to read from the source mesh. */
	UPROPERTY(EditAnywhere, Category = AttributeProxy, meta = (EditCondition = "SourceType!=EProxySourceType::All"))
	FName Source = NAME_None;
	
	/** Name to assign the triangle-label attribute on the destination mesh. */
	UPROPERTY(EditAnywhere, Category = AttributeProxy, meta = (EditCondition = "SourceType!=EProxySourceType::All"))
	FName Destination = NAME_None;
};

/**
 * Dataflow node that transfers mesh attributes (skin weights, morph targets, polygroups, triangle labels)
 * from SourceMesh onto Mesh using the UE::Geometry::FTransferAttributes operator.
 * Attributes to transfer are configured via the AttributeProxies instanced-struct array.
 *
 * @note This node is experimental and its interface may change without notice.
 */

USTRUCT(Meta = (Experimental))
struct FTransferMeshAttributesDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FTransferMeshAttributesDataflowNode, "TransferMeshAttributes", "General", "Transfer Mesh Attributes")

public:
	
	FTransferMeshAttributesDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

	/** The destination mesh modified in-place during evaluation. */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = Mesh, DataflowIntrinsic))
	TObjectPtr<UDataflowMesh> Mesh;
	
	/** The source mesh whose attributes are sampled during transfer. */
	UPROPERTY(EditAnywhere, Category = "SourceMesh", Meta = (DataflowInput))
	TObjectPtr<const UDataflowMesh> SourceMesh;

	/** Adds all available proxies for transfer. */
	UPROPERTY(EditAnywhere, Transient, SkipSerialization, Category=Attributes)
	FDataflowFunctionProperty All;
	
	/** Removes all proxies. */
	UPROPERTY(EditAnywhere, Transient, SkipSerialization, Category=Attributes)
	FDataflowFunctionProperty None;
	
	/** Attribute proxies descriptors that determines which attributes are transferred and how. */
	UPROPERTY(EditAnywhere, Category = Attributes, meta = (ExcludeBaseStruct, BaseStruct = "/Script/DataflowNodes.DataflowAttributeProxy"))
	TArray<FInstancedStruct> AttributeProxies;

private:
	
	/** Reads Mesh and SourceMesh from Context, transfer proxies and writes the modified mesh back as the node's output. */
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

	/**
	 * Constructs an FTransferAttributes operator from InSrcDynaMesh and InOutDstDynaMesh then invokes TransferAttributesToMesh.
	 * @param Context          Dataflow context, forwarded to BuildProxies for any context-dependent lookups.
	 * @param InSrcDynaMesh    Source dynamic mesh to sample attributes from.
	 * @param InOutDstDynaMesh Destination dynamic mesh to receive the transferred attributes.
	 */
	void TransferProxies(
		UE::Dataflow::FContext& Context,
		const UE::Geometry::FDynamicMesh3& InSrcDynaMesh,
		UE::Geometry::FDynamicMesh3& InOutDstDynaMesh) const;
	
	/**
	 * Converts the AttributeProxies array into concrete geometry proxy objects (FSkinWeightsProxy, FMorphTargetProxy, etc.).
	 * @param TransferAttributes  The operator to populate with proxy objects.
	 * @param SrcAttributes       Attribute set of the source dynamic mesh, used to look up source layers.
	 * @param DstAttributes       Attribute set of the destination dynamic mesh.
	 */
	void BuildProxies(
		UE::Geometry::FTransferAttributes& TransferAttributes,
		const UE::Geometry::FDynamicMeshAttributeSet* SrcAttributes,
		UE::Geometry::FDynamicMeshAttributeSet* DstAttributes) const;
	
	/** Adds all available proxies for transfer. */
	void AddAllAttributes(UE::Dataflow::FContext& Context);
	
	/** Removes all proxies. */
	void RemoveAllAttributes(UE::Dataflow::FContext& Context);
	
	/**
	 * Filters AttributeProxies and returns typed const pointers to all entries of type ProxyType.
	 *
	 * @tparam ProxyType  A concrete FDataflowAttributeProxy subtype to filter for.
	 * @return Array of non-owning const pointers to matching proxy entries in AttributeProxies.
	 */
	template<typename ProxyType>
	TArray<const ProxyType*> GetProxies() const
	{
		TArray<const ProxyType*> TypedProxies;
		for (const FInstancedStruct& Proxy: AttributeProxies)
		{
			if (const ProxyType* TypedProxy = Proxy.GetPtr<ProxyType>())
			{
				TypedProxies.Add(TypedProxy);
			}
		}
		return TypedProxies;
	} 
};


#undef UE_API
