// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GeometryTypes.h"
#include "UObject/NameTypes.h"
#include "TransformTypes.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/DynamicVertexSkinWeightsAttribute.h"
#include "Misc/TVariant.h"
#include "Polygroups/PolygroupUtil.h"

// Forward declarations
class FProgressCancel;

namespace UE::Geometry
{
	class FDynamicMesh3;
	template<typename MeshType> class TMeshAABBTree3;
	typedef TMeshAABBTree3<FDynamicMesh3> FDynamicMeshAABBTree3;
	class FMeshNormals;
}


namespace UE
{
namespace Geometry
{

/**
 * Stores the result of a closest-point query on a mesh surface. Used internally by FTransferAttributes to record which triangle a projected
 * point landed on and its position within that triangle.
 */
struct FClosestSample
{
	/** Index of the triangle that contains the closest point. Set to IndexConstants::InvalidID when no match was found. */
	int32 TriID = IndexConstants::InvalidID; 
	/** Barycentric coordinates of the closest point relative to the vertices of TriID. */
	FVector3d Bary = FVector3d::ZeroVector;
};
	
/**
 * Proxy that transfers a single named skin-weight attribute from a source FDynamicMesh3 to a target mesh.
 * Weights are blended using barycentric interpolation over the source triangle found at each target vertex.
 * Optionally remaps bone indices when the source and target skeletons differ.
 *
 * @experimental Available since UE 5.8; the API may change without notice.
 */
struct UE_EXPERIMENTAL(5.8, "Generic attribute transfer is experimental while under active development.") FSkinWeightsProxy
{
	/**
	 * Configuration options for FSkinWeightsProxy.
	 */
	struct FSkinWeightsProxyOptions
	{
		FSkinWeightsProxyOptions()
			: MaxNumInfluences(AnimationCore::MaxInlineBoneWeightCount)
			, bNormalizeToOne(true)
		{}

		/** Maximum number of bone influences kept per vertex after blending. Excess influences are pruned. */
		const int32 MaxNumInfluences = AnimationCore::MaxInlineBoneWeightCount;
		/**
		 * Maps each source bone index to its bone name.
		 * Must be provided (together with TargetBoneToIndex) when transferring across skeletons with different index layouts.
		 * May be empty when source and target share identical bone indices.
		 */
		TArray<FName> SourceIndexToBone;
		/**
		 * Maps each bone name to its index in the target skeleton.
		 * Must be provided (together with SourceIndexToBone) when transferring across skeletons with different index layouts.
		 * May be empty when source and target share identical bone indices.
		 */
		TMap<FName, FBoneIndexType> TargetBoneToIndex;
		/** If true, blended weights are renormalized to sum to 1 after interpolation and remapping. */
		bool bNormalizeToOne = true;
	};

	/**
	 * Constructs a proxy bound to the given source skin-weight attribute.
	 *
	 * @param InAttribute  Source attribute to read from. Must not be null.
	 * @param InDstName    Name of the attribute to write on the destination mesh.
	 * @param InOptions    Optional configuration controlling bone remapping and normalization.
	 */
	FSkinWeightsProxy(
		const FDynamicMeshVertexSkinWeightsAttribute* InAttribute,
		const FName InDstName,
		const FSkinWeightsProxyOptions& InOptions = FSkinWeightsProxyOptions())
		: SrcAttribute(InAttribute)
		, DstName(InDstName)
		, Options(InOptions)
	{
		check(SrcAttribute);
	}

	/**
	 * Looks up or creates the named skin-weight attribute on InDstMesh.
	 * If the attribute already exists it is reused; otherwise a new one is allocated and attached.
	 *
	 * @param InDstMesh  Destination mesh to query/modify.
	 * @return true on success, false if the attribute could not be created.
	 */
	bool GetOrCreateDestAttribute(FDynamicMesh3& InDstMesh)
	{
		FDynamicMeshAttributeSet* DstAttributes = InDstMesh.Attributes();
		DstAttribute = DstAttributes->GetSkinWeightsAttribute(DstName);
		if (!DstAttribute)
		{
			DstAttribute = new FDynamicMeshVertexSkinWeightsAttribute(&InDstMesh);
			DstAttributes->AttachSkinWeightsAttribute(DstName, DstAttribute);
		}
		return DstAttribute != nullptr;
	}

	/**
	 * Blends the skin weights of the three triangle vertices using BaryWeights, optionally remaps
	 * bone indices from the source skeleton to the target skeleton, and writes the result to VertexID.
	 * Must be called after GetOrCreateDestAttribute.
	 *
	 * @param VertexID     Target mesh vertex to write the blended weights to.
	 * @param Triangle     Indices of the three source mesh vertices forming the enclosing triangle.
	 * @param BaryWeights  Barycentric blend weights (sum to 1) for the three triangle vertices.
	 */
	void Transfer(const int32 VertexID, const FIndex3i& Triangle, const FVector3f& BaryWeights) const
	{
		using namespace AnimationCore;

		FBoneWeights Weight1, Weight2, Weight3;
		SrcAttribute->GetValue(Triangle[0], Weight1);
		SrcAttribute->GetValue(Triangle[1], Weight2);
		SrcAttribute->GetValue(Triangle[2], Weight3);

		FBoneWeightsSettings BlendSettings;
		BlendSettings.SetNormalizeType(Options.bNormalizeToOne ? EBoneWeightNormalizeType::Always : EBoneWeightNormalizeType::None);
		BlendSettings.SetBlendZeroInfluence(true);
		BlendSettings.SetMaxWeightCount(Options.MaxNumInfluences);

		FBoneWeights Weights = FBoneWeights::Blend(Weight1, Weight2, Weight3, BaryWeights[0], BaryWeights[1], BaryWeights[2], BlendSettings);

		// TODO: Blend method can potentially skip applying renormalization and prunning weights to match MaxNumInfluences
		// using the BlendSettings so force renormalization. Remove this once FBoneWeights::Blend is fixed.
		Weights.Renormalize(BlendSettings);

		// Check if we need to remap the indices
		if (!Options.SourceIndexToBone.IsEmpty() && !Options.TargetBoneToIndex.IsEmpty())
		{
			FBoneWeightsSettings BoneSettings;
			BoneSettings.SetNormalizeType(EBoneWeightNormalizeType::None);

			FBoneWeights MappedWeights;

			for (int32 WeightIdx = 0; WeightIdx < Weights.Num(); ++WeightIdx)
			{
				const FBoneWeight& BoneWeight = Weights[WeightIdx];

				FBoneIndexType FromIdx = BoneWeight.GetBoneIndex();
				uint16 FromWeight = BoneWeight.GetRawWeight();

				checkSlow(FromIdx < Options.SourceIndexToBone.Num());
				if (FromIdx < Options.SourceIndexToBone.Num())
				{
					FName BoneName = Options.SourceIndexToBone[FromIdx];
					if (Options.TargetBoneToIndex.Contains(BoneName))
					{
						FBoneIndexType ToIdx = Options.TargetBoneToIndex[BoneName];
						FBoneWeight MappedBoneWeight(ToIdx, FromWeight);
						MappedWeights.SetBoneWeight(MappedBoneWeight, BoneSettings);
					}
					else
					{
						UE_LOGF(LogGeometry, Error, "FTransferBoneWeights: Bone name %ls does not exist in the target mesh.", *BoneName.ToString());
					}
				}
			}

			if (MappedWeights.Num() == 0)
			{
				// If no bone mappings were found, add a single entry for the root bone
				MappedWeights.SetBoneWeight(FBoneWeight(0, 1.0f), FBoneWeightsSettings());
			}
			else if (Weights.Num() != MappedWeights.Num() && Options.bNormalizeToOne)
			{
				// In case some of the bones were not mapped we need to renormalize
				MappedWeights.Renormalize(FBoneWeightsSettings());
			}

			Weights = MappedWeights;
		}

		DstAttribute->SetValue(VertexID, Weights);
	}

	/** Read-only pointer to the source skin-weight attribute. */
	const FDynamicMeshVertexSkinWeightsAttribute* SrcAttribute = nullptr;
	/** Pointer to the destination attribute on the target mesh; populated by GetOrCreateDestAttribute. */
	FDynamicMeshVertexSkinWeightsAttribute* DstAttribute = nullptr;
	/** Name used to look up or create the destination attribute on the target mesh. */
	FName DstName = NAME_None;
	/** Options supplied at construction time. */
	FSkinWeightsProxyOptions Options;
};

/**
 * Proxy that transfers a single named morph-target delta attribute from a source FDynamicMesh3 to a target mesh.
 * Per-vertex position deltas are blended via barycentric interpolation over the enclosing source triangle.
 *
 * @experimental Available since UE 5.8; the API may change without notice.
 */
struct UE_EXPERIMENTAL(5.8, "Generic attribute transfer is experimental while under active development.") FMorphTargetProxy
{
	/**
	 * Constructs a proxy bound to the given source morph-target attribute.
	 *
	 * @param InAttribute  Source attribute to read from. Must not be null.
	 * @param InDstName    Name of the morph-target attribute to write on the destination mesh.
	 */
	FMorphTargetProxy(const FDynamicMeshMorphTargetAttribute* InAttribute, const FName InDstName)
		: SrcAttribute(InAttribute)
		, DstName(InDstName)
	{
		check(SrcAttribute);
	}

	/**
	 * Looks up or creates the named morph-target attribute on InDstMesh.
	 *
	 * @param InDstMesh  Destination mesh to query/modify.
	 * @return true on success, false if the attribute could not be created.
	 */
	bool GetOrCreateDestAttribute(FDynamicMesh3& InDstMesh)
	{
		FDynamicMeshAttributeSet* DstAttributes = InDstMesh.Attributes();
		DstAttribute = DstAttributes->GetMorphTargetAttribute(DstName);
		if (!DstAttribute)
		{
			DstAttribute = new FDynamicMeshMorphTargetAttribute(&InDstMesh);
			DstAttributes->AttachMorphTargetAttribute(DstName, DstAttribute);
		}
		return DstAttribute != nullptr;
	}

	/**
	 * Linearly interpolates the morph-target position deltas of the three triangle vertices using BaryWeights
	 * and writes the resulting delta to VertexID on the destination attribute.
	 * Must be called after GetOrCreateDestAttribute.
	 *
	 * @param VertexID     Target mesh vertex to write the interpolated delta to.
	 * @param Triangle     Indices of the three source mesh vertices forming the enclosing triangle.
	 * @param BaryWeights  Barycentric blend weights (sum to 1) for the three triangle vertices.
	 */
	void Transfer(const int32 VertexID, const FIndex3i& Triangle, const FVector3f& BaryWeights) const
	{
		FVector3f Delta1, Delta2, Delta3;
		SrcAttribute->GetValue(Triangle[0], Delta1);
		SrcAttribute->GetValue(Triangle[1], Delta2);
		SrcAttribute->GetValue(Triangle[2], Delta3);

		const float Alpha = BaryWeights[0], Beta = BaryWeights[1], Theta  = BaryWeights[2];
		const FVector3f Delta(
			Alpha * Delta1[0] + Beta * Delta2[0] + Theta * Delta3[0],
			Alpha * Delta1[1] + Beta * Delta2[1] + Theta * Delta3[1],
			Alpha * Delta1[2] + Beta * Delta2[2] + Theta * Delta3[2]);

		DstAttribute->SetValue(VertexID, Delta);
	}

	/** Read-only pointer to the source morph-target attribute. */
	const FDynamicMeshMorphTargetAttribute* SrcAttribute = nullptr;
	/** Pointer to the destination attribute on the target mesh; populated by GetOrCreateDestAttribute. */
	FDynamicMeshMorphTargetAttribute* DstAttribute = nullptr;

	/** Name used to look up or create the destination attribute on the target mesh. */
	FName DstName = NAME_None;
};

/**
 * Variant holding any per-vertex attribute proxy.
 * FTransferAttributes stores an array of these to dispatch transfers across different attribute types without virtual dispatch.
 */
using FVertexProxy = TVariant<
	FSkinWeightsProxy,
	FMorphTargetProxy
>;

/**
 * Proxy that copies a single named polygroup layer from source triangles to matching target triangles.
 * Polygroup values are discrete integer labels, so no interpolation is performed; the value of the
 * closest source triangle is assigned directly.
 *
 * @experimental Available since UE 5.8; the API may change without notice.
 */
struct UE_EXPERIMENTAL(5.8, "Generic attribute transfer is experimental while under active development.") FPolygroupProxy
{
	/**
	 * Constructs a proxy bound to the given source polygroup attribute.
	 *
	 * @param InAttribute  Source attribute to read from. Must not be null.
	 * @param InDstName    Name of the polygroup layer to write on the destination mesh.
	 */
	FPolygroupProxy(const FDynamicMeshPolygroupAttribute* InAttribute, const FName InDstName)
		: SrcAttribute(InAttribute)
		, DstName(InDstName)
	{
		check(SrcAttribute);
	}

	/**
	 * Finds the named polygroup layer on InDstMesh, or appends a new layer if it does not exist.
	 *
	 * @param InDstMesh  Destination mesh to query/modify.
	 * @return true on success, false if the layer could not be found or created.
	 */
	bool GetOrCreateDestAttribute(FDynamicMesh3& InDstMesh)
	{
		FDynamicMeshAttributeSet* DstAttributes = InDstMesh.Attributes();
		DstAttribute = FindPolygroupLayerByName(InDstMesh, DstName);
		if (!DstAttribute)
		{
			int32 TargetLayerIdx = DstAttributes->NumPolygroupLayers();
			DstAttributes->SetNumPolygroupLayers(TargetLayerIdx + 1);
			DstAttribute = DstAttributes->GetPolygroupLayer(TargetLayerIdx);
			DstAttribute->SetName(DstName);
		}
		return DstAttribute != nullptr;
	}

	/**
	 * Copies the polygroup value from SrcTriangleID on the source attribute to DstTriangleID on the
	 * destination attribute. Must be called after GetOrCreateDestAttribute.
	 *
	 * @param SrcTriangleID  Triangle index on the source mesh to read the group value from.
	 * @param DstTriangleID  Triangle index on the destination mesh to write the group value to.
	 */
	void Transfer(const int32 SrcTriangleID, const int32 DstTriangleID) const
	{
		DstAttribute->SetValue(DstTriangleID, SrcAttribute->GetValue(SrcTriangleID));
	}

	/** Read-only pointer to the source polygroup attribute. */
	const FDynamicMeshPolygroupAttribute* SrcAttribute = nullptr;
	/** Pointer to the destination attribute on the target mesh; populated by GetOrCreateDestAttribute. */
	FDynamicMeshPolygroupAttribute* DstAttribute = nullptr;
	/** Name used to look up or create the destination polygroup layer on the target mesh. */
	FName DstName = NAME_None;
};

/**
 * Proxy that copies a single named triangle-label attribute from source triangles to matching target triangles.
 * Labels are discrete values so no interpolation is performed.
 *
 * @experimental Available since UE 5.8; the API may change without notice.
 */
struct UE_EXPERIMENTAL(5.8, "Generic attribute transfer is experimental while under active development.") FTriangleLabelProxy
{
	/**
	 * Constructs a proxy bound to the given source triangle-label attribute.
	 *
	 * @param InAttribute  Source attribute to read from. Must not be null.
	 * @param InDstName    Name of the triangle-label attribute to write on the destination mesh.
	 */
	FTriangleLabelProxy(const FDynamicMeshTriangleLabelAttribute* InAttribute, const FName InDstName)
		: SrcAttribute(InAttribute)
		, DstName(InDstName)
	{
		check(SrcAttribute);
	}

	/**
	 * Looks up or creates the named triangle-label attribute on InDstMesh.
	 *
	 * @param InDstMesh  Destination mesh to query/modify.
	 * @return true on success, false if the attribute could not be created.
	 */
	bool GetOrCreateDestAttribute(FDynamicMesh3& InDstMesh)
	{
		FDynamicMeshAttributeSet* DstAttributes = InDstMesh.Attributes();
		DstAttribute = DstAttributes->FindTriangleLabelAttribute(DstName);
		if (!DstAttribute)
		{
			DstAttribute = new FDynamicMeshTriangleLabelAttribute(&InDstMesh);
			DstAttributes->AttachTriangleLabelAttribute(DstName, DstAttribute);
		}
		return DstAttribute != nullptr;
	}

	/**
	 * Copies the label value from SrcTriangleID on the source attribute to DstTriangleID on the
	 * destination attribute. Must be called after GetOrCreateDestAttribute.
	 *
	 * @param SrcTriangleID  Triangle index on the source mesh to read the label from.
	 * @param DstTriangleID  Triangle index on the destination mesh to write the label to.
	 */
	void Transfer(const int32 SrcTriangleID, const int32 DstTriangleID) const
	{
		DstAttribute->SetValue(DstTriangleID, SrcAttribute->GetValue(SrcTriangleID));
	}

	/** Read-only pointer to the source triangle-label attribute. */
	const FDynamicMeshTriangleLabelAttribute* SrcAttribute = nullptr;
	/** Pointer to the destination attribute on the target mesh; populated by GetOrCreateDestAttribute. */
	FDynamicMeshTriangleLabelAttribute* DstAttribute = nullptr;
	/** Name used to look up or create the destination triangle-label attribute on the target mesh. */
	FName DstName = NAME_None;
};

/**
 * Variant holding any per-triangle attribute proxy.
 * FTransferAttributes stores an array of these to dispatch transfers across different attribute types without virtual dispatch.
 */
using FTriangleProxy = TVariant<
	FPolygroupProxy,
	FTriangleLabelProxy
>;
	
/**
 * Transfers vertex and/or triangle attributes from one mesh (source) to another (target).
 *
 * This class generalizes FTransferBoneWeights to support arbitrary attribute types including skin weights, morph targets, polygroups,
 * and triangle labels.
 * Attributes to transfer are registered as proxy objects (FVertexProxy / FTriangleProxy) before calling TransferAttributesToMesh.
 *
 * The current implementation uses the ClosestPointOnSurface algorithm. An Inpaint fallback is planned but not yet exposed.
 * Most configuration members are private and will be progressively exposed as the API matures.
 *
 * @experimental Available since UE 5.8; the API may change without notice.
 */

class UE_EXPERIMENTAL(5.8, "Generic attribute transfer is experimental while under active development.") FTransferAttributes
{
public:
	/**
	 * Constructs the operator with a mandatory source mesh and an optional pre-built BVH.
	 * Providing a BVH is an optimization when the same source mesh is used for multiple targets.
	 *
	 * @param InSourceMesh  The mesh whose attributes will be sampled. Must outlive this object.
	 * @param InSourceBVH   Optional pre-built spatial acceleration structure for InSourceMesh.
	 *                      If null, an internal BVH is constructed on the first transfer call.
	 */
	FTransferAttributes(const FDynamicMesh3* InSourceMesh, const FDynamicMeshAABBTree3* InSourceBVH = nullptr);
	
	virtual ~FTransferAttributes();
	
	/**
	 * Validates that all required inputs are set before running the transfer.
	 * @return EOperationValidationResult::Ok if the operator is ready to run.
	 */
	virtual EOperationValidationResult Validate();
	
	/**
	 * Executes the configured attribute transfer, writing results into InOutTargetMesh.
	 * All registered vertex and triangle proxies are applied in order.
	 *
	 * @param InOutTargetMesh  The mesh to receive the transferred attributes.
	 * @return true on success, false if the operation was cancelled or an error occurred.
	 */
	virtual bool TransferAttributesToMesh(FDynamicMesh3& InOutTargetMesh);

	/**
	 * Constructs a vertex proxy of type T in-place at the end of VertexProxies.
	 * @tparam T        One of the types contained in FVertexProxy (e.g. FSkinWeightsProxy).
	 * @param Args      Constructor arguments forwarded to T.
	 */
	template<typename T, typename... ArgTypes>
	void AddVertexProxy(ArgTypes&&... Args)
	{
		VertexProxies.Emplace(FVertexProxy(TInPlaceType<T>(), Forward<ArgTypes>(Args)...));
	}
	
	/**
	 * Constructs a triangle proxy of type T in-place at the end of TriangleProxies.
	 *
	 * @tparam T        One of the types contained in FTriangleProxy (e.g. FPolygroupProxy).
	 * @param Args      Constructor arguments forwarded to T.
	 */
	template<typename T, typename... ArgTypes>
	void AddTriangleProxy(ArgTypes&&... Args)
	{
		TriangleProxies.Emplace(FTriangleProxy(TInPlaceType<T>(), Forward<ArgTypes>(Args)...));
	}
	
private:
	
	/** Set this to be able to cancel the running operation. */
	FProgressCancel* Progress = nullptr;

	/** Enable/disable multi-threading. */
	bool bUseParallel = true;

	/** Transform applied to the input target mesh or target point before transfer. */
	FTransformSRT3d TargetToWorld = FTransformSRT3d::Identity();
	
	enum class ETransferMethod : uint8
	{
        // For every vertex on the target mesh, find the closest point on the surface of the source mesh. If that point 
        // is within the SearchRadius, and their normals differ by less than the NormalThreshold, then we directly copy  
        // the weights from the source point to the target mesh vertex.
		ClosestPointOnSurface = 0,

        // Same as the ClosestPointOnSurface but for all the vertices we didn't copy the weights directly, automatically 
		// compute the smooth weights.
		Inpaint = 1
	};
	
	/** The transfer method to transfer data. */
	ETransferMethod TransferMethod = ETransferMethod::ClosestPointOnSurface;

	/**  Radius for searching the closest point. If negative, all points are considered. */
	double SearchRadius = -1;

	/** 
	 * Maximum angle (in radians) difference between target and source point normals to be considered a match. 
	 * If negative, normals are ignored.
	 */
	double NormalThreshold = -1;
	
	/** 
	 * If true, when the closest point doesn't pass the normal threshold test, will try again with a flipped normal. 
	 * This helps with layered meshes where the "inner" and "outer" layers are close to each other but whose normals 
	 * are pointing in the opposite directions. 
	 */
	bool LayeredMeshSupport = false;

	/** The number of optional post-processing smoothing iterations applied to the vertices without the match. */
	int32 NumSmoothingIterations = 0; 

	/** The strength of each post-processing smoothing iteration. */
	float SmoothingStrength = 0.0f;

	/** If true, will use the intrinsic Delaunay mesh to construct sparse Cotangent Laplacian matrix. */
	bool bUseIntrinsicLaplacian = false;

	/** 
	 * Optional mask where if ForceInpaint[VertexID] != 0 we want to force the colors for the vertex to be computed  
	 * automatically.
	 * 
	 * @note Only used when TransferMethod == ETransferMethod::Inpaint.
	 * 		 The size must be equal to the InTargetMesh.MaxVertexID(), otherwise the mask is ignored.
	 */
	TArray<float> ForceInpaint;

	/** 
	 * Optional subset of target mesh vertices to transfer weights to.
	 * If left empty, skin weights will be transferred to all target mesh vertices.
	 */
	TArray<int32> TargetVerticesSubset;

	/** Ordered list of vertex-level attribute proxies (skin weights, morph targets) to be applied during transfer. */
	TArray<FVertexProxy> VertexProxies;
	
	/**
	 * Optional subset of target mesh triangles to consider during triangle-level attribute transfer.
	 * If empty, all triangles in the target mesh are processed.
	 */
	TArray<int32> TargetTrianglesSubset;
	
	/** Ordered list of triangle-level attribute proxies (polygroups, triangle labels) to be applied during transfer. */
	TArray<FTriangleProxy> TriangleProxies;
	
	/** Output bitmask: MatchedVertices[VertexID] is true for every target VERTICES whose closest source triangle was found. */
	TBitArray<> MatchedVertices;
	
	/** Output bitmask: MatchedTriangles[TriangleID] is true for every target triangle whose closest source triangle was found. */
	TBitArray<> MatchedTriangles;

	/** Source mesh we are transferring colors from. */
	const FDynamicMesh3* SourceMesh = nullptr;
	
	/** 
	 * The caller can optionally specify the source mesh BVH in case this operator is run on multiple target meshes 
	 * while the source mesh remains the same. Otherwise BVH tree will be computed.
	 */
	const FDynamicMeshAABBTree3* SourceBVH = nullptr;
	
	/** If the caller doesn't pass BVH for the source mesh then we compute one. */
	TUniquePtr<FDynamicMeshAABBTree3> InternalSourceBVH;

	/** If the source mesh doesn't have per-vertex normals then compute them */
	TUniquePtr<FMeshNormals> InternalSourceMeshNormals;

	/**
	 * Returns the closest point on the source mesh surface to InPoint, filtered by InNormal when NormalThreshold >= 0.
	 * The result triangle ID is set to InvalidID when no match satisfies the configured constraints.
	 *
	 * @param InPoint   Query point in world space.
	 * @param InNormal  Query normal used for the normal-angle filter. Pass FVector3f::Zero() to skip normal filtering.
	 * @return FClosestSample with the enclosing triangle ID and barycentric coordinates, or InvalidID if no match.
	 */
	FClosestSample GetClosestSample(const FVector3d& InPoint, const FVector3f& InNormal = FVector3f::Zero()) const;
	
    /** @return if true, abort the computation. */
	bool Cancelled();
	
	/**
	 * Find the closest point on the surface of the source mesh and return the ID of the triangle containing it and its
	 * barycentric coordinates.
	 *
	 * @return true if point is found, false otherwise
	 */
	bool FindClosestPointOnSourceSurface(const FVector3d& InPoint, const FTransformSRT3d& InToWorld, int32& OutTriID, FVector3d& OutBary) const;
	
	/**
	 * Drives the full closest-point transfer: processes both vertex proxies and triangle proxies.
	 * Called internally by TransferAttributesToMesh when TransferMethod == ClosestPointOnSurface.
	 *
	 * @param InOutTargetMesh          Mesh being modified.
	 * @param InTargetMeshNormals      Per-vertex normals for the target mesh, used for normal-angle filtering.
	 * @return false if the operation was cancelled, true otherwise.
	 */
	bool TransferUsingClosestPoint(FDynamicMesh3& InOutTargetMesh, const TUniquePtr<FMeshNormals>& InTargetMeshNormals);

	/**
	 * Iterates over target vertices (or TargetVerticesSubset), finds the closest source surface point for each,
	 * and dispatches to all registered vertex proxies.
	 *
	 * @param InOutTargetMesh          Mesh being modified; vertex attributes are written here.
	 * @param InTargetMeshNormals      Per-vertex normals for the target mesh used in normal filtering.
	 * @return Number of target vertices that were successfully matched to a source surface point.
	 */
	int32 TransferVerticesUsingClosestPoint(FDynamicMesh3& InOutTargetMesh, const TUniquePtr<FMeshNormals>& InTargetMeshNormals);
	
	/**
	 * Iterates over target triangles (or TargetTrianglesSubset), finds the closest source triangle for each,
	 * and dispatches to all registered triangle proxies.
	 *
	 * @param InOutTargetMesh  Mesh being modified; triangle attributes are written here.
	 * @return Number of target triangles that were successfully matched to a source triangle.
	 */
	int32 TransferTrianglesUsingClosestPoint(FDynamicMesh3& InOutTargetMesh);
};
	

} // end namespace UE::Geometry
} // end namespace UE
